// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class GraphEditor : ModuleRules
{
	public GraphEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PublicIncludePathModuleNames.AddRange(
            new string[] {                
				"ClassViewer",
				"StructViewer",
				"MaterialEditor"
			}
        );

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "AppFramework",
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
                
				"EditorWidgets",
				"EditorFramework",
				"UnrealEd",
				"AssetRegistry",
				"Kismet",
				"KismetCompiler",
				"KismetWidgets",
				"BlueprintGraph",
				"Documentation",
				"Persona",
				"PropertyEditor",
				"RenderCore",
				"RHI",
				"ToolMenus",
				"ToolWidgets",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"ContentBrowser",
				"ClassViewer",
				"StructViewer",
			}
		);

		// Circular references that need to be cleaned up
		CircularlyReferencedDependentModules.AddRange(
		   [
				"Kismet",
				"Persona",
		   ]
	   );
	}
}
