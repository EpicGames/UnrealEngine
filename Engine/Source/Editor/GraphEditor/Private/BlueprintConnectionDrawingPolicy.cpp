// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintConnectionDrawingPolicy.h"

#include "BlueprintEditorSettings.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "K2Node_Knot.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Layout/ArrangedWidget.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "SGraphPin.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FArrangedChildren;
class FSlateRect;
class SWidget;

/////////////////////////////////////////////////////
// FKismetConnectionDrawingPolicy

FKismetConnectionDrawingPolicy::FKismetConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, GraphObj(InGraphObj)
{
	// Don't want to draw ending arrowheads
	ArrowImage = nullptr;
	ArrowRadius = FVector2f::ZeroVector;

	// But we do want to draw midpoint arrowheads
	if (GetDefault<UBlueprintEditorSettings>()->bDrawMidpointArrowsInBlueprints)
	{
		MidpointImage = FAppStyle::GetBrush( TEXT("Graph.Arrow") );
		MidpointRadius = MidpointImage->ImageSize * ZoomFactor * 0.5f;
	}

	// Cache off the editor options
	AttackColor = Settings->TraceAttackColor;
	SustainColor = Settings->TraceSustainColor;
	ReleaseColor = Settings->TraceReleaseColor;

	AttackWireThickness = Settings->TraceAttackWireThickness;
	SustainWireThickness = Settings->TraceSustainWireThickness;
	ReleaseWireThickness = Settings->TraceReleaseWireThickness;
	DefaultDataWireThickness = Settings->DefaultDataWireThickness;
	DefaultExecutionWireThickness = Settings->DefaultExecutionWireThickness;

	TracePositionBonusPeriod = Settings->TracePositionBonusPeriod;
	TracePositionExponent = Settings->TracePositionExponent;
	AttackHoldPeriod = Settings->TraceAttackHoldPeriod;
	DecayPeriod = Settings->TraceDecayPeriod;
	DecayExponent = Settings->TraceDecayExponent;
	SustainHoldPeriod = Settings->TraceSustainHoldPeriod;
	ReleasePeriod = Settings->TraceReleasePeriod;
	ReleaseExponent = Settings->TraceReleaseExponent;

	CurrentTime = 0.0;
	LatestTimeDiscovered = 0.0;
}

void FKismetConnectionDrawingPolicy::Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	// Build the execution roadmap (also populates execution times)
	BuildExecutionRoadmap();

	// Draw everything
	FConnectionDrawingPolicy::Draw(InPinGeometries, ArrangedNodes);
}

UBlueprint* FKismetConnectionDrawingPolicy::GetTargetBlueprint() const
{
	// Start out with the current graph Blueprint context.
	if (UBlueprint* TargetBP = FBlueprintEditorUtils::FindBlueprintForGraph(GraphObj))
	{
		// Macro library Blueprints have no associated "active" object debugging context, so we need to determine which one to use from the call stack.
		if (TargetBP->BlueprintType == BPTYPE_MacroLibrary)
		{
			// Walk backwards through the stack trace (check the most recent sample first) until we find a Blueprint context with the expansion and a valid "active" object for debugging.
			const TSimpleRingBuffer<FKismetTraceSample>& TraceStack = FKismetDebugUtilities::GetTraceStack();
			for (int32 i = TraceStack.Num() - 1; i >= 0 && !TargetBP->GetObjectBeingDebugged(); --i)
			{
				const FKismetTraceSample& Sample = TraceStack(i);
				if (UObject* TestObject = Sample.Context.Get())
				{
					if (UBlueprintGeneratedClass* TargetClass = Cast<UBlueprintGeneratedClass>(TestObject->GetClass()))
					{
						// Check to see if the current instruction maps back to the Macro library source graph; if it does, this Blueprint contains an expansion of the Macro source graph.
						const FBlueprintDebugData& DebugData = TargetClass->GetDebugData();
						if (UEdGraphNode* Node = DebugData.FindSourceNodeFromCodeLocation(Sample.Function.Get(), Sample.Offset, /*bAllowImpreciseHit=*/ false))
						{
							if (GraphObj == Node->GetGraph())
							{
								// Switch the target Blueprint to the context containing the expansion of the current Macro source graph in the Macro library.
								TargetBP = CastChecked<UBlueprint>(TargetClass->ClassGeneratedBy);
							}
						}
					}
				}
			}
		}

		return TargetBP;
	}

	return nullptr;
}

bool FKismetConnectionDrawingPolicy::CanBuildRoadmap(UBlueprint* TargetBP) const
{
	UObject* ActiveObject = GetObjectBeingDebugged(TargetBP);
	return ActiveObject != nullptr;
}

bool FKismetConnectionDrawingPolicy::CanBuildRoadmap() const
{
	return CanBuildRoadmap(GetTargetBlueprint());
}

void FKismetConnectionDrawingPolicy::BuildExecutionRoadmap()
{
	LatestTimeDiscovered = 0.0;

	UBlueprint* TargetBP = GetTargetBlueprint();
	
	// Only do highlighting in PIE or SIE
	if (!CanBuildRoadmap(TargetBP))
	{
		return;
	}

	UObject* ActiveObject = GetObjectBeingDebugged(TargetBP);
	check(ActiveObject); // Due to CanBuildRoadmap

	TArray<UEdGraphNode*> SequentialNodesInGraph;
	TArray<double> SequentialNodeTimes;
	TArray<UEdGraphPin*> SequentialExecPinsInGraph;

	{
		const TSimpleRingBuffer<FKismetTraceSample>& TraceStack = FKismetDebugUtilities::GetTraceStack();

		UBlueprintGeneratedClass* TargetClass = Cast<UBlueprintGeneratedClass>(TargetBP->GeneratedClass);
		FBlueprintDebugData& DebugData = TargetClass->GetDebugData();

		for (int32 i = 0; i < TraceStack.Num(); ++i)
		{
			const FKismetTraceSample& Sample = TraceStack(i);

			if (UObject* TestObject = Sample.Context.Get())
			{
				if (TestObject == ActiveObject)
				{
					UEdGraphPin* AssociatedPin = DebugData.FindSourcePinFromCodeLocation(Sample.Function.Get(), Sample.Offset);

					if (UEdGraphNode* Node = DebugData.FindSourceNodeFromCodeLocation(Sample.Function.Get(), Sample.Offset, /*bAllowImpreciseHit=*/ false))
					{
						if (GraphObj == Node->GetGraph())
						{
							SequentialNodesInGraph.Add(Node);
							SequentialNodeTimes.Add(Sample.ObservationTime);
							SequentialExecPinsInGraph.Add(AssociatedPin);
						}
						else if (const TArray<TWeakObjectPtr<UEdGraphNode> >* ExpansionSourceNodes = DebugData.FindExpansionSourceNodesFromCodeLocation(Sample.Function.Get(), Sample.Offset))
						{
							// Attempt to find an outer expansion (tunnel instance) source node that may have also been mapped to the sample code offset.
							for (TWeakObjectPtr<UEdGraphNode> ExpansionSourceNodePtr : *ExpansionSourceNodes)
							{
								if (UEdGraphNode* TunnelInstanceNode = ExpansionSourceNodePtr.Get())
								{
									if (GraphObj == TunnelInstanceNode->GetGraph())
									{
										SequentialNodesInGraph.Add(TunnelInstanceNode);
										SequentialNodeTimes.Add(Sample.ObservationTime);
										SequentialExecPinsInGraph.Add(AssociatedPin);

										break;
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// Run thru and apply bonus time
	const float InvNumNodes = 1.0f / (float)SequentialNodeTimes.Num();
	for (int32 i = 0; i < SequentialNodesInGraph.Num(); ++i)
	{
		double& ObservationTime = SequentialNodeTimes[i];

		const float PositionRatio = static_cast<float>(SequentialNodeTimes.Num() - i) * InvNumNodes;
		const float PositionBonus = FMath::Pow(PositionRatio, TracePositionExponent) * TracePositionBonusPeriod;
		ObservationTime += PositionBonus;

		LatestTimeDiscovered = FMath::Max<double>(LatestTimeDiscovered, ObservationTime);
	}

	UEdGraphPin* LastExecPin = NULL;
	// Record the unique exec-pin to time pairings, keeping only the most recent 
	// times for each pairing... reverse the "SequentialNodes" because right now
	// it is in stack order (with the last executed node first)
	for (int32 i = SequentialNodesInGraph.Num() - 1; i >= 1; --i)
	{
		UEdGraphNode* CurNode  = SequentialNodesInGraph[i];
		UEdGraphNode* NextNode = SequentialNodesInGraph[i-1];

		// keep track of the last exec-pin executed by CurNode (these tracked 
		// pins coincide with "WireTraceSite" op-codes that have been injected 
		// prior to every "goto" statement... this way we have context for which
		// pin executed the jump)
		if (UEdGraphPin* AssociatedPin = SequentialExecPinsInGraph[i])
		{
			LastExecPin = AssociatedPin;
		}
		
		// if this statement is a jump (from one node to another)
		if (CurNode != NextNode)
		{
			// if there was a wire-trace op-code inserted before this jump
			if (LastExecPin != NULL)
			{
				//ensure(LastExecPin->GetOwningNode() == CurNode);
				double NextNodeTime = SequentialNodeTimes[i-1];

				FExecPairingMap& ExecPaths  = PredecessorPins.FindOrAdd(NextNode);
				FTimePair&       ExecTiming = ExecPaths.FindOrAdd(LastExecPin);
				// make sure that if we've already visited this exec-pin (like 
				// in a for-loop or something), that we're replacing it with a 
				// more recent execution time
				//
				// @TODO I don't see when this wouldn't be the case
				if (ExecTiming.ThisExecTime < NextNodeTime)
				{
					double CurNodeTime = SequentialNodeTimes[i];
					ExecTiming.ThisExecTime = NextNodeTime;
					ExecTiming.PredExecTime = CurNodeTime;
				}
			}
			// if the nodes aren't graphically connected how could they be 
			// executed back-to-back? well, this could be a pop back to a 
			// sequence node from the end of one thread of execution, etc.
			else if (AreNodesGraphicallySequential(CurNode, NextNode))
			{
				// only warn when the nodes are directly connected (this is all
				// for execution flow visualization after all)
				UE_LOGF(LogConnectionDrawingPolicy, Verbose, "Looks like a wire-trace was not injected before the jump from '%ls' to '%ls'.", 
					*CurNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString(), *NextNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString());
			}

			// clear the exec-pin (we're moving to a new node and want to find 
			// it's executed out pin)
			LastExecPin = NULL;
		}
		// else, we're only collecting this data for tracing node-to-node
		// executions (so we don't care about this sequence of statements)
	}

	// Fade only when free-running (since we're using FApp::GetCurrentTime(), instead of FPlatformTime::Seconds)
	const double MaxTimeAhead = FMath::Min(FApp::GetCurrentTime() + 2*TracePositionBonusPeriod, LatestTimeDiscovered); //@TODO: Rough clamping; should be exposed as a parameter
	CurrentTime = FMath::Max(FApp::GetCurrentTime(), MaxTimeAhead);
}

void FKismetConnectionDrawingPolicy::CalculateEnvelopeAlphas(double ExecutionTime, /*out*/ float& AttackAlpha, /*out*/ float& SustainAlpha) const
{
	const float DeltaTime = (float)(CurrentTime - ExecutionTime);

	{
		const float UnclampedDecayRatio = 1.0f - ((DeltaTime - AttackHoldPeriod) / DecayPeriod);
		const float ClampedDecayRatio = FMath::Clamp<float>(UnclampedDecayRatio, 0.0f, 1.0f);
		AttackAlpha = FMath::Pow(ClampedDecayRatio, DecayExponent);
	}

	{
		const float SustainEndTime = AttackHoldPeriod + DecayPeriod + SustainHoldPeriod;

		const float UnclampedReleaseRatio = 1.0f - ((DeltaTime - SustainEndTime) / ReleasePeriod);
		const float ClampedReleaseRatio = FMath::Clamp<float>(UnclampedReleaseRatio, 0.0f, 1.0f);
		SustainAlpha = FMath::Pow(ClampedReleaseRatio, ReleaseExponent);
	}
}

bool FKismetConnectionDrawingPolicy::TreatWireAsExecutionPin(UEdGraphPin* InputPin, UEdGraphPin* OutputPin) const
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	return (InputPin != NULL) && (Schema->IsExecPin(*OutputPin));
}

bool FKismetConnectionDrawingPolicy::AreNodesGraphicallySequential(UEdGraphNode* InputNode, UEdGraphNode* OutputNode) const
{
	for (UEdGraphPin* Pin : InputNode->Pins)
	{
		if (Pin->Direction != EGPD_Output)
		{
			continue;
		}

		for (UEdGraphPin* Connection : Pin->LinkedTo)
		{
			if (!TreatWireAsExecutionPin(Pin, Connection))
			{
				continue;
			}

			if (Connection->GetOwningNode() == OutputNode)
			{
				return true;
			}
		}
	}

	return false;
}

void FKismetConnectionDrawingPolicy::DetermineStyleOfExecWire(float& Thickness, FLinearColor& WireColor, bool& bDrawBubbles, const FTimePair& Times)
{
	// It's a followed link, make it strong and yellowish but fading over time
	const double ExecTime = Times.ThisExecTime;

	float AttackAlpha;
	float SustainAlpha;
	CalculateEnvelopeAlphas(ExecTime, /*out*/ AttackAlpha, /*out*/ SustainAlpha);

	const float DecayedAttackThickness = FMath::Lerp<float>(SustainWireThickness, AttackWireThickness, AttackAlpha);
	Thickness = FMath::Lerp<float>(ReleaseWireThickness, DecayedAttackThickness, SustainAlpha);

	const FLinearColor DecayedAttackColor = FMath::Lerp<FLinearColor>(SustainColor, AttackColor, AttackAlpha);
	WireColor = WireColor * FMath::Lerp<FLinearColor>(ReleaseColor, DecayedAttackColor, SustainAlpha);

	if (SustainAlpha > KINDA_SMALL_NUMBER)
	{
		bDrawBubbles = true;
	}
}

FKismetConnectionDrawingPolicy::FTimePair const* FKismetConnectionDrawingPolicy::BackTraceExecPath(UEdGraphPin const* const OutputPin, FExecPairingMap const* const NodeExecutionList)
{
	FTimePair const* FoundExecPath = nullptr;

	UEdGraphNode const* const OwningNode = OutputPin->GetOwningNode();
	if (UK2Node_Knot const* const KnotNode = Cast<UK2Node_Knot>(OwningNode))
	{
		UEdGraphPin const* const KnotInputPin = KnotNode->GetInputPin();
		for (UEdGraphPin const* KnotInput : KnotInputPin->LinkedTo)
		{
			FoundExecPath = BackTraceExecPath(KnotInput, NodeExecutionList);
			if (FoundExecPath != nullptr)
			{
				break;
			}
		}
	}
	else
	{
		FoundExecPath = NodeExecutionList->Find(OutputPin);
	}

	return FoundExecPath;
}

bool FKismetConnectionDrawingPolicy::FindPinCenter(UEdGraphPin* Pin, FVector2f& OutCenter) const
{
	if (const TSharedPtr<SGraphPin>* pPinWidget = PinToPinWidgetMap.Find(Pin))
	{
		if (FArrangedWidget* pPinEntry = PinGeometries->Find((*pPinWidget).ToSharedRef()))
		{
			OutCenter = FGeometryHelper::CenterOf(pPinEntry->Geometry);
			return true;
		}
	}

	return false;
}

UObject* FKismetConnectionDrawingPolicy::GetObjectBeingDebugged(UBlueprint* TargetBP)
{
	UObject* ActiveObject = nullptr;
	if (TargetBP)
	{
		ActiveObject = TargetBP->GetObjectBeingDebugged();
		if (UClass* GeneratedClass = TargetBP->GeneratedClass;
			ActiveObject == nullptr && BPTYPE_FunctionLibrary == TargetBP->BlueprintType && GeneratedClass)
		{
			ActiveObject = GeneratedClass->GetDefaultObject(false);
		}
	}
	return ActiveObject;
}

bool FKismetConnectionDrawingPolicy::GetAverageConnectedPosition(class UK2Node_Knot* Knot, EEdGraphPinDirection Direction, FVector2f& OutPos) const
{
	FVector2f Result = FVector2f::ZeroVector;
	int32 ResultCount = 0;

	UEdGraphPin* Pin = (Direction == EGPD_Input) ? Knot->GetInputPin() : Knot->GetOutputPin();
	for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
	{
		FVector2f CenterPoint;
		if (FindPinCenter(LinkedPin, /*out*/ CenterPoint))
		{
			Result += CenterPoint;
			ResultCount++;
		}
	}

	if (ResultCount > 0)
	{
		OutPos = Result * (1.0f / static_cast<float>(ResultCount));
		return true;
	}
	else
	{
		return false;
	}
}

bool FKismetConnectionDrawingPolicy::ShouldChangeTangentForKnot(UK2Node_Knot* Knot)
{
	if (bool* pResult = KnotToReversedDirectionMap.Find(Knot))
	{
		return *pResult;
	}
	else
	{
		bool bPinReversed = false;

		FVector2f AverageLeftPin;
		FVector2f AverageRightPin;
		FVector2f CenterPin;
		bool bCenterValid = FindPinCenter(Knot->GetOutputPin(), /*out*/ CenterPin);
		bool bLeftValid = GetAverageConnectedPosition(Knot, EGPD_Input, /*out*/ AverageLeftPin);
		bool bRightValid = GetAverageConnectedPosition(Knot, EGPD_Output, /*out*/ AverageRightPin);

		if (bLeftValid && bRightValid)
		{
			bPinReversed = AverageRightPin.X < AverageLeftPin.X;
		}
		else if (bCenterValid)
		{
			if (bLeftValid)
			{
				bPinReversed = CenterPin.X < AverageLeftPin.X;
			}
			else if (bRightValid)
			{
				bPinReversed = AverageRightPin.X < CenterPin.X;
			}
		}

		KnotToReversedDirectionMap.Add(Knot, bPinReversed);

		return bPinReversed;
	}
}

// Give specific editor modes a chance to highlight this connection or darken non-interesting connections
void FKismetConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	Params.WireThickness = DefaultDataWireThickness;
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;

	// Get the schema and grab the default color from it
	check(OutputPin);
	check(GraphObj);
	const UEdGraphSchema* Schema = GraphObj->GetSchema();

	if (OutputPin->bOrphanedPin || (InputPin && InputPin->bOrphanedPin))
	{
		Params.WireColor = FLinearColor::Red;
	}
	else
	{
		Params.WireColor = Schema->GetPinTypeColor(OutputPin->PinType);
	}

	UEdGraphNode* OutputNode = OutputPin->GetOwningNode();
	UEdGraphNode* InputNode = (InputPin != nullptr) ? InputPin->GetOwningNode() : nullptr;

	const bool bDeemphasizeUnhoveredPins = HoveredPins.Num() > 0;

	// If this is a K2 graph, try to be a little more specific
	if (const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(Schema))
	{
		// If the output or input connect to a knot that is going backwards, we will flip the direction on values going into them
		{
			if (UK2Node_Knot* OutputKnotNode = Cast<UK2Node_Knot>(OutputNode))
			{
				if (ShouldChangeTangentForKnot(OutputKnotNode))
				{
					Params.StartDirection = EGPD_Input;
				}
			}

			if (UK2Node_Knot* InputKnotNode = Cast<UK2Node_Knot>(InputNode))
			{
				if (ShouldChangeTangentForKnot(InputKnotNode))
				{
					Params.EndDirection = EGPD_Output;
				}
			}
		}

		if (TreatWireAsExecutionPin(InputPin, OutputPin))
		{
			if (CanBuildRoadmap())
			{
				// knot nodes are removed from the graph at compile time, so we 
				// have to follow them until we find something that would have 
				// actually executed
				while (UK2Node_Knot* InputKnotNode = Cast<UK2Node_Knot>(InputNode))
				{
					InputNode = nullptr;

					UEdGraphPin* OutPin = InputKnotNode->GetOutputPin();
					if (OutPin->LinkedTo.Num() > 0)
					{
						check(OutPin->LinkedTo.Num() == 1);
						InputNode = OutPin->LinkedTo[0]->GetOwningNode();
					}	
				}

				// track if this node connection was ran or not
				bool bExecuted = false;

				// if the node belonging to InputPin was actually executed
				if (FExecPairingMap* ExecPaths = PredecessorPins.Find(InputNode))
				{
					// if the output pin is one of the pins that lead to InputNode being ran
					if (FTimePair const* ExecTiming = BackTraceExecPath(OutputPin, ExecPaths))
					{
						bExecuted = true;
						DetermineStyleOfExecWire(/*inout*/ Params.WireThickness, /*inout*/ Params.WireColor, /*inout*/ Params.bDrawBubbles, *ExecTiming);
					}
				}

				if (!bExecuted)
				{
					// It's not followed, fade it and keep it thin
					Params.WireColor = ReleaseColor;
					Params.WireThickness = ReleaseWireThickness;
				}
			}
			else
			{
				// Make exec wires slightly thicker even outside of debug
				Params.WireThickness = DefaultExecutionWireThickness;
			}
		}
		else
		{
			// Container types should draw thicker
			if ((InputPin && InputPin->PinType.IsContainer()) || (OutputPin && OutputPin->PinType.IsContainer()))
			{
				Params.WireThickness = DefaultExecutionWireThickness;
			}
		}


		// If either end of the connection is not enabled (and not a passthru to something else), draw the wire differently
		bool bWireIsOnDisabledNodeAndNotPassthru = false;
		if (OutputNode && (OutputNode->IsDisplayAsDisabledForced() || !OutputNode->IsNodeEnabled()))
		{
			if (OutputNode->GetPassThroughPin(OutputPin) == nullptr)
			{
				bWireIsOnDisabledNodeAndNotPassthru = true;
			}
		}
		
		if (InputNode && (InputNode->IsDisplayAsDisabledForced() || !InputNode->IsNodeEnabled()))
		{
			if (InputNode->GetPassThroughPin(InputPin) == nullptr)
			{
				bWireIsOnDisabledNodeAndNotPassthru = true;
			}
		}

		if ((OutputPin && OutputPin->GetOwningNode()->IsNodeUnrelated()) || (InputPin && InputPin->GetOwningNode()->IsNodeUnrelated()))
		{
			bWireIsOnDisabledNodeAndNotPassthru = true;
		}

		if (bWireIsOnDisabledNodeAndNotPassthru)
		{
			Params.WireColor *= 0.5f;
			Params.WireThickness = 0.5f;
		}
	}

	if (bDeemphasizeUnhoveredPins)
	{
		ApplyHoverDeemphasis(OutputPin, InputPin, /*inout*/ Params.WireThickness, /*inout*/ Params.WireColor);
	}

}

void FKismetConnectionDrawingPolicy::SetIncompatiblePinDrawState(const TSharedPtr<SGraphPin>& StartPin, const TSet< TSharedRef<SWidget> >& VisiblePins)
{
	if (VisiblePins.IsEmpty())
	{
		return;
	}
	
	static const FLinearColor DisallowColor(0.25f, 0.25f, 0.25f, 0.5f);
	
	ResetIncompatiblePinDrawState(VisiblePins);

	const UEdGraphPin* StartPinObj = StartPin->GetPinObj();
	const UEdGraphSchema* EdGraphSchema = StartPinObj->GetSchema();
	
	for (auto VisiblePinIterator = VisiblePins.CreateConstIterator(); VisiblePinIterator; ++VisiblePinIterator)
	{
		TSharedPtr<SGraphPin> CheckPin = StaticCastSharedRef<SGraphPin>(*VisiblePinIterator);
		if (CheckPin != StartPin)
		{
			const FPinConnectionResponse Response = EdGraphSchema->CanCreateConnection(StartPinObj, CheckPin->GetPinObj());
			if (Response.Response == CONNECT_RESPONSE_DISALLOW //-V1051
				&& !EdGraphSchema->ShouldShowPanelContextMenuForIncompatibleConnections()) 
			{
				CheckPin->SetPinColorModifier(DisallowColor);
			}
		}
	}
}

void FKismetConnectionDrawingPolicy::ResetIncompatiblePinDrawState(const TSet< TSharedRef<SWidget> >& VisiblePins)
{
	for (auto VisiblePinIterator = VisiblePins.CreateConstIterator(); VisiblePinIterator; ++VisiblePinIterator)
	{
		TSharedPtr<SGraphPin> VisiblePin = StaticCastSharedRef<SGraphPin>(*VisiblePinIterator);
		VisiblePin->SetPinColorModifier(FLinearColor::White);
	}
}
