// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConnectionDrawingPolicy.h"
#include "SGraphPanel.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SToolTip.h"
#include "Framework/Application/SlateApplication.h"
#include "Algo/ForEach.h"

DEFINE_LOG_CATEGORY(LogConnectionDrawingPolicy);

//////////////////////////////////////////////////////////////////////////
// FGeometryHelper

UE::Slate::FDeprecateVector2DResult FGeometryHelper::VerticalMiddleLeftOf(const FGeometry& SomeGeometry)
{
	const FVector2f GeometryDrawSize = SomeGeometry.GetDrawSize();
	return FVector2f(SomeGeometry.AbsolutePosition.X,
		SomeGeometry.AbsolutePosition.Y + GeometryDrawSize.Y/2.f);
}

UE::Slate::FDeprecateVector2DResult FGeometryHelper::VerticalMiddleRightOf(const FGeometry& SomeGeometry)
{
	const FVector2f GeometryDrawSize = SomeGeometry.GetDrawSize();
	return FVector2f(
		SomeGeometry.AbsolutePosition.X + GeometryDrawSize.X,
		SomeGeometry.AbsolutePosition.Y + GeometryDrawSize.Y/2.f );
}

UE::Slate::FDeprecateVector2DResult FGeometryHelper::CenterOf(const FGeometry& SomeGeometry)
{
	const FVector2f GeometryDrawSize = SomeGeometry.GetDrawSize();
	return FVector2f(SomeGeometry.AbsolutePosition) + (GeometryDrawSize * 0.5f);
}

void FGeometryHelper::ConvertToPoints(const FGeometry& Geom, TArray<FVector2D>& Points)
{
	TArray<FVector2f> PointsAsFloats;
	ConvertToPoints(Geom, PointsAsFloats);

	Algo::ForEach(PointsAsFloats, [&Points](const FVector2f& PointAsFloat)
	{
		Points.Emplace(FVector2D(PointAsFloat));
	});
}

void FGeometryHelper::ConvertToPoints(const FGeometry& Geom, TArray<FVector2f>& Points)
{
	const FVector2f Size = Geom.GetDrawSize();
	const FVector2f Location = FVector2f(Geom.AbsolutePosition);

	int32 Index = Points.AddUninitialized(4);
	Points[Index++] = Location;
	Points[Index++] = Location + FVector2f(0.0f, Size.Y);
	Points[Index++] = Location + FVector2f(Size.X, Size.Y);
	Points[Index++] = Location + FVector2f(Size.X, 0.0f);
}

/** Find the point on line segment from LineStart to LineEnd which is closest to Point */
UE::Slate::FDeprecateVector2DResult FGeometryHelper::FindClosestPointOnLine(const UE::Slate::FDeprecateVector2DParameter& LineStart, const UE::Slate::FDeprecateVector2DParameter& LineEnd, const UE::Slate::FDeprecateVector2DParameter& TestPoint)
{
	const FVector2f LineVector = LineEnd - LineStart;

	const float A = -FVector2f::DotProduct(LineStart - TestPoint, LineVector);
	const float B = LineVector.SizeSquared();
	const float T = FMath::Clamp(A / B, 0.0f, 1.0f);

	// Generate closest point
	return LineStart + (T * LineVector);
}

UE::Slate::FDeprecateVector2DResult FGeometryHelper::FindClosestPointOnGeom(const FGeometry& Geom, const UE::Slate::FDeprecateVector2DParameter& TestPoint)
{
	TArray<FVector2f> Points;
	FGeometryHelper::ConvertToPoints(Geom, Points);

	float BestDistanceSquared = MAX_FLT;
	FVector2f BestPoint = FVector2f::ZeroVector;
	for (int32 i = 0; i < Points.Num(); ++i)
	{
		const FVector2f Candidate = FindClosestPointOnLine(Points[i], Points[(i + 1) % Points.Num()], TestPoint);
		const float CandidateDistanceSquared = (Candidate-TestPoint).SizeSquared();
		if (CandidateDistanceSquared < BestDistanceSquared)
		{
			BestPoint = Candidate;
			BestDistanceSquared = CandidateDistanceSquared;
		}
	}

	return BestPoint;
}

namespace IntersectionHelpers
{
	static bool SegmentIntersection2D(const FVector2f& SegmentAStart, const FVector2f& SegmentAEnd, const FVector2f& SegmentBStart, const FVector2f& SegmentBEnd, FVector2f& OutIntersectionPoint)
	{
		FVector Intersection;
		if ( FMath::SegmentIntersection2D(FVector(SegmentAStart.X, SegmentAStart.Y, 0),
			FVector(SegmentAEnd.X, SegmentAEnd.Y, 0),
			FVector(SegmentBStart.X, SegmentBStart.Y, 0),
			FVector(SegmentBEnd.X, SegmentBEnd.Y, 0),
			Intersection))
		{
			OutIntersectionPoint = FVector2f(static_cast<float>(Intersection.X), static_cast<float>(Intersection.Y));
			return true;
		}
		return false;
	}
}

/////////////////////////////////////////////////////
// FConnectionDrawingPolicy

FConnectionDrawingPolicy::FConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements)
	: WireLayerID(InBackLayerID)
	, ArrowLayerID(InFrontLayerID)
	, Settings(GetDefault<UGraphEditorSettings>())
	, ZoomFactor(InZoomFactor)
	, ClippingRect(InClippingRect)
	, DrawElementsList(InDrawElements)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, LocalMousePosition(0.0f, 0.0f)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, AbsoluteMousePosition(0.0f, 0.0f)
{
	ArrowImage = FAppStyle::GetBrush( TEXT("Graph.Arrow") );
	ArrowRadius = ArrowImage->ImageSize * ZoomFactor * 0.5f;
	MidpointImage = nullptr;
	MidpointRadius = FVector2f::ZeroVector;
	HoverDeemphasisDarkFraction = 0.8f;
	SliceDeemphasisAlphaMultiplier = 0.5f;

	BubbleImage = FAppStyle::GetBrush( TEXT("Graph.ExecutionBubble") );
}

void FConnectionDrawingPolicy::DrawSplineWithArrow(const FVector2D& StartPoint, const FVector2D& EndPoint, const FConnectionParams& Params)
{
	DrawSplineWithArrow_DeprecationHelper(StartPoint, EndPoint, Params);
}

void FConnectionDrawingPolicy::DrawSplineWithArrow(const FVector2f& StartPoint, const FVector2f& EndPoint, const FConnectionParams& Params)
{
	DrawSplineWithArrow_DeprecationHelper(StartPoint, EndPoint, Params);
}

void FConnectionDrawingPolicy::DrawSplineWithArrow_DeprecationHelper(const UE::Slate::FDeprecateVector2DParameter& StartPoint, const UE::Slate::FDeprecateVector2DParameter& EndPoint, const FConnectionParams& Params)
{
	// Draw the spline
	DrawConnection(
		WireLayerID,
		StartPoint,
		EndPoint,
		Params);

	// Draw the arrow
	if (ArrowImage != nullptr)
	{
		FVector2f ArrowPoint = EndPoint - ArrowRadius;

		FSlateDrawElement::MakeBox(
			DrawElementsList,
			ArrowLayerID,
			FPaintGeometry(ArrowPoint, ArrowImage->ImageSize * ZoomFactor, ZoomFactor),
			ArrowImage,
			ESlateDrawEffect::None,
			Params.WireColor
		);
	}
}

void GetSplineEndPoints(const FGeometry& StartGeom, const FGeometry& EndGeom, const FVector2f& InArrowRadius, FVector2f& OutStartPoint, FVector2f& OutEndPoint)
{
	//@TODO: These values should be pushed into the Slate style, they are compensating for a bit of
	// empty space inside of the pin brush images.
	const float StartFudgeX = 4.0f;
	const float EndFudgeX = 4.0f;
	OutStartPoint = FGeometryHelper::VerticalMiddleRightOf(StartGeom) - FVector2f(StartFudgeX, 0.0f);
	OutEndPoint = FGeometryHelper::VerticalMiddleLeftOf(EndGeom) - FVector2f(InArrowRadius.X - EndFudgeX, 0);
}

void FConnectionDrawingPolicy::DrawSplineWithArrow(const FGeometry& StartGeom, const FGeometry& EndGeom, const FConnectionParams& Params)
{
	FVector2f StartPoint;
	FVector2f EndPoint;
	GetSplineEndPoints(StartGeom, EndGeom, ArrowRadius, StartPoint, EndPoint);

	DrawSplineWithArrow(StartPoint, EndPoint, Params);
}

// Update the drawing policy with the set of hovered pins (which can be empty)
void FConnectionDrawingPolicy::SetHoveredPins(const TSet< FEdGraphPinReference >& InHoveredPins, const TArray< TSharedPtr<SGraphPin> >& OverridePins, double HoverTime)
{
	HoveredPins.Empty();

	LastHoverTimeEvent = (OverridePins.Num() > 0) ? 0.0 : HoverTime;

	for (auto PinIt = OverridePins.CreateConstIterator(); PinIt; ++PinIt)
	{
		if (SGraphPin* GraphPin = PinIt->Get())
		{
			HoveredPins.Add(GraphPin->GetPinObj());
		}
	}

	// When we have only a single pin selected, we'll extend selection to apply the hover effect on the links
	const bool bMakeConnectedPinsHovered = (InHoveredPins.Num() == 1);

	// Convert the widget pointer for hovered pins to be EdGraphPin pointers for their connected nets (both ends of any connection)
	for (auto PinIt = InHoveredPins.CreateConstIterator(); PinIt; ++PinIt)
	{
		if (UEdGraphPin* Pin = PinIt->Get())
		{
			if (Pin->LinkedTo.Num() > 0)
			{
				HoveredPins.Add(Pin);

				if (bMakeConnectedPinsHovered)
				{
					for (auto LinkIt = Pin->LinkedTo.CreateConstIterator(); LinkIt; ++LinkIt)
					{
						HoveredPins.Add(*LinkIt);
					}
				}
			}
		}
	}
}

void FConnectionDrawingPolicy::SetMousePosition(const UE::Slate::FDeprecateVector2DParameter& InMousePos)
{
	SetAbsoluteMousePosition(InMousePos);
}

void FConnectionDrawingPolicy::SetAbsoluteMousePosition(FVector2f InAbsoluteMousePos)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LocalMousePosition = InAbsoluteMousePos;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	AbsoluteMousePosition = MoveTemp(InAbsoluteMousePos);
}

void FConnectionDrawingPolicy::SetSliceLine(const FMarqueeRect& InLine)
{
	SliceLine = InLine;
}

void FConnectionDrawingPolicy::SetMarkedPin(TWeakPtr<SGraphPin> InMarkedPin)
{
	if (InMarkedPin.IsValid())
	{
		LastHoverTimeEvent = 0.0;

		TSharedPtr<SGraphPin> MarkedPinWidget = InMarkedPin.Pin();
		if(UEdGraphPin* MarkedPin = MarkedPinWidget->GetPinObj())
		{
			HoveredPins.Add(MarkedPin);

			for (auto LinkIt = MarkedPin->LinkedTo.CreateConstIterator(); LinkIt; ++LinkIt)
			{
				HoveredPins.Add(*LinkIt);
			}
		}
	}
}

/** Util to make a 'distance->alpha' table and also return spline length */
float FConnectionDrawingPolicy::MakeSplineReparamTable(const UE::Slate::FDeprecateVector2DParameter& P0, const UE::Slate::FDeprecateVector2DParameter& P0Tangent, const UE::Slate::FDeprecateVector2DParameter& P1, const UE::Slate::FDeprecateVector2DParameter& P1Tangent, FInterpCurve<float>& OutReparamTable)
{
	const int32 NumSteps = 10; // TODO: Make this adaptive...

	OutReparamTable.Points.Empty(NumSteps);

	// Find range of input
	float Param = 0.f;
	const float MaxInput = 1.f;
	const float Interval = (MaxInput - Param)/((float)(NumSteps-1)); 

	// Add first entry, using first point on curve, total distance will be 0
	FVector2f OldSplinePos = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, Param);
	float TotalDist = 0.f;
	OutReparamTable.AddPoint(TotalDist, Param);
	Param += Interval;

	// Then work over rest of points	
	for(int32 i=1; i<NumSteps; i++)
	{
		// Iterate along spline at regular param intervals
		const FVector2f NewSplinePos = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, Param);
		TotalDist += (NewSplinePos - OldSplinePos).Size();
		OldSplinePos = NewSplinePos;

		OutReparamTable.AddPoint(TotalDist, Param);

		Param += Interval;
	}

	return TotalDist;
}

FVector2D FConnectionDrawingPolicy::ComputeSplineTangent(const FVector2D& Start, const FVector2D& End) const
{
	return ComputeSplineTangent_DeprecationHelper(Start, End);
}

FVector2f FConnectionDrawingPolicy::ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const
{
	return ComputeSplineTangent_DeprecationHelper(Start, End);
}

UE::Slate::FDeprecateVector2DResult FConnectionDrawingPolicy::ComputeSplineTangent_DeprecationHelper(const UE::Slate::FDeprecateVector2DParameter& Start, const UE::Slate::FDeprecateVector2DParameter& End) const
{
	return Settings->ComputeSplineTangent(Start, End);
}

void FConnectionDrawingPolicy::DrawConnection(int32 LayerId, const FVector2D& Start, const FVector2D& End, const FConnectionParams& Params)
{
	DrawConnection_DeprecationHelper(LayerId, Start, End, Params);
}

void FConnectionDrawingPolicy::DrawConnection(int32 LayerId, const FVector2f& Start, const FVector2f& End, const FConnectionParams& Params)
{
	DrawConnection_DeprecationHelper(LayerId, Start, End, Params);
}

void FConnectionDrawingPolicy::DrawConnection_DeprecationHelper(int32 LayerId, const UE::Slate::FDeprecateVector2DParameter& Start, const UE::Slate::FDeprecateVector2DParameter& End, const FConnectionParams& Params)
{
	const FVector2f& P0 = Start;
	const FVector2f& P1 = End;

	const FVector2f SplineTangent = ComputeSplineTangent(P0, P1);

	const FVector2f P0Tangent = (Params.StartTangent.IsNearlyZero()) ? ((Params.StartDirection == EGPD_Output) ? SplineTangent : -SplineTangent) : Params.StartTangent;
	const FVector2f P1Tangent = (Params.EndTangent.IsNearlyZero()) ? ((Params.EndDirection == EGPD_Input) ? SplineTangent : -SplineTangent) : Params.EndTangent;

	bool bSliceLineIntersectsSpline = false;
	CheckSplineConnectionOverlapWithCursor(AbsoluteMousePosition, Start, End, Params, /*out*/ SplineOverlapResult, /*out*/ &bSliceLineIntersectsSpline);

	// Apply an opacity fade to wires intersected by the slice line
	FLinearColor WireColor = Params.WireColor;
	if (bSliceLineIntersectsSpline)
	{
		WireColor.A *= SliceDeemphasisAlphaMultiplier;
		ConnectionsIntersectingSliceLine.Emplace(Params.AssociatedPin1, Params.AssociatedPin2);
	}

	// Draw the spline itself
	FSlateDrawElement::MakeDrawSpaceSpline(
		DrawElementsList,
		LayerId,
		P0, P0Tangent,
		P1, P1Tangent,
		Params.WireThickness,
		ESlateDrawEffect::None,
		WireColor
	);

	if (Params.bDrawBubbles || (MidpointImage != nullptr))
	{
		// This table maps distance along curve to alpha
		FInterpCurve<float> SplineReparamTable;
		const float SplineLength = MakeSplineReparamTable(P0, P0Tangent, P1, P1Tangent, SplineReparamTable);

		// Draw bubbles on the spline
		if (Params.bDrawBubbles)
		{
			const float BubbleSpacing = 64.f * ZoomFactor;
			const float BubbleSpeed = 192.f * ZoomFactor;
			const FVector2f BubbleSize = BubbleImage->ImageSize * ZoomFactor * 0.2f * Params.WireThickness;

			float Time = static_cast<float>(FPlatformTime::Seconds() - GStartTime);
			const float BubbleOffset = FMath::Fmod(Time * BubbleSpeed, BubbleSpacing);
			const int32 NumBubbles = FMath::CeilToInt(SplineLength / BubbleSpacing);
			for (int32 i = 0; i < NumBubbles; ++i)
			{
				const float Distance = ((float)i * BubbleSpacing) + BubbleOffset;
				if (Distance < SplineLength)
				{
					const float Alpha = SplineReparamTable.Eval(Distance, 0.f);
					FVector2f BubblePos = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, Alpha);
					BubblePos -= (BubbleSize * 0.5f);

					FSlateDrawElement::MakeBox(
						DrawElementsList,
						LayerId,
						FPaintGeometry(BubblePos, BubbleSize, ZoomFactor),
						BubbleImage,
						ESlateDrawEffect::None,
						Params.WireColor
					);
				}
			}
		}

		// Draw the midpoint image
		if (MidpointImage != nullptr)
		{
			// Determine the spline position for the midpoint
			const float MidpointAlpha = SplineReparamTable.Eval(SplineLength * 0.5f, 0.f);
			const FVector2f Midpoint = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, MidpointAlpha);

			// Approximate the slope at the midpoint (to orient the midpoint image to the spline)
			const FVector2f MidpointPlusE = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, MidpointAlpha + KINDA_SMALL_NUMBER);
			const FVector2f MidpointMinusE = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, MidpointAlpha - KINDA_SMALL_NUMBER);
			const FVector2f SlopeUnnormalized = MidpointPlusE - MidpointMinusE;

			// Draw the arrow
			const FVector2f MidpointDrawPos = Midpoint - MidpointRadius;
			const float AngleInRadians = SlopeUnnormalized.IsNearlyZero() ? 0.0f : FMath::Atan2(SlopeUnnormalized.Y, SlopeUnnormalized.X);

			FSlateDrawElement::MakeRotatedBox(
				DrawElementsList,
				LayerId,
				FPaintGeometry(MidpointDrawPos, MidpointImage->ImageSize * ZoomFactor, ZoomFactor),
				MidpointImage,
				ESlateDrawEffect::None,
				AngleInRadians,
				TOptional<FVector2f>(),
				FSlateDrawElement::RelativeToElement,
				Params.WireColor
			);
		}
	}
}

void FConnectionDrawingPolicy::DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2D& StartPoint, const FVector2D& EndPoint, UEdGraphPin* Pin)
{
	DrawPreviewConnector_DeprecationHelper(PinGeometry, StartPoint, EndPoint, Pin);
}

void FConnectionDrawingPolicy::DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2f& StartPoint, const FVector2f& EndPoint, UEdGraphPin* Pin)
{
	DrawPreviewConnector_DeprecationHelper(PinGeometry, StartPoint, EndPoint, Pin);
}

void FConnectionDrawingPolicy::DrawPreviewConnector_DeprecationHelper(const FGeometry& PinGeometry, const UE::Slate::FDeprecateVector2DParameter& StartPoint, const UE::Slate::FDeprecateVector2DParameter& EndPoint, UEdGraphPin* Pin)
{
	FConnectionParams Params;
	DetermineWiringStyle(Pin, nullptr, /*inout*/ Params);

	DrawSplineWithArrow(StartPoint, EndPoint, Params);
}

bool FConnectionDrawingPolicy::DrawSliceLine()
{
	if (SliceLine.IsValid())
	{
		TArray<FVector2f> LinePoints;
		LinePoints.Add(SliceLine.StartPoint);
		LinePoints.Add(SliceLine.EndPoint);
		
		// Draw a solid line instead of dashes if the dash length is zero (or close to, to avoid the line disappearing due to dashes being too small)
		if (Settings->SliceLineDashLength <= 1.f)
		{
			FSlateDrawElement::MakeLines(
				DrawElementsList,
				WireLayerID,
				FPaintGeometry(),
				LinePoints,
				ESlateDrawEffect::None,
				Settings->SliceLineColor,
				true,
				Settings->SliceLineThickness
			);
		}
		else
		{
			FSlateDrawElement::MakeDashedLines(
				DrawElementsList,
				WireLayerID,
				FPaintGeometry(),
				CopyTemp(LinePoints),
				ESlateDrawEffect::None,
				Settings->SliceLineColor,
				Settings->SliceLineThickness,
				Settings->SliceLineDashLength
			);
		}
		return true;
	}
	return false;
}

void FConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;
}

void FConnectionDrawingPolicy::DetermineLinkGeometry(
	FArrangedChildren& ArrangedNodes, 
	TSharedRef<SWidget>& OutputPinWidget,
	UEdGraphPin* OutputPin,
	UEdGraphPin* InputPin,
	/*out*/ FArrangedWidget*& StartWidgetGeometry,
	/*out*/ FArrangedWidget*& EndWidgetGeometry
	)
{
	StartWidgetGeometry = PinGeometries->Find(OutputPinWidget);
	
	if (TSharedPtr<SGraphPin>* pTargetWidget = PinToPinWidgetMap.Find(InputPin))
	{
		TSharedRef<SGraphPin> InputWidget = (*pTargetWidget).ToSharedRef();
		EndWidgetGeometry = PinGeometries->Find(InputWidget);
	}
}

void FConnectionDrawingPolicy::Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	PinGeometries = &InPinGeometries;

	BuildPinToPinWidgetMap(InPinGeometries);

	DrawPinGeometries(InPinGeometries, ArrangedNodes);
}

bool FConnectionDrawingPolicy::HaveConnectionsGraphicallyChanged(const SGraphPanel& InGraphPanel, const FVector2f& AbsoluteCursorPosition) const
{
	FGraphSplineOverlapResult OverlapResult;

	for (const FSimpleConnectionData& Connection : LastConnections)
	{
		UEdGraphPin* EndPinObj = Connection.EndPinHandle.GetPinObj(InGraphPanel);
		if (!EndPinObj)
		{
			return true; // pin is not valid anymore, just redraw
		}

		FConnectionParams Params;
		Params.AssociatedPin1 = EndPinObj;
		Params.AssociatedPin2 = Connection.StartPinHandle.GetPinObj(InGraphPanel);
		CheckSplineConnectionOverlapWithCursor(AbsoluteCursorPosition, Connection.StartPoint, Connection.EndPoint, Params, /*out*/ OverlapResult);
	}

	TSharedPtr<SGraphPin> NewPin1Widget;
	TSharedPtr<SGraphPin> NewPin2Widget;
	OverlapResult.GetPinWidgets(InGraphPanel, NewPin1Widget, NewPin2Widget);

	TSet<FEdGraphPinReference> NewlyHoveredPins;
	if (NewPin1Widget && NewPin1Widget->GetPinObj())
	{
		NewlyHoveredPins.Add(NewPin1Widget->GetPinObj());
	}
	if (NewPin2Widget && NewPin2Widget->GetPinObj())
	{
		NewlyHoveredPins.Add(NewPin2Widget->GetPinObj());
	}

	return HaveConnectionsGraphicallyChanged(InGraphPanel, NewlyHoveredPins);
}

bool FConnectionDrawingPolicy::HaveConnectionsGraphicallyChanged(const SGraphPanel& InGraphPanel, const TSet<FEdGraphPinReference>& InNewlyHoveredPins) const
{
	if ( HoveredPins.Num() != InNewlyHoveredPins.Num() || !HoveredPins.Difference(InNewlyHoveredPins).IsEmpty() )
	{
		return true; // If the Hovered Pins are different, we do not check further
	}

	// Now we know that TempHoveredPins and HoveredPins are equivalent, so DetermineWiringStyle to give us the right information
	// (as DetermineWiringStyle often checks the content of HoveredPins to decide how to draw them)
	for (const FSimpleConnectionData& Connection : LastConnections)
	{
		UEdGraphPin* EndPinObj = Connection.EndPinHandle.GetPinObj(InGraphPanel);
		if (!EndPinObj)
		{
			return true; // pin is not valid anymore, just redraw
		}

		FConnectionParams Params;
		const_cast<FConnectionDrawingPolicy*>(this)->DetermineWiringStyle(EndPinObj, Connection.StartPinHandle.GetPinObj(InGraphPanel), /*inout*/ Params);
		if (Connection.bAreConnectionsFaded)
		{
			Params.WireColor.A = 0.2f;
		}

		if (Params.WireThickness != Connection.Params.WireThickness || Params.WireColor != Connection.Params.WireColor)
		{
			return true;
		}
	}
	return false;
}

void FConnectionDrawingPolicy::BuildPinToPinWidgetMap(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries)
{
	PinToPinWidgetMap.Empty();
	for (TMap<TSharedRef<SWidget>, FArrangedWidget>::TIterator ConnectorIt(InPinGeometries); ConnectorIt; ++ConnectorIt)
	{
		TSharedRef<SWidget> SomePinWidget = ConnectorIt.Key();
		SGraphPin& PinWidget = static_cast<SGraphPin&>(SomePinWidget.Get());

		PinToPinWidgetMap.Add(PinWidget.GetPinObj(), StaticCastSharedRef<SGraphPin>(SomePinWidget));
	}
}

void FConnectionDrawingPolicy::DrawPinGeometries(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	for (TMap<TSharedRef<SWidget>, FArrangedWidget>::TIterator ConnectorIt(InPinGeometries); ConnectorIt; ++ConnectorIt)
	{
		TSharedRef<SWidget> SomePinWidget = ConnectorIt.Key();
		SGraphPin& PinWidget = static_cast<SGraphPin&>(SomePinWidget.Get());
		UEdGraphPin* ThePin = PinWidget.GetPinObj();

		if (ThePin && ThePin->Direction == EGPD_Output)
		{
			for (int32 LinkIndex=0; LinkIndex < ThePin->LinkedTo.Num(); ++LinkIndex)
			{
				FArrangedWidget* LinkStartWidgetGeometry = nullptr;
				FArrangedWidget* LinkEndWidgetGeometry = nullptr;

				UEdGraphPin* TargetPin = ThePin->LinkedTo[LinkIndex];

				DetermineLinkGeometry(ArrangedNodes, SomePinWidget, ThePin, TargetPin, /*out*/ LinkStartWidgetGeometry, /*out*/ LinkEndWidgetGeometry);

				if (( LinkEndWidgetGeometry && LinkStartWidgetGeometry ) && !IsConnectionCulled( *LinkStartWidgetGeometry, *LinkEndWidgetGeometry ))
				{
					FConnectionParams Params;
					bool bAreConnectionsFaded = false;

					DetermineWiringStyle(ThePin, TargetPin, /*inout*/ Params);
					const TSharedPtr<SGraphPin>* ConnectedPinWidget = PinToPinWidgetMap.Find(TargetPin);
					if (ConnectedPinWidget && ConnectedPinWidget->IsValid())
					{
						if ( PinWidget.AreConnectionsFaded() && (*ConnectedPinWidget)->AreConnectionsFaded() )
						{
							Params.WireColor.A = 0.2f;
							bAreConnectionsFaded = true;
						}
					}

					DrawSplineWithArrow(LinkStartWidgetGeometry->Geometry, LinkEndWidgetGeometry->Geometry, Params);

					FVector2f StartPoint;
					FVector2f EndPoint;
					GetSplineEndPoints(LinkStartWidgetGeometry->Geometry, LinkEndWidgetGeometry->Geometry, ArrowRadius, StartPoint, EndPoint);

					TSharedPtr<SGraphPin> StartPinWidget = ConnectedPinWidget ? *ConnectedPinWidget : nullptr;
					TSharedRef<SGraphPin> EndPinWidget = StaticCastSharedRef<SGraphPin>(SomePinWidget);
					FGraphPinHandle StartPinHandle(StartPinWidget ? StartPinWidget->GetPinObj() : nullptr);
					FGraphPinHandle EndPinHandle(EndPinWidget->GetPinObj());
					LastConnections.Add(FSimpleConnectionData{ StartPinHandle, EndPinHandle, StartPoint, EndPoint, Params, bAreConnectionsFaded });
				}
			}
		}
	}
}

void FConnectionDrawingPolicy::CheckSplineConnectionOverlapWithCursor(const FVector2f& AbsoluteCursorPosition, const FVector2f& Start, const FVector2f& End, const FConnectionParams& Params, FGraphSplineOverlapResult& InOutOverlapResults, bool* bOutSliceLineIntersectsSpline) const
{
	const FVector2f& P0 = Start;
	const FVector2f& P1 = End;

	const FVector2f SplineTangent = ComputeSplineTangent(P0, P1);

	const FVector2f P0Tangent = (Params.StartTangent.IsNearlyZero()) ? ((Params.StartDirection == EGPD_Output) ? SplineTangent : -SplineTangent) : Params.StartTangent;
	const FVector2f P1Tangent = (Params.EndTangent.IsNearlyZero()) ? ((Params.EndDirection == EGPD_Input) ? SplineTangent : -SplineTangent) : Params.EndTangent;

	if (bOutSliceLineIntersectsSpline)
	{
		*bOutSliceLineIntersectsSpline = false;
	}

	if (!Settings->bTreatSplinesLikePins)
	{
		return;
	}

	// Distance to consider as an overlap
	const float QueryDistanceTriggerThresholdSquared = FMath::Square(Settings->SplineHoverTolerance + Params.WireThickness * 0.5f);

	// Distance to pass the bounding box cull test. This is used for the bCloseToSpline output that can be used as a
	// dead zone to avoid mistakes caused by missing a double-click on a connection.
	const float QueryDistanceForCloseSquared = FMath::Square(FMath::Sqrt(QueryDistanceTriggerThresholdSquared) + Settings->SplineCloseTolerance);

	bool bSliceLineIntersectsSplineBounds = false;
	bool bCloseToSpline = false;
	{
		// The curve will include the endpoints but can extend out of a tight bounds because of the tangents
		// P0Tangent coefficient maximizes to 4/27 at a=1/3, and P1Tangent minimizes to -4/27 at a=2/3.
		constexpr float MaximumTangentContribution = 4.0f / 27.0f;
		FBox2f Bounds(ForceInit);

		Bounds += FVector2f(P0);
		Bounds += FVector2f(P0 + MaximumTangentContribution * P0Tangent);
		Bounds += FVector2f(P1);
		Bounds += FVector2f(P1 - MaximumTangentContribution * P1Tangent);

		bCloseToSpline = Bounds.ComputeSquaredDistanceToPoint(AbsoluteCursorPosition) < QueryDistanceForCloseSquared;

		// Re-use the bounds we just constructed to check if the slice line intersects the spline
		if (SliceLine.IsValid() && bOutSliceLineIntersectsSpline)
		{
			bSliceLineIntersectsSplineBounds = FMath::LineBoxIntersection2D(Bounds, SliceLine.StartPoint, SliceLine.EndPoint);
		}

		// Draw the bounding box for debugging
#if 0
#define DrawSpaceLine(Point1, Point2, DebugWireColor) {const FVector2f FakeTangent = (Point2 - Point1).GetSafeNormal(); FSlateDrawElement::MakeDrawSpaceSpline(DrawElementsList, LayerId, Point1, FakeTangent, Point2, FakeTangent, ClippingRect, 1.0f, ESlateDrawEffect::None, DebugWireColor); }

		if (bCloseToSpline)
		{
			const FLinearColor BoundsWireColor = bCloseToSpline ? FLinearColor::Green : FLinearColor::White;

			FVector2f TL = Bounds.Min;
			FVector2f BR = Bounds.Max;
			FVector2f TR = FVector2f(Bounds.Max.X, Bounds.Min.Y);
			FVector2f BL = FVector2f(Bounds.Min.X, Bounds.Max.Y);

			DrawSpaceLine(TL, TR, BoundsWireColor);
			DrawSpaceLine(TR, BR, BoundsWireColor);
			DrawSpaceLine(BR, BL, BoundsWireColor);
			DrawSpaceLine(BL, TL, BoundsWireColor);
		}
#endif
	}

	constexpr int32 NumStepsToTest = 16;
	constexpr float TestStepInterval = 1.0f / NumStepsToTest;

	// Only bother checking for precise spline intersection if we intersected the bounding box
	if (bSliceLineIntersectsSplineBounds)
	{
		// Find the intersecting point on the spline
		FVector2f OutIntersectionPoint;
		FVector2f Point1 = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, 0.0f);
		for (float TestAlpha = 0.0f; TestAlpha < 1.0f; TestAlpha += TestStepInterval)
		{
			const FVector2f Point2 = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, TestAlpha + TestStepInterval);

			if (IntersectionHelpers::SegmentIntersection2D(SliceLine.StartPoint, SliceLine.EndPoint, Point1, Point2, OutIntersectionPoint))
			{
				check(bOutSliceLineIntersectsSpline);
				*bOutSliceLineIntersectsSpline = true;
				break;
			}

			Point1 = Point2;
		}
	}

	if (bCloseToSpline)
	{
		// Find the closest approach to the spline
		FVector2f ClosestPoint(ForceInit);
		float ClosestDistanceSquared = FLT_MAX;
		FVector2f Point1 = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, 0.0f);
		for (float TestAlpha = 0.0f; TestAlpha < 1.0f; TestAlpha += TestStepInterval)
		{
			const FVector2f Point2 = FMath::CubicInterp(P0, P0Tangent, P1, P1Tangent, TestAlpha + TestStepInterval);

			const FVector2f ClosestPointToSegment = FMath::ClosestPointOnSegment2D(AbsoluteCursorPosition, Point1, Point2);
			const float DistanceSquared = (AbsoluteCursorPosition - ClosestPointToSegment).SizeSquared();

			if (DistanceSquared < ClosestDistanceSquared)
			{
				ClosestDistanceSquared = DistanceSquared;
				ClosestPoint = ClosestPointToSegment;
			}

			Point1 = Point2;
		}

		// Record the overlap
		if (ClosestDistanceSquared < QueryDistanceTriggerThresholdSquared)
		{
			if (ClosestDistanceSquared < InOutOverlapResults.GetDistanceSquared())
			{
				const float SquaredDistToPin1 = (Params.AssociatedPin1 != nullptr) ? (P0 - ClosestPoint).SizeSquared() : FLT_MAX;
				const float SquaredDistToPin2 = (Params.AssociatedPin2 != nullptr) ? (P1 - ClosestPoint).SizeSquared() : FLT_MAX;

				InOutOverlapResults = FGraphSplineOverlapResult(Params.AssociatedPin1, Params.AssociatedPin2, ClosestDistanceSquared, SquaredDistToPin1, SquaredDistToPin2, true);
			}
		}
		else if (ClosestDistanceSquared < QueryDistanceForCloseSquared)
		{
			InOutOverlapResults.SetCloseToSpline(true);
		}
	}
}

bool FConnectionDrawingPolicy::IsConnectionCulled( const FArrangedWidget& StartLink, const FArrangedWidget& EndLink ) const
{
	const float Top		= FMath::Min( StartLink.Geometry.AbsolutePosition.Y, EndLink.Geometry.AbsolutePosition.Y );
	const float Left	= FMath::Min( StartLink.Geometry.AbsolutePosition.X, EndLink.Geometry.AbsolutePosition.X );
	const float Bottom	= FMath::Max( StartLink.Geometry.AbsolutePosition.Y + StartLink.Geometry.Size.Y, EndLink.Geometry.AbsolutePosition.Y + EndLink.Geometry.Size.Y );
	const float Right	= FMath::Max( StartLink.Geometry.AbsolutePosition.X + StartLink.Geometry.Size.X, EndLink.Geometry.AbsolutePosition.X + EndLink.Geometry.Size.X ); 

	return	Left > ClippingRect.Right || Right < ClippingRect.Left || 
			Bottom < ClippingRect.Top || Top > ClippingRect.Bottom;
}

void FConnectionDrawingPolicy::SetIncompatiblePinDrawState(const TSharedPtr<SGraphPin>& StartPin, const TSet< TSharedRef<SWidget> >& VisiblePins)
{
}

void FConnectionDrawingPolicy::ResetIncompatiblePinDrawState(const TSet< TSharedRef<SWidget> >& VisiblePins)
{
}

void FConnectionDrawingPolicy::ApplyHoverDeemphasis(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ float& Thickness, /*inout*/ FLinearColor& WireColor)
{
	//@TODO: Move these parameters into the settings object
	const float FadeInBias = 0.75f; // Time in seconds before the fading starts to occur
	const float FadeInPeriod = 0.6f; // Time in seconds after the bias before the fade is fully complete
	const float TimeFraction = FMath::SmoothStep(0.0f, FadeInPeriod, (float)(FSlateApplication::Get().GetCurrentTime() - LastHoverTimeEvent - FadeInBias));

	const float LightFraction = 0.25f;
	const FLinearColor DarkenedColor(0.0f, 0.0f, 0.0f, 0.5f);
	const FLinearColor LightenedColor(1.0f, 1.0f, 1.0f, 1.0f);

	const bool bContainsBoth = HoveredPins.Contains(InputPin) && HoveredPins.Contains(OutputPin);
	const bool bContainsOutput = HoveredPins.Contains(OutputPin);
	const bool bEmphasize = bContainsBoth || (bContainsOutput && (InputPin == nullptr));
	if (bEmphasize)
	{
		Thickness = FMath::Lerp(Thickness, Thickness * ((Thickness < 2.5f) ? 3.5f : 2.5f), TimeFraction);
		WireColor = FMath::Lerp<FLinearColor>(WireColor, LightenedColor, LightFraction * TimeFraction);
	}
	else
	{
		WireColor = FMath::Lerp<FLinearColor>(WireColor, DarkenedColor, HoverDeemphasisDarkFraction * TimeFraction);
	}
}

//////////////////////////////////////////////////////////////////////////
// FGraphSplineOverlapResult

void FGraphSplineOverlapResult::ComputeBestPin()
{
	UEdGraphPin* BestPin = nullptr;
	if (Pin1 == nullptr)
	{
		BestPin = Pin2;
	}
	else if (Pin2 == nullptr)
	{
		BestPin = Pin1;
	}
	else
	{
		// choose based on distance to the pins
		BestPin = (DistanceSquaredToPin1 < DistanceSquaredToPin2) ? Pin1 : Pin2;
	}

	BestPinHandle = FGraphPinHandle(BestPin);

	Pin1 = nullptr;
	Pin2 = nullptr;
}

bool FGraphSplineOverlapResult::GetPins(const class SGraphPanel& InGraphPanel, UEdGraphPin*& OutPin1, UEdGraphPin*& OutPin2) const
{
	OutPin1 = nullptr;
	OutPin2 = nullptr;

	if (IsValid())
	{
		TSharedPtr<SGraphPin> Pin1Widget = Pin1Handle.FindInGraphPanel(InGraphPanel);
		TSharedPtr<SGraphPin> Pin2Widget = Pin2Handle.FindInGraphPanel(InGraphPanel);

		if (Pin1Widget.IsValid())
		{
			OutPin1 = Pin1Widget->GetPinObj();
		}

		if (Pin2Widget.IsValid())
		{
			OutPin2 = Pin2Widget->GetPinObj();
		}
	}

	return (OutPin1 != nullptr) && (OutPin2 != nullptr);
}

void FGraphSplineOverlapResult::GetPinWidgets(const class SGraphPanel& InGraphPanel, TSharedPtr<class SGraphPin>& OutPin1, TSharedPtr<class SGraphPin>& OutPin2) const
{
	OutPin1 = nullptr;
	OutPin2 = nullptr;

	if (IsValid())
	{
		OutPin1 = Pin1Handle.FindInGraphPanel(InGraphPanel);
		OutPin2 = Pin2Handle.FindInGraphPanel(InGraphPanel);
	}
}

TSharedPtr<IToolTip> FConnectionDrawingPolicy::GetConnectionToolTip(const SGraphPanel& GraphPanel, const FGraphSplineOverlapResult& OverlapData) const
{
	if (SGraphPin* BestPinFromHoveredSpline = OverlapData.GetBestPinWidget(GraphPanel).Get())
	{
		return BestPinFromHoveredSpline->GetToolTip();
	}
	return const_cast<SGraphPanel&>(GraphPanel).GetToolTip();
}