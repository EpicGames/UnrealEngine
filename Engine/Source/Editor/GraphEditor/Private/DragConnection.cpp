// Copyright Epic Games, Inc. All Rights Reserved.


#include "DragConnection.h"

#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "EdGraphSchema_K2.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphHandleTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FSlateBrush;

TSharedRef<FDragConnection> FDragConnection::New(const TSharedRef<SGraphPanel>& GraphPanel, const FDraggedPinTable& DraggedPins)
{
	TSharedRef<FDragConnection> Operation = MakeShareable(new FDragConnection(GraphPanel, DraggedPins));
	Operation->Construct();

	return Operation;
}

void FDragConnection::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	GraphPanel->OnStopMakingConnection();
	GraphPanel->OnEndRelinkConnection();

	Super::OnDrop(bDropWasHandled, MouseEvent);
}

void FDragConnection::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	FVector2f TargetPosition = DragDropEvent.GetScreenSpacePosition();

	// Reposition the info window wrt to the drag
	CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition() + DecoratorAdjust);
	// Request the active panel to scroll if required
	GraphPanel->RequestDeferredPan(TargetPosition);
	GraphPanel->Invalidate(EInvalidateWidgetReason::Paint);
}

void FDragConnection::HoverTargetChanged()
{
	TArray<FPinConnectionResponse> UniqueMessages;

	if (UEdGraphPin* TargetPinObj = GetHoveredPin())
	{
		TArray<UEdGraphPin*> ValidSourcePins;
		ValidateGraphPinList(/*out*/ ValidSourcePins);

		// Check the schema for connection responses
		for (UEdGraphPin* StartingPinObj : ValidSourcePins)
		{
			if (TargetPinObj != StartingPinObj)
			{
				// The Graph object in which the pins reside.
				UEdGraph* GraphObj = StartingPinObj->GetOwningNode()->GetGraph();

				// Determine what the schema thinks
				FPinConnectionResponse Response;
				const UEdGraphSchema* Schema = GraphObj->GetSchema();
				switch (DragMode)
				{
				case EDragMode::CreateConnection:
					Response = Schema->CanCreateConnection( StartingPinObj, TargetPinObj );
					if( Response.Response == CONNECT_RESPONSE_DISALLOW && 
						Schema->ShouldShowPanelContextMenuForIncompatibleConnections())
					{
						// override the response, these pins aren't compatible but in this context
						// we can show the panel context menu listing compatible nodes:
						Response = FPinConnectionResponse(
							CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE, 
							FString::Printf(
								TEXT("Search for Nodes matching %s (%s) and %s (%s)"), 
								*UEdGraphSchema_K2::TypeToText(StartingPinObj->PinType).ToString(), 
								LexToString(StartingPinObj->Direction), 
								*UEdGraphSchema_K2::TypeToText(TargetPinObj->PinType).ToString(), 
								LexToString(TargetPinObj->Direction)));
					}
					break;
				case EDragMode::RelinkConnection:
					Response = Schema->CanRelinkConnectionToPin(StartingPinObj, TargetPinObj);
					break;
				}

				if (Response.Response == ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW)
				{
					TSharedPtr<SGraphNode> NodeWidget = TargetPinObj->GetOwningNode()->DEPRECATED_NodeWidget.Pin();
					 if (NodeWidget.IsValid())
					 {
						 NodeWidget->NotifyDisallowedPinConnection(StartingPinObj, TargetPinObj);
					 }
				}

				UniqueMessages.AddUnique(Response);
			}
		}
	}
	else if(GetHoveredNode() && DragMode == EDragMode::CreateConnection)
	{
		TArray<UEdGraphPin*> ValidSourcePins;
		ValidateGraphPinList(/*out*/ ValidSourcePins);

		// Check the schema for connection responses
		for (UEdGraphPin* StartingPinObj : ValidSourcePins)
		{
			FPinConnectionResponse Response;
			FText ResponseText;
			if (StartingPinObj->GetOwningNode() != GetHoveredNode() && StartingPinObj->GetSchema()->SupportsDropPinOnNode(GetHoveredNode(), StartingPinObj->PinType, StartingPinObj->Direction, ResponseText))
			{
				Response.Response = ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE;
			}
			else
			{
				Response.Response = ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW;
			}
			
			// Do not display an error if there is no message
			if (!ResponseText.IsEmpty())
			{
				Response.Message = ResponseText;
				UniqueMessages.AddUnique(Response);
			}
		}
	}
	else if(GetHoveredGraph() && DragMode == EDragMode::CreateConnection)
	{
		TArray<UEdGraphPin*> ValidSourcePins;
		ValidateGraphPinList(/*out*/ ValidSourcePins);

		for (UEdGraphPin* StartingPinObj : ValidSourcePins)
		{
			// Let the schema describe the connection we might make
			FPinConnectionResponse Response = GetHoveredGraph()->GetSchema()->CanCreateNewNodes(StartingPinObj);
			if(!Response.Message.IsEmpty())
			{
				UniqueMessages.AddUnique(Response);
			}
		}
	}

	// Let the user know the status of dropping now
	if (UniqueMessages.Num() == 0 && DragMode == EDragMode::CreateConnection)
	{
		// Display the place a new node icon, we're not over a valid pin and have no message from the schema
		SetSimpleFeedbackMessage(
			FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.NewNode")),
			FLinearColor::White,
			NSLOCTEXT("GraphEditor.Feedback", "PlaceNewNode", "Place a new node."));
	}
	else
	{
		// Take the unique responses and create visual feedback for it
		TSharedRef<SVerticalBox> FeedbackBox = SNew(SVerticalBox);
		for (auto ResponseIt = UniqueMessages.CreateConstIterator(); ResponseIt; ++ResponseIt)
		{
			// Determine the icon
			const FSlateBrush* StatusSymbol = NULL;

			switch (ResponseIt->Response)
			{
			case CONNECT_RESPONSE_MAKE:
			case CONNECT_RESPONSE_BREAK_OTHERS_A:
			case CONNECT_RESPONSE_BREAK_OTHERS_B:
			case CONNECT_RESPONSE_BREAK_OTHERS_AB:
				StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				break;

			case CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE:
				StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.ViaCast"));
				break;

			case CONNECT_RESPONSE_MAKE_WITH_PROMOTION:
				StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.ViaCast"));
				break;

			case CONNECT_RESPONSE_DISALLOW:
			default:
				StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				break;
			}

			// Add a new message row
			FeedbackBox->AddSlot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage) .Image( StatusSymbol )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock) .Text( ResponseIt->Message )
				]
			];
		}

		SetFeedbackMessage(FeedbackBox);
	}
}

FDragConnection::FDragConnection(const TSharedRef<SGraphPanel>& GraphPanelIn, const FDraggedPinTable& DraggedPinsIn)
	: GraphPanel(GraphPanelIn)
	, DraggingPins(DraggedPinsIn)
	, DecoratorAdjust(FSlateApplication::Get().GetCursorSize())
{
	// Handle connection relinking, if supported by the schema
	const FGraphSplineOverlapResult& PreviousFrameSplineOverlap = GraphPanel->GetPreviousFrameSplineOverlap();
	FGraphPinHandle Pin1Handle = PreviousFrameSplineOverlap.GetPin1Handle();
	if (Pin1Handle.IsValid())
	{
		UEdGraphPin* SourcePin = Pin1Handle.GetPinObj(*GraphPanel.Get());
		if (SourcePin)
		{
			const UEdGraphSchema* Schema = SourcePin->GetSchema();
			if (Schema->IsConnectionRelinkingAllowed(SourcePin))
			{
				FGraphPinHandle Pin2Handle = PreviousFrameSplineOverlap.GetPin2Handle();
				UEdGraphPin* TargetPin = Pin2Handle.GetPinObj(*GraphPanel.Get());
				if (TargetPin)
				{
					SourcePinHandle = SourcePin;
					TargetPinHandle = TargetPin;
					DragMode = RelinkConnection;
				}
			}
		}
	}

	switch (DragMode)
	{
	case EDragMode::CreateConnection:
	{
		if (DraggingPins.Num() > 0)
		{
			const UEdGraphPin* PinObj = FDraggedPinTable::TConstIterator(DraggedPinsIn)->GetPinObj(*GraphPanelIn);
			if (PinObj && PinObj->Direction == EGPD_Input)
			{
				DecoratorAdjust *= FVector2f(-1.0f, 1.0f);
			}
		}

		for (const FGraphPinHandle& DraggedPin : DraggedPinsIn)
		{
			GraphPanelIn->OnBeginMakingConnection(DraggedPin);
		}
		break;
	}

	case EDragMode::RelinkConnection:
	{
		if (SourcePinHandle.IsValid())
		{
			const UEdGraphPin* PinObj = SourcePinHandle.GetPinObj(*GraphPanel.Get());
			if (PinObj && PinObj->Direction == EGPD_Input)
			{
				DecoratorAdjust *= FVector2f(-1.0f, 1.0f);
			}
		}

		GraphPanel->OnBeginRelinkConnection(SourcePinHandle, TargetPinHandle);
		break;
	}
	}
}

FReply FDragConnection::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	bool bError = false;
	TSet<FEdGraphNodeHandle> NodeList;
	const UEdGraphSchema* Schema = nullptr;

	switch (DragMode)
	{
	case EDragMode::CreateConnection:
	{
		TArray<UEdGraphPin*> ValidSourcePins;
		ValidateGraphPinList(/*out*/ ValidSourcePins);

		// store the pins as pin tuples since the structure of the
		// graph may change during the creation of a connection
		TArray<FEdGraphPinHandle> ValidSourcePinHandles;
		for (const UEdGraphPin* ValidSourcePin : ValidSourcePins)
		{
			ValidSourcePinHandles.Add(ValidSourcePin);
		}

		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_CreateConnection", "Create Pin Link"));

		FEdGraphPinHandle PinB(GetHoveredPin());

		for (const FEdGraphPinHandle& PinA : ValidSourcePinHandles)
		{
			if ((PinA.GetPin() != NULL) && (PinB.GetPin() != NULL))
			{
				const UEdGraph* MyGraphObj = PinA.GetGraph();

				// the pin may change during the creation of the link
				Schema = MyGraphObj->GetSchema();
				if (Schema->TryCreateConnection(PinA.GetPin(), PinB.GetPin()))
				{
					if (PinA.GetPin() && !PinA.GetPin()->IsPendingKill())
					{
						NodeList.Add(PinA.GetPin()->GetOwningNode());
					}
					if (PinB.GetPin() && !PinB.GetPin()->IsPendingKill())
					{
						NodeList.Add(PinB.GetNode());
					}
				}
			}
			else
			{
				bError = true;
			}
		}
		break;
	}

	case EDragMode::RelinkConnection:
	{
		const UEdGraphPin* SourceGraphPin = SourcePinHandle.GetPinObj(*GraphPanel);
		FEdGraphPinHandle SourceGraphPinHandle = SourceGraphPin;

		UEdGraphPin* TargetGraphPin = TargetPinHandle.GetPinObj(*GraphPanel);
		FEdGraphPinHandle TargetGraphPinHandle = SourceGraphPin;

		FEdGraphPinHandle HoveredGraphPinHandle(GetHoveredPin());
		UEdGraphPin* HoveredGraphPin = HoveredGraphPinHandle.GetPin();
		
		if (SourceGraphPin && TargetGraphPin && HoveredGraphPin)
		{
			const UEdGraph* Graph = SourceGraphPinHandle.GetGraph();
			if (Graph)
			{
				const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_RelinkConnection", "Relink Connection"));

				Schema = Graph->GetSchema();
				const TArray<UEdGraphNode*> SelectedGraphNodes = GraphPanel->GetSelectedGraphNodes();
				bError = !Schema->TryRelinkConnectionTarget(SourceGraphPinHandle.GetPin(), TargetGraphPin, HoveredGraphPinHandle.GetPin(), SelectedGraphNodes);				if (!bError)
				{
					// Send all nodes that received a new pin connection a notification
					TArray<FEdGraphPinHandle> ModifiedGraphPinHandles = { SourceGraphPinHandle, TargetGraphPinHandle, HoveredGraphPinHandle };
					for (const FEdGraphPinHandle& PinHandle : ModifiedGraphPinHandles)
					{
						if (PinHandle.GetPin() && !PinHandle.GetPin()->IsPendingKill() && PinHandle.GetNode())
						{
							NodeList.Add(PinHandle.GetNode());
						}
					}
				}
			}
		}

		break;
	}
	}

	// Send all nodes that received a new pin connection a notification
	for (auto It = NodeList.CreateConstIterator(); It; ++It)
	{
		if(UEdGraphNode* Node = It->GetNode())
		{
			Node->NodeConnectionListChanged();
		}
	}

	if( Schema && Schema->ShouldShowPanelContextMenuForIncompatibleConnections() && 
		NodeList.IsEmpty())
	{
		// no connections made, lets show a list of compatible nodes if any:	
		TArray<UEdGraphPin*> PinObjects;
		ValidateGraphPinList(/*out*/ PinObjects); // this gathers source pins for the drag operation
		if(UEdGraphPin* CurrentHoveredPin = GetHoveredPin())
		{
			if(PinObjects.Contains(CurrentHoveredPin))
			{
				PinObjects.Reset(); // user dropped a pin on itself, ignore
			}
			else
			{
				PinObjects.Add(CurrentHoveredPin);
			}
		}

		if(PinObjects.Num() > 0 )
		{
			TSharedPtr<SWidget> WidgetToFocus = GraphPanel->SummonContextMenu(
				ScreenPosition, GraphPosition, nullptr, nullptr, PinObjects);		
			if (WidgetToFocus.IsValid())
			{
				return FReply::Handled().SetUserFocus(WidgetToFocus.ToSharedRef(), EFocusCause::SetDirectly);
			}
		}
	}

	if (bError)
	{
		return FReply::Unhandled();
	}

	return FReply::Handled();
}

FReply FDragConnection::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	if (DragMode != EDragMode::CreateConnection)
	{
		return FReply::Unhandled();
	}

	bool bHandledPinDropOnNode = false;
	UEdGraphNode* NodeOver = GetHoveredNode();

	if (NodeOver)
	{
		// Gather any source drag pins
		TArray<UEdGraphPin*> ValidSourcePins;
		ValidateGraphPinList(/*out*/ ValidSourcePins);

		if (ValidSourcePins.Num())
		{
			for (UEdGraphPin* SourcePin : ValidSourcePins)
			{
				// copy it here since the pin might no longer be valid
				const UEdGraphSchema* SourcePinSchema = SourcePin->GetSchema();
				SourcePinSchema->SetPinBeingDroppedOnNode(SourcePin);

				// Check for pin drop support
				FText ResponseText;
				if (SourcePin->GetOwningNode() != NodeOver && SourcePinSchema->SupportsDropPinOnNode(NodeOver, SourcePin->PinType, SourcePin->Direction, ResponseText))
				{
					bHandledPinDropOnNode = true;

					// Find which pin name to use and drop the pin on the node
					const FName PinName = SourcePin->PinFriendlyName.IsEmpty()? SourcePin->PinName : *SourcePin->PinFriendlyName.ToString();

					const FScopedTransaction Transaction((SourcePin->Direction == EGPD_Output) ? NSLOCTEXT("UnrealEd", "AddInParam", "Add In Parameter" ) : NSLOCTEXT("UnrealEd", "AddOutParam", "Add Out Parameter"));

					UEdGraphPin* EdGraphPin = NodeOver->GetSchema()->DropPinOnNode(GetHoveredNode(), PinName, SourcePin->PinType, SourcePin->Direction);

					// This can invalidate the source pin due to node reconstruction, abort in that case
					if(SourcePin->GetOwningNodeUnchecked() && EdGraphPin)
					{
						SourcePin->Modify();
						EdGraphPin->Modify();
						SourcePinSchema->TryCreateConnection(SourcePin, EdGraphPin);
					}
				}

				// If we have not handled the pin drop on node and there is an error message, do not let other actions occur.
				if(!bHandledPinDropOnNode && !ResponseText.IsEmpty())
				{
					bHandledPinDropOnNode = true;
				}

				SourcePinSchema->SetPinBeingDroppedOnNode(nullptr);
			}
		}
	}
	return bHandledPinDropOnNode? FReply::Handled() : FReply::Unhandled();
}

FReply FDragConnection::DroppedOnPanel( const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	if (DragMode != EDragMode::CreateConnection)
	{
		return FReply::Unhandled();
	}

	// Gather any source drag pins
	TArray<UEdGraphPin*> PinObjects;
	ValidateGraphPinList(/*out*/ PinObjects);

	// Create a context menu
	TSharedPtr<SWidget> WidgetToFocus = GraphPanel->SummonContextMenu(ScreenPosition, GraphPosition, NULL, NULL, PinObjects);

	// Give the context menu focus
	return (WidgetToFocus.IsValid())
		? FReply::Handled().SetUserFocus(WidgetToFocus.ToSharedRef(), EFocusCause::SetDirectly)
		: FReply::Handled();
}


void FDragConnection::ValidateGraphPinList(TArray<UEdGraphPin*>& OutValidPins)
{
	OutValidPins.Empty(DraggingPins.Num());
	for (const FGraphPinHandle& PinHandle : DraggingPins)
	{
		if (UEdGraphPin* GraphPin = PinHandle.GetPinObj(*GraphPanel))
		{
			OutValidPins.Add(GraphPin);
		}
	}
}
