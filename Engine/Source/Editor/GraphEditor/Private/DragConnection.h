// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "GraphEditorDragDropAction.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"

class SGraphPanel;
class SWidget;
class UEdGraph;
class UEdGraphPin;
struct FPointerEvent;

class FDragConnection : public FGraphEditorDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDragConnection, FGraphEditorDragDropAction)

	enum EDragMode : uint8
	{
		CreateConnection = 0,
		RelinkConnection
	};

	typedef TArray<FGraphPinHandle> FDraggedPinTable;
	static TSharedRef<FDragConnection> New(const TSharedRef<SGraphPanel>& InGraphPanel, const FDraggedPinTable& InStartingPins);

	// FDragDropOperation interface
	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override;
	// End of FDragDropOperation interface

	// FGraphEditorDragDropAction interface
	virtual void HoverTargetChanged() override;
	virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	virtual FReply DroppedOnPanel(const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;
	// End of FGraphEditorDragDropAction interface

	/*
	 *	Function to check validity of graph pins in the StartPins list. This check helps to prevent processing graph pins which are outdated.
	 */
	virtual void ValidateGraphPinList(TArray<UEdGraphPin*>& OutValidPins);

protected:
	typedef FGraphEditorDragDropAction Super;

	// Constructor: Make sure to call Construct() after factorying one of these
	FDragConnection(const TSharedRef<SGraphPanel>& GraphPanel, const FDraggedPinTable& DraggedPins);

protected:
	TSharedPtr<SGraphPanel> GraphPanel;
	FDraggedPinTable DraggingPins;
	EDragMode DragMode = EDragMode::CreateConnection;

	/** Offset information for the decorator widget */
	FDeprecateSlateVector2D DecoratorAdjust;

	FGraphPinHandle SourcePinHandle;
	FGraphPinHandle TargetPinHandle;
};
