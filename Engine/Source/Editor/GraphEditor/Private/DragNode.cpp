// Copyright Epic Games, Inc. All Rights Reserved.


#include "DragNode.h"

#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FSlateBrush;

TSharedRef<FDragNode> FDragNode::New(const TSharedRef<SGraphPanel>& InGraphPanel, const TSharedRef<SGraphNode>& InDraggedNode)
{
	TSharedRef<FDragNode> Operation = MakeShareable(new FDragNode);
	Operation->GraphPanel = InGraphPanel;
	Operation->DraggedNodes.Add(InDraggedNode);
	// adjust the decorator away from the current mouse location a small amount based on cursor size
	Operation->DecoratorAdjust = FSlateApplication::Get().GetCursorSize();
	Operation->Construct();
	return Operation;
}

TSharedRef<FDragNode> FDragNode::New(const TSharedRef<SGraphPanel>& InGraphPanel, const TArray< TSharedRef<SGraphNode> >& InDraggedNodes)
{
	TSharedRef<FDragNode> Operation = MakeShareable(new FDragNode);
	Operation->GraphPanel = InGraphPanel;
	Operation->DraggedNodes = InDraggedNodes;
	Operation->DecoratorAdjust = FSlateApplication::Get().GetCursorSize();
	Operation->Construct();
	return Operation;
}

const TArray< TSharedRef<SGraphNode> > & FDragNode::GetNodes() const
{
	return DraggedNodes;
}

bool FDragNode::IsValidOperation() const
{
	return bValidOperation;
}

UEdGraphNode* FDragNode::GetGraphNodeForSGraphNode(TSharedRef<SGraphNode>& SNode)
{
	return SNode->GetNodeObj();
}

void FDragNode::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	FVector2f TargetPosition = DragDropEvent.GetScreenSpacePosition();

	// Reposition the info window to the dragged position
	CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition() + DecoratorAdjust);
	// Request the active panel to scroll if required
	GraphPanel->RequestDeferredPan(TargetPosition);
}

void FDragNode::HoverTargetChanged()
{
	TArray<FPinConnectionResponse> UniqueMessages;
	UEdGraphNode* TargetNodeObj = GetHoveredNode();
	
	if (TargetNodeObj != NULL)
	{
		// Check the schema for connection responses
		for (TArray< TSharedRef<SGraphNode> >::TIterator NodeIterator(DraggedNodes); NodeIterator; ++NodeIterator)
		{
			UEdGraphNode* DraggedNodeObj = GetGraphNodeForSGraphNode(*NodeIterator);

			// The Graph object in which the nodes reside.
			UEdGraph* GraphObj = DraggedNodeObj->GetGraph();

			// Determine what the schema thinks about the action
			const FPinConnectionResponse Response = GraphObj->GetSchema()->CanMergeNodes( DraggedNodeObj, TargetNodeObj );

			UniqueMessages.AddUnique(Response);
		}
	}

	// Let the user know the status of dropping now
	if (UniqueMessages.Num() == 0)
	{
		bValidOperation = false;
		// Display the place a new node icon, we're not over a valid pin
		SetSimpleFeedbackMessage(
			FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")),
			FLinearColor::White,
			NSLOCTEXT("GraphEditor.Feedback", "DragNode", "This node cannot be placed here."));
	}
	else
	{
		bValidOperation = true;
		// Take the unique responses and create visual feedback for it
		TSharedRef<SVerticalBox> FeedbackBox = SNew(SVerticalBox);
		for (auto ResponseIt = UniqueMessages.CreateConstIterator(); ResponseIt; ++ResponseIt)
		{
			// Determine the icon
			const FSlateBrush* StatusSymbol = NULL;

			switch (ResponseIt->Response)
			{
			case CONNECT_RESPONSE_MAKE:
				StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
				break;

			case CONNECT_RESPONSE_DISALLOW:
			default:
				StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
				bValidOperation = false;
				break;
			}

			// Add a new message row
			FeedbackBox->AddSlot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f)
				[
					SNew(SImage) .Image( StatusSymbol )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock) .Text( ResponseIt->Message )
				]
			];
		}

		for (int32 i=0; i < DraggedNodes.Num(); i++)
		{
			FeedbackBox->AddSlot()
			.AutoHeight()
			[
				DraggedNodes[i]
			];
		}


		SetFeedbackMessage(FeedbackBox);
	}
}

FReply FDragNode::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	return FReply::Handled();
}

FReply FDragNode::DroppedOnPanel( const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	return FReply::Handled();
}
