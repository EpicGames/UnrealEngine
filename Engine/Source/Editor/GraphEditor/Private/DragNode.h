// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "GraphEditorDragDropAction.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

#define UE_API GRAPHEDITOR_API

class SGraphNode;
class SGraphPanel;
class SWidget;
class UEdGraph;
class UEdGraphNode;

class FDragNode : public FGraphEditorDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDragNode, FGraphEditorDragDropAction)

	static UE_API TSharedRef<FDragNode> New(const TSharedRef<SGraphPanel>& InGraphPanel, const TSharedRef<SGraphNode>& InDraggedNode);

	static UE_API TSharedRef<FDragNode> New(const TSharedRef<SGraphPanel>& InGraphPanel, const TArray< TSharedRef<SGraphNode> >& InDraggedNodes);
	
	// FGraphEditorDragDropAction interface
	UE_API virtual void HoverTargetChanged() override;
	UE_API virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	UE_API virtual FReply DroppedOnPanel( const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	//virtual void OnDragBegin( const TSharedRef<class SGraphPin>& InPin) override;
	UE_API virtual void OnDragged (const class FDragDropEvent& DragDropEvent ) override;
	// End of FGraphEditorDragDropAction interface
	
	UE_API const TArray< TSharedRef<SGraphNode> > & GetNodes() const;

	UE_API virtual bool IsValidOperation() const;

protected:
	typedef FGraphEditorDragDropAction Super;

	UE_API UEdGraphNode* GetGraphNodeForSGraphNode(TSharedRef<SGraphNode>& SNode);

protected:

	/** graph panel */
	TSharedPtr<SGraphPanel> GraphPanel;

	/** our dragged nodes*/
	TArray< TSharedRef<SGraphNode> > DraggedNodes;

	/** Offset information for the decorator widget */
	FVector2f	DecoratorAdjust;

	/** if we can drop our node here*/
	bool bValidOperation;
};

#undef UE_API
