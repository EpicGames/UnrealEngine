// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindInGraph.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "EdGraphNode_Comment.h"
#include "EdGraph/EdGraphNode.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "FindInGraph"

//////////////////////////////////////////////////////////////////////////
// FFindInGraphResult

FFindInGraphResult::FFindInGraphResult(const FCreateParams& InCreateParams)
	: Parent(InCreateParams.Parent)
	, Value(InCreateParams.Value)
	, DuplicationIndex(InCreateParams.DuplicationIndex)
	, Class(InCreateParams.Class)
	, Pin(InCreateParams.Pin)
	, GraphNode(InCreateParams.GraphNode)
{
	if (!Class && InCreateParams.GraphNode)
	{
		Class = InCreateParams.GraphNode->GetClass();
	}

	if (GraphNode.IsValid())
	{
		CommentText = GraphNode->NodeComment;
	}
}

void FFindInGraphResult::JumpToNode(TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit, const UEdGraphNode* InNode) const
{
}

FReply FFindInGraphResult::OnClick(TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit)
{
	if (GraphNode.IsValid())
	{
		JumpToNode(AssetEditorToolkit, GraphNode.Get());
	}
	else if (UEdGraphPin* ResolvedPin = Pin.Get())
	{
		JumpToNode(AssetEditorToolkit, ResolvedPin->GetOwningNode());
	}
	return FReply::Handled();
}

FText FFindInGraphResult::GetCategory() const
{
	if (Class == nullptr && Pin.Get())
	{
		return LOCTEXT("PinCategory", "Pin");
	}

	return LOCTEXT("NodeCategory", "Node");
}

TSharedRef<SWidget> FFindInGraphResult::CreateIcon() const
{
	FSlateColor IconColor = FSlateColor::UseForeground();
	const FSlateBrush* Brush = NULL;
	if (UEdGraphPin* ResolvedPin = Pin.Get())
	{
		if (ResolvedPin->PinType.IsArray())
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.ArrayPinIcon"));
		}
		else if (ResolvedPin->PinType.bIsReference)
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.RefPinIcon"));
		}
		else
		{
			Brush = FAppStyle::GetBrush(TEXT("GraphEditor.PinIcon"));
		}
		const UEdGraphSchema* Schema = ResolvedPin->GetSchema();
			IconColor = Schema->GetPinTypeColor(ResolvedPin->PinType);
		}
	else if (GraphNode.IsValid())
	{
		Brush = FAppStyle::GetBrush(TEXT("GraphEditor.NodeGlyph"));
	}

	return 	SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor)
		.ToolTipText(GetCategory());
}

FString FFindInGraphResult::GetCommentText() const
{
	return CommentText;
}

FString FFindInGraphResult::GetValueText() const
{
	FString DisplayVal;

	if (UEdGraphPin* ResolvedPin = Pin.Get())
	{
		if ((!ResolvedPin->DefaultValue.IsEmpty() || !ResolvedPin->AutogeneratedDefaultValue.IsEmpty() || ResolvedPin->DefaultObject || !ResolvedPin->DefaultTextValue.IsEmpty())
			&& (ResolvedPin->Direction == EGPD_Input)
			&& (ResolvedPin->LinkedTo.Num() == 0))
		{
			if (ResolvedPin->DefaultObject)
			{
				const AActor* DefaultActor = Cast<AActor>(ResolvedPin->DefaultObject);
				DisplayVal = FString::Printf(TEXT("(%s)"), DefaultActor ? *DefaultActor->GetActorLabel() : *ResolvedPin->DefaultObject->GetName());
			}
			else if (!ResolvedPin->DefaultValue.IsEmpty())
			{
				DisplayVal = FString::Printf(TEXT("(%s)"), *ResolvedPin->DefaultValue);
			}
			else if (!ResolvedPin->AutogeneratedDefaultValue.IsEmpty())
			{
				DisplayVal = FString::Printf(TEXT("(%s)"), *ResolvedPin->AutogeneratedDefaultValue);
			}
			else if (!ResolvedPin->DefaultTextValue.IsEmpty())
			{
				DisplayVal = FString::Printf(TEXT("(%s)"), *ResolvedPin->DefaultTextValue.ToString());
			}
		}
	}

	return DisplayVal;
}

//////////////////////////////////////////////////////////////////////////
// SFindInGraph

void SFindInGraph::Construct(const FArguments& InArgs, TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit)
{
	AssetEditorToolkitPtr = InAssetEditorToolkit;

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SAssignNew(SearchTextField, SSearchBox)
				.HintText(LOCTEXT("GraphSearchHint", "Search"))
				.OnTextChanged(this, &SFindInGraph::OnSearchTextChanged)
				.OnTextCommitted(this, &SFindInGraph::OnSearchTextCommitted)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.f, 4.f, 0.f, 0.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SAssignNew(TreeView, STreeViewType)
				.TreeItemsSource(&ItemsFound)
				.OnGenerateRow(this, &SFindInGraph::OnGenerateRow)
				.OnGetChildren(this, &SFindInGraph::OnGetChildren)
				.OnSelectionChanged(this, &SFindInGraph::OnTreeSelectionChanged)
				.OnMouseButtonDoubleClick(this, &SFindInGraph::OnTreeSelectionDoubleClick)
				.SelectionMode(ESelectionMode::Multi)
			]
		]
	];
}

void SFindInGraph::FocusForUse()
{
	// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
	FWidgetPath FilterTextBoxWidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath);

	// Set keyboard focus directly
	FSlateApplication::Get().SetKeyboardFocus(FilterTextBoxWidgetPath, EFocusCause::SetDirectly);
}

TSharedPtr<FFindInGraphResult> SFindInGraph::MakeSearchResult(const FFindInGraphResult::FCreateParams& InParams)
{
	return MakeShared<FFindInGraphResult>(InParams);
}

void SFindInGraph::OnSearchTextChanged(const FText& Text)
{
	SearchValue = Text.ToString();
}

void SFindInGraph::OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		InitiateSearch();
	}
}

void SFindInGraph::InitiateSearch()
{
	TArray<FString> Tokens;
	if (SearchValue.Contains("\"") && SearchValue.ParseIntoArray(Tokens, TEXT("\""), true) > 0)
	{
		for (FString& Token : Tokens)
		{
			// We have the token, we don't need the quotes anymore, they'll just confused the comparison later on
			Token = Token.TrimQuotes();

			// We remove the spaces as all later comparison strings will also be de-spaced
			Token = Token.Replace(TEXT(" "), TEXT(""));
		}

		// due to being able to handle multiple quoted blocks like ("Make Epic" "Game Now") we can end up with
		// and empty string between (" ") blocks so this simply removes them
		struct FRemoveMatchingStrings
		{
			bool operator()(const FString& RemovalCandidate) const
			{
				return RemovalCandidate.IsEmpty();
			}
		};
		Tokens.RemoveAll(FRemoveMatchingStrings());
	}
	else
	{
		// unquoted search equivalent to a match-any-of search
		SearchValue.ParseIntoArray(Tokens, TEXT(" "), true);
	}

	for (FSearchResult& ItemFound : ItemsFound)
	{
		TreeView->SetItemExpansion(ItemFound, false);
	}
	ItemsFound.Empty();

	if (Tokens.Num() > 0)
	{
		HighlightText = FText::FromString(SearchValue);
		MatchTokens(Tokens);
	}

	// Insert a fake result to inform user if none found
	if (ItemsFound.Num() == 0)
	{
		ItemsFound.Add(MakeSearchResult({ LOCTEXT("GraphSearchNoResults", "No Results found").ToString() }));
	}

	TreeView->RequestTreeRefresh();

	for (FSearchResult& ItemFound : ItemsFound)
	{
		TreeView->SetItemExpansion(ItemFound, true);
	}
}

const UEdGraph* SFindInGraph::GetGraph()
{
	return nullptr;
}

void SFindInGraph::MatchTokens(const TArray<FString> &Tokens)
{
	RootSearchResult.Reset();

	if (const UEdGraph* Graph = GetGraph())
	{
		MatchTokensInGraph(Graph, Tokens);
	}
}

bool SFindInGraph::MatchTokensInNode(const UEdGraphNode* Node, const TArray<FString>& Tokens)
{
	return false;
}

void SFindInGraph::MatchTokensInGraph(const UEdGraph* Graph, const TArray<FString>& Tokens)
{
	if (!Graph)
	{
		return;
	}

	RootSearchResult = MakeSearchResult({ FString("GraphSearchTreeRoot") });

	for (UEdGraphNode* Node : Graph->Nodes)
	{
		const FString NodeName = Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString();
		const FString NodeType = Node->GetNodeTitle(ENodeTitleType::ListView).ToString();

		FSearchResult NodeResult = MakeSearchResult(
		{ 
			.Value = NodeName == NodeType ? NodeName : NodeName + " - " + NodeType, 
			.Parent = RootSearchResult, 
			.GraphNode = Node 
		});

		FString NodeSearchString = NodeName + NodeType + Node->NodeComment;
		NodeSearchString = NodeSearchString.Replace(TEXT(" "), TEXT(""));

		bool bNodeMatchesSearch = StringMatchesSearchTokens(Tokens, NodeSearchString);

		// Allow for node-type specific searches to trigger positive matches if we didn't already find one
		if (!bNodeMatchesSearch)
		{
			bNodeMatchesSearch = MatchTokensInNode(Node, Tokens);
		}

		for (UEdGraphPin* Pin : Node->Pins)
		{
			if (Pin && Pin->PinFriendlyName.CompareTo(FText::FromString(TEXT(" "))) != 0)
			{
				FText PinName = Pin->GetSchema()->GetPinDisplayName(Pin);
				FString PinSearchString = Pin->PinName.ToString() 
					+ Pin->PinFriendlyName.ToString() 
					+ Pin->DefaultValue 
					+ Pin->PinType.PinCategory.ToString() 
					+ Pin->PinType.PinSubCategory.ToString() 
					+ (Pin->PinType.PinSubCategoryObject.IsValid() ? Pin->PinType.PinSubCategoryObject.Get()->GetFullName() : TEXT(""));

				PinSearchString = PinSearchString.Replace(TEXT(" "), TEXT(""));
				if (StringMatchesSearchTokens(Tokens, PinSearchString))
				{
					FSearchResult PinResult = MakeSearchResult(
					{
						.Value = PinName.ToString(),
						.Parent = NodeResult,
						.Pin = Pin
					});
					NodeResult->Children.Add(PinResult);
				}
			}
		}

		if ((NodeResult->Children.Num() > 0) || bNodeMatchesSearch)
		{
			ItemsFound.Add(NodeResult);
		}
	}

	for (const UEdGraph* Subgraph : Graph->SubGraphs)
	{
		MatchTokensInGraph(Subgraph, Tokens);
	}
}

TSharedRef<ITableRow> SFindInGraph::OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FString CommentText = InItem->GetCommentText();

	return SNew(STableRow< TSharedPtr<FFindInGraphResult> >, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				InItem->CreateIcon()
			]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(InItem->Value))
					.HighlightText(HighlightText)
					.ToolTipText(FText::Format(LOCTEXT("GraphResultSearchToolTipFmt", "{0} : {1}"), InItem->GetCategory(), FText::FromString(InItem->Value)))
				]
			+ SHorizontalBox::Slot()
				.FillWidth(1)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(InItem->GetValueText()))
					.HighlightText(HighlightText)
				]
			+ SHorizontalBox::Slot()
				.FillWidth(1)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2, 0)
				[
					SNew(STextBlock)
					.Text(CommentText.IsEmpty() ? FText::GetEmpty() : FText::Format(LOCTEXT("NodeCommentFmt", "Node Comment:[{0}]"), FText::FromString(CommentText)))
					.ColorAndOpacity(FLinearColor::Yellow)
					.HighlightText(HighlightText)
				]
		];
}

void SFindInGraph::OnGetChildren(FSearchResult InItem, TArray< FSearchResult >& OutChildren)
{
	OutChildren += InItem->Children;
}

void SFindInGraph::OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type)
{
	if (Item.IsValid())
	{
		Item->OnClick(AssetEditorToolkitPtr);
	}
}

void SFindInGraph::OnTreeSelectionDoubleClick(FSearchResult Item)
{
	if (Item.IsValid())
	{
		Item->OnClick(AssetEditorToolkitPtr);
	}
}

bool SFindInGraph::StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString)
{
	bool bFoundAllTokens = true;
	//search the entry for each token, it must have all of them to pass
	for (const FString& Token : Tokens)
	{
		if (!ComparisonString.Contains(Token))
		{
			bFoundAllTokens = false;
			break;
		}
	}
	return bFoundAllTokens;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
