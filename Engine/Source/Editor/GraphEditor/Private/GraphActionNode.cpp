// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphActionNode.h"

#include "Containers/BitArray.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

/*******************************************************************************
 * Static FGraphActionNode Helpers
 ******************************************************************************/

struct FGraphActionNodeImpl
{
	static const int32 DEFAULT_GROUPING = 0;

	/**
	 * Utility sort function. Compares nodes based off of section, grouping, and
	 * type.
	 * 
	 * @param  LhsMenuNodePtr	The node to determine if it should come first.
	 * @param  RhsMenuNodePtr	The node to determine if it should come second.
	 * @return True if LhsMenuNodePtr should come before RhsMenuNodePtr.
	 */
	static bool NodeCompare(TSharedPtr<FGraphActionNode> const& LhsMenuNodePtr, TSharedPtr<FGraphActionNode> const& RhsMenuNodePtr);

	/**
	 * Utility sort function. Compares nodes based off of section, grouping, 
	 * type, and then alphabetically.
	 * 
	 * @param  LhsMenuNodePtr	The node to determine if it should come first.
	 * @param  RhsMenuNodePtr	The node to determine if it should come second.
	 * @return True if LhsMenuNodePtr should come before RhsMenuNodePtr.
	 */
	static bool AlphabeticalNodeCompare(TSharedPtr<FGraphActionNode> const& LhsMenuNodePtr, TSharedPtr<FGraphActionNode> const& RhsMenuNodePtr);
};

//------------------------------------------------------------------------------
bool FGraphActionNodeImpl::NodeCompare(TSharedPtr<FGraphActionNode> const& LhsMenuNodePtr, TSharedPtr<FGraphActionNode> const& RhsMenuNodePtr)
{
	FGraphActionNode* LhsMenuNode = LhsMenuNodePtr.Get();
	FGraphActionNode* RhsMenuNode = RhsMenuNodePtr.Get();

	bool const bLhsIsCategory  = LhsMenuNode->IsCategoryNode();
	bool const bRhsIsCategory  = RhsMenuNode->IsCategoryNode();
	bool const bLhsIsSeparator = LhsMenuNode->IsSeparator();
	bool const bRhsIsSeparator = RhsMenuNode->IsSeparator();
	bool const bLhsIsSectionHeader = LhsMenuNode->IsSectionHeadingNode();
	bool const bRhsIsSectionHeader = RhsMenuNode->IsSectionHeadingNode();

	if (LhsMenuNode->SectionID != RhsMenuNode->SectionID)
	{
		// since we don't add section headers for children that have the same
		// section as their parents (the header is above the parent), we need to
		// organize them first (so they're seemingly under the same header)
		if ((LhsMenuNode->SectionID == LhsMenuNode->ParentNode.Pin()->SectionID) && 
			(LhsMenuNode->SectionID != FGraphActionNode::INVALID_SECTION_ID))
		{
			return true;
		}
		else // otherwise...
		{
			// sections are ordered in ascending order
			return (LhsMenuNode->SectionID < RhsMenuNode->SectionID);
		}		
	}
	else if (bLhsIsSectionHeader != bRhsIsSectionHeader)
	{
		// section headers go to the top of that section
		return bLhsIsSectionHeader;
	}
	else if (LhsMenuNode->Grouping != RhsMenuNode->Grouping)
	{
		// groups are ordered in descending order
		return (LhsMenuNode->Grouping >= RhsMenuNode->Grouping);
	}
	// next, make sure separators are preserved
	else if (bLhsIsSeparator != bRhsIsSeparator)
	{
		// separators with the same grouping go to the bottom of that "group"
		return bRhsIsSeparator;
	}
	// next, categories get listed before action nodes
	else if (bLhsIsCategory != bRhsIsCategory)
	{
		return bLhsIsCategory;
	}
	else
	{
		// both lhs and rhs are seemingly the same, so to keep them menu from 
		// jumping around everytime an entry is added, we sort by the order they
		// were inserted
		return (LhsMenuNode->InsertOrder < RhsMenuNode->InsertOrder);
	}
}

//------------------------------------------------------------------------------
bool FGraphActionNodeImpl::AlphabeticalNodeCompare(TSharedPtr<FGraphActionNode> const& LhsMenuNodePtr, TSharedPtr<FGraphActionNode> const& RhsMenuNodePtr)
{
	FGraphActionNode* LhsMenuNode = LhsMenuNodePtr.Get();
	FGraphActionNode* RhsMenuNode = RhsMenuNodePtr.Get();

	bool const bLhsIsCategory  = LhsMenuNode->IsCategoryNode();
	bool const bRhsIsCategory  = RhsMenuNode->IsCategoryNode();
	bool const bLhsIsSeparator = LhsMenuNode->IsSeparator();
	bool const bRhsIsSeparator = RhsMenuNode->IsSeparator();
	bool const bLhsIsSectionHeader = LhsMenuNode->IsSectionHeadingNode();
	bool const bRhsIsSectionHeader = RhsMenuNode->IsSectionHeadingNode();

	if (LhsMenuNode->SectionID != RhsMenuNode->SectionID)
	{
		// since we don't add section headers for children that have the same
		// section as their parents (the header is above the parent), we need to
		// organize them first (so they're seemingly under the same header)
		if ((LhsMenuNode->SectionID == LhsMenuNode->ParentNode.Pin()->SectionID) &&
			(LhsMenuNode->SectionID != FGraphActionNode::INVALID_SECTION_ID))
		{
			return true;
		}
		else // otherwise...
		{
			// sections are ordered in ascending order
			return (LhsMenuNode->SectionID < RhsMenuNode->SectionID);
		}
	}
	else if (bLhsIsSectionHeader != bRhsIsSectionHeader)
	{
		// section headers go to the top of that section
		return bLhsIsSectionHeader;
	}
	else if (LhsMenuNode->Grouping != RhsMenuNode->Grouping)
	{
		// groups are ordered in descending order
		return (LhsMenuNode->Grouping >= RhsMenuNode->Grouping);
	}
	// next, make sure separators are preserved
	else if (bLhsIsSeparator != bRhsIsSeparator)
	{
		// separators with the same grouping go to the bottom of that "group"
		return bRhsIsSeparator;
	}
	// next, categories get listed before action nodes
	else if (bLhsIsCategory != bRhsIsCategory)
	{
		return bLhsIsCategory;
	}
	else if (bLhsIsCategory) // if both nodes are category nodes
	{
		// @TODO: Should we be doing localized compares for categories? Probably.
		return (LhsMenuNode->GetDisplayName().ToString() <= RhsMenuNode->GetDisplayName().ToString());
	}
	else // both nodes are action nodes
	{
		return (LhsMenuNode->GetDisplayName().CompareTo(RhsMenuNode->GetDisplayName()) <= 0);
	}
}

/*******************************************************************************
 * FGraphActionNode
 ******************************************************************************/

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::NewRootNode()
{
	// same as a group-divider node, just with an invalid parent
	return MakeShareable(new FGraphActionNode(FGraphActionNodeImpl::DEFAULT_GROUPING, INVALID_SECTION_ID));
}

//------------------------------------------------------------------------------
FGraphActionNode::FGraphActionNode(int32 InGrouping, int32 InSectionID)
	: SectionID(InSectionID)
	, Grouping(InGrouping)
	, bPendingRenameRequest(false)
	, InsertOrder(0)
	, TotalLeafs(0)
{
}

//------------------------------------------------------------------------------
FGraphActionNode::FGraphActionNode(const TSharedPtr<FEdGraphSchemaAction>& InAction, int32 InGrouping, int32 InSectionID)
	: SectionID(InSectionID)
	, Grouping(InGrouping)
	, Action(InAction)
	, bPendingRenameRequest(false)
	, InsertOrder(0)
	, TotalLeafs(0)
{
}

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::AddChild(const TSharedPtr<FEdGraphSchemaAction>& InAction)
{
	const TArray<FString>& CategoryStack = InAction->GetCategoryChain();

	TSharedPtr<FGraphActionNode> ActionNode = FGraphActionNode::NewActionNode(InAction);
	if (!ActionNode->IsCategoryNode() && !ActionNode->IsSectionHeadingNode())
	{
		++TotalLeafs;
	}

	AddChildRecursively(CategoryStack, 0, ActionNode);

	return ActionNode;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<FGraphActionNode> FGraphActionNode::AddChild(FGraphActionListBuilderBase::ActionGroup const& ActionSet)
{
	const TArray<FString>& CategoryStack = ActionSet.GetCategoryChain();

	TSharedPtr<FGraphActionNode> ActionNode = FGraphActionNode::NewActionNode(ActionSet.Actions[0]);
	if (!ActionNode->IsCategoryNode() && !ActionNode->IsSectionHeadingNode())
	{
		++TotalLeafs;
	}

	AddChildRecursively(CategoryStack, 0, ActionNode);
	
	return ActionNode;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::AddChildAlphabetical(const TSharedPtr<FEdGraphSchemaAction>& InAction)
{
	TSharedPtr<FGraphActionNode> ActionNode = FGraphActionNode::NewActionNode(InAction);
	check(ActionNode->SectionID == INVALID_SECTION_ID); // this method does not support sections, those should be built statically

	if (!ActionNode->IsCategoryNode() && !ActionNode->IsSectionHeadingNode())
	{
		++TotalLeafs;
	}

	// if a divider hasn't been created for the grouping, create one:
	AddChildGrouping(ActionNode, this->AsShared(), true);

	// find or add categories iteratively, inserting as needed:
	FGraphActionNode* OwningCategory = this;
	const TArray<FString>& CategoryStack = InAction->GetCategoryChain();
	for (const FString& CategorySection : CategoryStack)
	{
		TSharedPtr<FGraphActionNode> CategoryNode = OwningCategory->FindMatchingParent(CategorySection, ActionNode);
		if (!CategoryNode.IsValid())
		{
			CategoryNode = NewCategoryNode(CategorySection, ActionNode->Grouping, ActionNode->SectionID);
			OwningCategory->InsertChildAlphabetical(CategoryNode);
		}

		OwningCategory = CategoryNode.Get();
	}

	// finally insert the leaf:
	OwningCategory->InsertChildAlphabetical(ActionNode);
	return ActionNode;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<FGraphActionNode> FGraphActionNode::AddChildAlphabetical(FGraphActionListBuilderBase::ActionGroup const& ActionSet)
{
	TSharedPtr<FGraphActionNode> ActionNode = FGraphActionNode::NewActionNode(ActionSet.Actions[0]);
	check(ActionNode->SectionID == INVALID_SECTION_ID); // this method does not support sections, those should be built statically

	if (!ActionNode->IsCategoryNode() && !ActionNode->IsSectionHeadingNode())
	{
		++TotalLeafs;
	}

	// if a divider hasn't been created for the grouping, create one:
	AddChildGrouping(ActionNode, this->AsShared(), true);

	// find or add categories iteratively, inserting as needed:
	FGraphActionNode* OwningCategory = this;
	const TArray<FString>& CategoryStack = ActionSet.GetCategoryChain();
	for (const FString& CategorySection : CategoryStack)
	{
		TSharedPtr<FGraphActionNode> CategoryNode = OwningCategory->FindMatchingParent(CategorySection, ActionNode);
		if (!CategoryNode.IsValid())
		{
			CategoryNode = NewCategoryNode(CategorySection, ActionNode->Grouping, ActionNode->SectionID);
			OwningCategory->InsertChildAlphabetical(CategoryNode);
		}

		OwningCategory = CategoryNode.Get();
	}

	// finally insert the leaf:
	OwningCategory->InsertChildAlphabetical(ActionNode);
	return ActionNode;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::AddSection(int32 InGrouping, int32 InSectionID)
{
	if ( !ChildSections.Contains(InSectionID) )
	{
		ChildSections.Add(InSectionID);

		TSharedPtr<FGraphActionNode> Section = NewSectionHeadingNode(SharedThis(this), InGrouping, InSectionID);
		InsertChild(Section);

		return Section;
	}

	return nullptr;
}

//------------------------------------------------------------------------------
void FGraphActionNode::SortChildren(bool bAlphabetically/* = true*/, bool bRecursive/* = true*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SGraphActionMenu::GenerateFilteredItems_SortNodes);

	if (bRecursive)
	{
		for (TSharedPtr<FGraphActionNode>& ChildNode : Children)
		{
			ChildNode->SortChildren(bAlphabetically, bRecursive);
		}
	}

	if (bAlphabetically)
	{
		Children.Sort(FGraphActionNodeImpl::AlphabeticalNodeCompare);
	}
	else
	{
		Children.Sort(FGraphActionNodeImpl::NodeCompare);
	}
}

//------------------------------------------------------------------------------
void FGraphActionNode::GetAllNodes(TArray< TSharedPtr<FGraphActionNode> >& OutNodeArray) const
{
	for (TSharedPtr<FGraphActionNode> const& ChildNode : Children)
	{
		OutNodeArray.Add(ChildNode);
		ChildNode->GetAllNodes(OutNodeArray);
	}
}

void FGraphActionNode::GetAllActionNodes(TArray<TSharedPtr<FGraphActionNode>>& OutNodeArray) const
{	
	for (TSharedPtr<FGraphActionNode> const& ChildNode : Children)
	{
		if(ChildNode->IsActionNode())
		{
			OutNodeArray.Add(ChildNode);
		}
		
		ChildNode->GetAllActionNodes(OutNodeArray);
	}
}

//------------------------------------------------------------------------------
void FGraphActionNode::GetLeafNodes(TArray< TSharedPtr<FGraphActionNode> >& OutLeafArray) const
{
	for (TSharedPtr<FGraphActionNode> const& ChildNode : Children)
	{
		if (ChildNode->IsCategoryNode() || ChildNode->IsSectionHeadingNode())
		{
			ChildNode->GetLeafNodes(OutLeafArray);
		}
		else if (!ChildNode->IsGroupDividerNode())
		{
			// @TODO: sometimes, certain action nodes can have children as well
			//        (for sub-graphs in the "MyBlueprint" tab)
			OutLeafArray.Add(ChildNode);
		}
	}
}

int32 FGraphActionNode::GetTotalLeafNodes() const 
{ 
	return TotalLeafs; 
}

//------------------------------------------------------------------------------
void FGraphActionNode::ExpandAllChildren(TSharedPtr< STreeView< TSharedPtr<FGraphActionNode> > > TreeView, bool bRecursive/*= true*/)
{
	if (Children.Num() > 0)
	{
		TreeView->SetItemExpansion(this->AsShared(), /*ShouldExpandItem =*/true);
		for (TSharedPtr<FGraphActionNode>& ChildNode : Children)
		{
			if (bRecursive)
			{
				ChildNode->ExpandAllChildren(TreeView);
			}
			else
			{
				TreeView->SetItemExpansion(ChildNode, /*ShouldExpandItem =*/true);
			}
		}
	}
}

//------------------------------------------------------------------------------
void FGraphActionNode::ClearChildren()
{
	TotalLeafs = 0;
	Children.Empty();
	CategoryNodes.Empty();
	ChildGroupings.Empty();
	ChildSections.Empty();
}

//------------------------------------------------------------------------------
bool FGraphActionNode::IsRootNode() const
{
	return (!IsActionNode() && !IsCategoryNode() && !ParentNode.IsValid());
}

//------------------------------------------------------------------------------
bool FGraphActionNode::IsSectionHeadingNode() const
{
	return (!IsActionNode() && !IsCategoryNode() && !IsRootNode() && (SectionID != INVALID_SECTION_ID));
}

//------------------------------------------------------------------------------
bool FGraphActionNode::IsCategoryNode() const
{
	return (!IsActionNode() && !DisplayText.IsEmpty());
}

//------------------------------------------------------------------------------
bool FGraphActionNode::IsActionNode() const
{
	return Action.IsValid();
}

//------------------------------------------------------------------------------
bool FGraphActionNode::IsGroupDividerNode() const
{
	return (!IsActionNode() && !IsCategoryNode() && !IsRootNode() && (SectionID == INVALID_SECTION_ID));
}

//------------------------------------------------------------------------------
bool FGraphActionNode::IsSeparator() const
{
	return IsSectionHeadingNode() || IsGroupDividerNode();
}

//------------------------------------------------------------------------------
FText const& FGraphActionNode::GetDisplayName() const
{
	return DisplayText;
}

//------------------------------------------------------------------------------
FText FGraphActionNode::GetCategoryPath() const
{
	FText CategoryPath;
	if (IsCategoryNode())
	{
		CategoryPath = DisplayText;
	}

	TWeakPtr<FGraphActionNode> AncestorNode = ParentNode;
	while (AncestorNode.IsValid())
	{
		const FText& AncestorDisplayText = AncestorNode.Pin()->DisplayText;

		if( !AncestorDisplayText.IsEmpty() )
		{
			CategoryPath = FText::Format(FText::FromString(TEXT("{0}|{1}")), AncestorDisplayText, CategoryPath);
		}
		AncestorNode = AncestorNode.Pin()->GetParentNode();
	}
	return CategoryPath;
}

//------------------------------------------------------------------------------
bool FGraphActionNode::HasValidAction() const
{
	return GetPrimaryAction().IsValid();
}

//------------------------------------------------------------------------------
TSharedPtr<FEdGraphSchemaAction> FGraphActionNode::GetPrimaryAction() const
{
	return Action;
}

//------------------------------------------------------------------------------
bool FGraphActionNode::BroadcastRenameRequest()
{
	if (RenameRequestEvent.IsBound())
	{
		RenameRequestEvent.Execute();
		bPendingRenameRequest = false;
	}
	else
	{
		bPendingRenameRequest = true;
	}
	return bPendingRenameRequest;
}

//------------------------------------------------------------------------------
bool FGraphActionNode::IsRenameRequestPending() const
{
	return bPendingRenameRequest;
}

//------------------------------------------------------------------------------
int32 FGraphActionNode::GetLinearizedIndex(TSharedPtr<FGraphActionNode> Node) const
{
	int32 Counter = 0;
	return GetLinearizedIndex(Node, Counter);
}

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::NewSectionHeadingNode(TWeakPtr<FGraphActionNode> Parent, int32 Grouping, int32 SectionID)
{
	checkSlow(SectionID != INVALID_SECTION_ID);

	FGraphActionNode* SectionNode = new FGraphActionNode(Grouping, SectionID);
	SectionNode->ParentNode = Parent;
	checkSlow(Parent.IsValid());

	return MakeShareable(SectionNode);
}

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::NewCategoryNode(FString const& Category, int32 Grouping, int32 SectionID)
{
	FGraphActionNode* CategoryNode = new FGraphActionNode(Grouping, SectionID);
	CategoryNode->DisplayText = FText::FromString(Category);

	return MakeShareable(CategoryNode);
}

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::NewActionNode(const TSharedPtr<FEdGraphSchemaAction>& Action)
{
	const int32 Grouping = FMath::Max(FGraphActionNodeImpl::DEFAULT_GROUPING, Action->GetGrouping());
	const int32 SectionID = Action->GetSectionID();

	FGraphActionNode* ActionNode = new FGraphActionNode(Action, Grouping, SectionID);
	TSharedPtr<FEdGraphSchemaAction> PrimeAction = ActionNode->GetPrimaryAction();
	checkSlow(PrimeAction.IsValid());
	ActionNode->DisplayText = PrimeAction->GetMenuDescription();

	return MakeShareable(ActionNode);
}

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::NewGroupDividerNode(TWeakPtr<FGraphActionNode> Parent, int32 Grouping)
{
	FGraphActionNode* DividerNode = new FGraphActionNode(Grouping, INVALID_SECTION_ID);
	DividerNode->ParentNode = Parent;
	checkSlow(Parent.IsValid());

	return MakeShareable(DividerNode);
}

//------------------------------------------------------------------------------
void FGraphActionNode::AddChildRecursively(const TArray<FString>& CategoryStack, int32 Idx, TSharedPtr<FGraphActionNode> NodeToAdd)
{
	if (NodeToAdd->SectionID != INVALID_SECTION_ID)
	{
		TSharedPtr<FGraphActionNode> FoundSectionNode;
		for ( TSharedPtr<FGraphActionNode> const& ChildNode : Children )
		{
			if ( NodeToAdd->SectionID == ChildNode->SectionID && ChildNode->IsSectionHeadingNode() )
			{
				FoundSectionNode = ChildNode;
				break;
			}
		}

		if ( FoundSectionNode.IsValid() )
		{
			FoundSectionNode->AddChildRecursively(CategoryStack, Idx, NodeToAdd);
			return;
		}
	}

	if ( Idx < CategoryStack.Num() )
	{
		const FString& CategorySection = CategoryStack[Idx];
		++Idx;

		// make sure we don't already have a child that this can nest under
		TSharedPtr<FGraphActionNode> ExistingNode = FindMatchingParent(CategorySection, NodeToAdd);
		if ( ExistingNode.IsValid() )
		{
			ExistingNode->AddChildRecursively(CategoryStack, Idx, NodeToAdd);
		}
		else
		{
			TSharedPtr<FGraphActionNode> CategoryNode = NewCategoryNode(CategorySection, NodeToAdd->Grouping, NodeToAdd->SectionID);
			InsertChild(CategoryNode);
			CategoryNode->AddChildRecursively(CategoryStack, Idx, NodeToAdd);
		}
	}
	else
	{
		InsertChild(NodeToAdd);
	}
}

//------------------------------------------------------------------------------
TSharedPtr<FGraphActionNode> FGraphActionNode::FindMatchingParent(FString const& ParentName, TSharedPtr<FGraphActionNode> NodeToAdd)
{
	TSharedPtr<FGraphActionNode> FoundCategoryNode;

	// for the "MyBlueprint" tab, sub-graph actions can be nested under graph
	// actions (meaning that action node can have children).
	bool const bCanNestUnderActionNodes = NodeToAdd->IsActionNode() && NodeToAdd->GetPrimaryAction()->IsParentable();

	if (bCanNestUnderActionNodes)
	{
		// slow path, not commonly used:
		for (TSharedPtr<FGraphActionNode> const& ChildNode : Children)
		{
			if (ChildNode->IsCategoryNode())
			{
				if ((NodeToAdd->SectionID == ChildNode->SectionID) &&
					(ParentName == ChildNode->DisplayText.ToString()))
				{
					FoundCategoryNode = ChildNode;
					break;
				}
			}
			else if (bCanNestUnderActionNodes && ChildNode->IsActionNode())
			{
				// make the action's name into a display name, all categories are 
				// set as such (to ensure that the action name best matches the 
				// category ParentName)
				FString ChildNodeName = FName::NameToDisplayString(ChildNode->DisplayText.ToString(), /*bIsBool =*/false);

				// @TODO: should we be matching section/grouping as well?
				if (ChildNodeName == ParentName)
				{
					FoundCategoryNode = ChildNode;
					break;
				}
			}
		}
	}
	else
	{
		// fast path, just look up in category map:
		TSharedPtr<FGraphActionNode>* PotentialCategoryNode = CategoryNodes.Find(ParentName);
		if (PotentialCategoryNode && (*PotentialCategoryNode)->SectionID == NodeToAdd->SectionID)
		{
			FoundCategoryNode = *PotentialCategoryNode;
		}
	}

	return FoundCategoryNode;
}

//------------------------------------------------------------------------------
void FGraphActionNode::InsertChild(TSharedPtr<FGraphActionNode> NodeToAdd)
{
	ensure(!NodeToAdd->IsRootNode());
	//ensure(!IsSeparator());

	NodeToAdd->ParentNode = this->AsShared();

	if (NodeToAdd->SectionID != INVALID_SECTION_ID)
	{
		// don't need a section heading if the parent is under the same section
		bool const bAddSectionHeading = (NodeToAdd->SectionID != SectionID) && 
			// make sure we already haven't already added a heading for this section
			!ChildSections.Contains(NodeToAdd->SectionID) &&
			// if this node also has a category, use that over a section heading
			(!NodeToAdd->IsActionNode() || NodeToAdd->GetPrimaryAction()->GetCategory().IsEmpty());

		if (bAddSectionHeading)
		{
			ChildSections.Add(NodeToAdd->SectionID); // to avoid recursion, add before we insert
			//InsertChild(NewSectionHeadingNode(NodeToAdd->ParentNode, NodeToAdd->Grouping, NodeToAdd->SectionID));

			TSharedPtr<FGraphActionNode> NewSection = NewSectionHeadingNode(NodeToAdd->ParentNode, NodeToAdd->Grouping, NodeToAdd->SectionID);
			InsertChild(NewSection);

			NodeToAdd->InsertOrder = NewSection->Children.Num();
			NewSection->Children.Add(NodeToAdd);
			if (NodeToAdd->IsCategoryNode())
			{
				NewSection->CategoryNodes.Add(NodeToAdd->DisplayText.ToString(), NodeToAdd);
			}
			return;
		}
	}
	// we don't use group-dividers inside of sections (we use groups to more to
	// hardcode the order), but if this isn't in a section...
	else
	{
		AddChildGrouping(NodeToAdd, NodeToAdd->ParentNode, false);
	}

	NodeToAdd->InsertOrder = Children.Num();
	Children.Add(NodeToAdd);
	if (NodeToAdd->IsCategoryNode())
	{
		CategoryNodes.Add(NodeToAdd->DisplayText.ToString(), NodeToAdd);
	}
}

//------------------------------------------------------------------------------
void FGraphActionNode::AddChildGrouping(TSharedPtr<FGraphActionNode> ActionNode, TWeakPtr<FGraphActionNode> Parent, bool bInsertAlphabetically)
{
	if (ChildGroupings.Find(ActionNode->Grouping))
	{
		return;
	}

	if (ChildGroupings.Num() > 0)
	{
		int32 LowestGrouping = MAX_int32;
		for (int32 Group : ChildGroupings)
		{
			LowestGrouping = FMath::Min(LowestGrouping, Group);
		}
		// dividers come at the end of a menu group, so it would be 
		// undesirable to add it for NodeToAdd->Grouping if that group is 
		// lower than all the others (the lowest group should not have a 
		// divider associated with it)
		int32 DividerGrouping = FMath::Max(LowestGrouping, ActionNode->Grouping);

		ChildGroupings.Add(ActionNode->Grouping);  // to avoid recursion, add before we insert
		if(bInsertAlphabetically)
		{
			InsertChildAlphabetical(NewGroupDividerNode(this->AsShared(), DividerGrouping));
		}
		else
		{
			InsertChild(NewGroupDividerNode(this->AsShared(), DividerGrouping));
		}
	}
	else
	{
		ChildGroupings.Add(ActionNode->Grouping);
	}
}

//------------------------------------------------------------------------------
void FGraphActionNode::InsertChildAlphabetical(TSharedPtr<FGraphActionNode> NodeToAdd)
{
	check(NodeToAdd->SectionID == INVALID_SECTION_ID);
	int Idx = Algo::LowerBound(Children, NodeToAdd, FGraphActionNodeImpl::AlphabeticalNodeCompare);
	if (Idx != INDEX_NONE)
	{
		NodeToAdd->InsertOrder = Idx;
		Children.Insert(NodeToAdd, Idx);
		for (int32 I = Idx + 1; I < Children.Num(); ++I)
		{
			++(Children[I]->InsertOrder);
		}
	}
	else
	{
		NodeToAdd->InsertOrder = Children.Num();
		Children.Add(NodeToAdd);
	}

	if (NodeToAdd->IsCategoryNode())
	{
		CategoryNodes.Add(NodeToAdd->DisplayText.ToString(), NodeToAdd);
	}
}

int32 FGraphActionNode::GetLinearizedIndex(TSharedPtr<FGraphActionNode> Node, int32& Counter) const
{
	if (Node.Get() == this)
	{
		return Counter;
	}

	// we didn't match, count ourself:
	++Counter;

	// and check/count each child:
	for (const TSharedPtr<FGraphActionNode>& Child : Children)
	{
		int32 Result = Child->GetLinearizedIndex(Node, Counter);
		if (Result != INDEX_NONE)
		{
			return Result;
		}
	}

	// no matches, return INDEX_NONE to indicate we found no valid index, Counter will continue counting
	return INDEX_NONE;
}
