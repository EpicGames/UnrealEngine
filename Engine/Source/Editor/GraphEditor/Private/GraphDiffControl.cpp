// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphDiffControl.h"

#include "Containers/Set.h"
#include "DiffResults.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "GraphDiffControl"

//If we are collecting all Diff results, keep going. If we just want to know if there *is* any diffs, we can early out
#define KEEP_GOING_IF_RESULTS() 	bHasResult = true;	\
if(!Results.CanStoreResults())		{ break; }


/*******************************************************************************
* Static helper functions
*******************************************************************************/

/** Diff result when a node was added to the graph */
static void DiffR_NodeAdded( const FGraphDiffControl::FNodeDiffContext& DiffContext, FDiffResults& Results, UEdGraphNode* Node )
{
	FDiffSingleResult Diff;
	Diff.Diff = EDiffType::NODE_ADDED;
	Diff.Node1 = Node;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeType"), DiffContext.NodeTypeDisplayName);
		Args.Add(TEXT("NodeTitle"), Node->GetNodeTitle(ENodeTitleType::ListView));
		Diff.ToolTip =  FText::Format(LOCTEXT("DIF_AddNode", "Added {NodeType} '{NodeTitle}'"), Args);
		Diff.DisplayString = Diff.ToolTip;
		Diff.Category = EDiffType::ADDITION;
	}
	
	Results.Add(Diff);
}

/** Diff result when a node was removed from the graph */
static void DiffR_NodeRemoved( const FGraphDiffControl::FNodeDiffContext& DiffContext, FDiffResults& Results, UEdGraphNode* Node )
{
	FDiffSingleResult Diff;
	Diff.Diff = EDiffType::NODE_REMOVED;
	Diff.Node1 = Node;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeType"), DiffContext.NodeTypeDisplayName);
		Args.Add(TEXT("NodeTitle"), Node->GetNodeTitle(ENodeTitleType::ListView));
		Diff.ToolTip =  FText::Format(LOCTEXT("DIF_RemoveNode", "Removed {NodeType} '{NodeTitle}'"), Args);
		Diff.DisplayString = Diff.ToolTip;
		Diff.Category = EDiffType::SUBTRACTION;
	}

	Results.Add(Diff);
}

/** Diff result when a node comment was changed */
static void DiffR_NodeCommentChanged(const FGraphDiffControl::FNodeDiffContext& DiffContext, FDiffResults& Results, UEdGraphNode* NewNode, UEdGraphNode* OldNode)
{
	FDiffSingleResult Diff;
	Diff.Diff = EDiffType::NODE_COMMENT;
	Diff.Node1 = OldNode;
	Diff.Node2 = NewNode;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeType"), DiffContext.NodeTypeDisplayName);
		Args.Add(TEXT("NodeTitle"), NewNode->GetNodeTitle(ENodeTitleType::ListView));
		Diff.ToolTip =  FText::Format(LOCTEXT("DIF_CommentModified", "Comment Modified {NodeType} '{NodeTitle}'"), Args);
		Diff.DisplayString = Diff.ToolTip;
		Diff.Category = EDiffType::MINOR;
	}

	Results.Add(Diff);
}

/** Diff result when a node was moved on the graph */
static void DiffR_NodeMoved(const FGraphDiffControl::FNodeDiffContext& DiffContext, FDiffResults& Results, UEdGraphNode* NewNode, UEdGraphNode* OldNode)
{
	FDiffSingleResult Diff;
	Diff.Diff = EDiffType::NODE_MOVED;
	Diff.Category = EDiffType::MINOR;
	Diff.Node1 = OldNode;
	Diff.Node2 = NewNode;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeType"), DiffContext.NodeTypeDisplayName);
		Args.Add(TEXT("NodeTitle"), NewNode->GetNodeTitle(ENodeTitleType::ListView));
		Diff.ToolTip = FText::Format(LOCTEXT("DIF_MoveNode", "Moved {NodeType} '{NodeTitle}'"), Args);
		Diff.DisplayString = Diff.ToolTip;
	}

	Results.Add(Diff);
}

/** Diff result when a pin type was changed */
static void DiffR_PinTypeChanged(FDiffResults& Results, UEdGraphPin* NewPin, UEdGraphPin* OldPin)
{
	FEdGraphPinType Type1 = OldPin->PinType;
	FEdGraphPinType Type2 = NewPin->PinType;

	FDiffSingleResult Diff;

	const UObject* T1Obj = Type1.PinSubCategoryObject.Get();
	const UObject* T2Obj = Type2.PinSubCategoryObject.Get();

	if(Type1.PinCategory != Type2.PinCategory)
	{
		Diff.Diff = EDiffType::PIN_TYPE_CATEGORY;

		// Only bother setting up the display data if we're storing the result
		if(Results.CanStoreResults())
		{
			Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinCategoryToolTipFmt", "Pin '{0}' Category was '{1}', but is now '{2}"), FText::FromName(NewPin->PinName), FText::FromName(OldPin->PinType.PinCategory), FText::FromName(NewPin->PinType.PinCategory));
			Diff.Category = EDiffType::MODIFICATION;
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinCategoryFmt", "Pin Category '{0}' ['{1}' -> '{2}']"), FText::FromName(NewPin->PinName), FText::FromName(OldPin->PinType.PinCategory), FText::FromName(NewPin->PinType.PinCategory));
		}
	}
	else if(Type1.PinSubCategory != Type2.PinSubCategory)
	{
		Diff.Diff = EDiffType::PIN_TYPE_SUBCATEGORY;

		// Only bother setting up the display data if we're storing the result
		if(Results.CanStoreResults())
		{
			Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinSubCategoryToolTipFmt", "Pin '{0}' SubCategory was '{1}', but is now '{2}"), FText::FromName(NewPin->PinName), FText::FromName(OldPin->PinType.PinSubCategory), FText::FromName(NewPin->PinType.PinSubCategory));
			Diff.Category = EDiffType::MODIFICATION;
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinSubCategoryFmt", "Pin SubCategory '{0}'  ['{1}' -> '{2}']"), FText::FromName(NewPin->PinName), FText::FromName(OldPin->PinType.PinSubCategory), FText::FromName(NewPin->PinType.PinSubCategory));
		}
	}
	else if(T1Obj != T2Obj && (T1Obj && T2Obj ) &&
		(T1Obj->GetFName() != T2Obj->GetFName()))
	{
		Diff.Diff = EDiffType::PIN_TYPE_SUBCATEGORY_OBJECT;

		// Only bother setting up the display data if we're storing the result
		if(Results.CanStoreResults())
		{
			const FName Obj1 = T1Obj->GetFName();
			const FName Obj2 = T2Obj->GetFName();

			Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinSubCategorObjToolTipFmt", "Pin '{0}' was SubCategoryObject '{1}', but is now '{2}"), FText::FromName(NewPin->PinName), FText::FromName(Obj1), FText::FromName(Obj2));
			Diff.Category = EDiffType::MODIFICATION;
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinSubCategoryObjFmt", "Pin SubCategoryObject '{0}' ['{1}' -> '{2}']"), FText::FromName(NewPin->PinName), FText::FromName(Obj1), FText::FromName(Obj2));
		}
	}
	else if(Type1.ContainerType != Type2.ContainerType)
	{
		// TODO: Make the messaging correct about the nature of the diff
		Diff.Diff = EDiffType::PIN_TYPE_IS_ARRAY;

		// Only bother setting up the display data if we're storing the result
		if(Results.CanStoreResults())
		{
			FText IsArray1 = OldPin->PinType.IsArray() ? LOCTEXT("true", "true") : LOCTEXT("false", "false");
			FText IsArray2 = NewPin->PinType.IsArray() ? LOCTEXT("true", "true") : LOCTEXT("false", "false");

			Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinIsArrayToolTipFmt", "PinType IsArray for '{0}' modified. Was '{1}', but is now '{2}"), FText::FromName(NewPin->PinName), IsArray1, IsArray2);
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinIsArrayFmt", "Pin IsArray '{0}' ['{1}' -> '{2}']"), FText::FromName(NewPin->PinName), IsArray1, IsArray2);
			Diff.Category = EDiffType::MODIFICATION;
		}
	}
	else if(Type1.bIsReference != Type2.bIsReference)
	{
		Diff.Diff = EDiffType::PIN_TYPE_IS_REF;

		// Only bother setting up the display data if we're storing the result
		if(Results.CanStoreResults())
		{
			FText IsRef1 = OldPin->PinType.bIsReference ? LOCTEXT("true", "true") : LOCTEXT("false", "false");
			FText IsRef2 = NewPin->PinType.bIsReference ? LOCTEXT("true", "true") : LOCTEXT("false", "false");

			Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinIsRefToolTipFmt", "PinType IsReference for '{0}' modified. Was '{1}', but is now '{2}"), FText::FromName(NewPin->PinName), IsRef1, IsRef2);
			Diff.Category = EDiffType::MODIFICATION;
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinIsRefFmt", "Pin IsReference '{0}' ['{1}' -> '{2}']"), FText::FromName(NewPin->PinName), IsRef1, IsRef2);
		}
	}

	Diff.Pin1 = OldPin;
	Diff.Pin2 = NewPin;
	Results.Add(Diff);
}

/** Diff result when the # of links to a pin was changed */
static void DiffR_PinLinkCountChanged(FDiffResults& Results, UEdGraphPin* NewPin, UEdGraphPin* OldPin)
{
	FDiffSingleResult Diff;
	Diff.Diff = NewPin->LinkedTo.Num() > OldPin->LinkedTo.Num()  ?  EDiffType::PIN_LINKEDTO_NUM_INC : EDiffType::PIN_LINKEDTO_NUM_DEC;
	Diff.Pin2 = NewPin;
	Diff.Pin1 = OldPin;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		if(Diff.Diff == EDiffType::PIN_LINKEDTO_NUM_INC)
		{
			Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinLinkCountIncToolTipFmt", "Pin '{0}' has more links (was {1} now {2})"), FText::FromName(OldPin->PinName), FText::AsNumber(OldPin->LinkedTo.Num()), FText::AsNumber(NewPin->LinkedTo.Num()));
			Diff.Category = EDiffType::ADDITION;
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinLinkCountIncFmt", "Added Link to '{0}'"), FText::FromName(OldPin->PinName));
		}
		else
		{
			Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinLinkCountDecToolTipFmt", "Pin '{0}' has fewer links (was {1} now {2})"), FText::FromName(OldPin->PinName), FText::AsNumber(OldPin->LinkedTo.Num()), FText::AsNumber(NewPin->LinkedTo.Num()));
			Diff.Category = EDiffType::SUBTRACTION;
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinLinkCountDecFmt", "Removed Link to '{0}'"), FText::FromName(OldPin->PinName));
		}
	}

	Results.Add(Diff);
}

/** Diff result when a pin to relinked to a different node */
static void DiffR_LinkedToNode(FDiffResults& Results, UEdGraphPin* OldPin, UEdGraphPin* NewPin, UEdGraphNode* OldNode, UEdGraphNode* NewNode)
{
	FDiffSingleResult Diff;
	Diff.Diff = EDiffType::PIN_LINKEDTO_NODE;
	Diff.Pin1 = OldPin;
	Diff.Pin2 = NewPin;
	Diff.Node1 = OldNode;
	Diff.Node2 = NewNode;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		FText Node1Name = OldNode->GetNodeTitle(ENodeTitleType::ListView);
		FText Node2Name = NewNode->GetNodeTitle(ENodeTitleType::ListView);

		FFormatNamedArguments Args;
		Args.Add(TEXT("PinNameForNode1"), FText::FromName(OldPin->PinName));
		Args.Add(TEXT("NodeName1"), Node1Name);
		Args.Add(TEXT("NodeName2"), Node2Name);
		Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinLinkMovedToolTip", "Pin '{PinNameForNode1}' was linked to Node '{NodeName1}', but is now linked to Node '{NodeName2}'"), Args);

		Diff.Category = EDiffType::MODIFICATION;
		Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinLinkMoved", "Link Moved  '{PinNameForNode1}' ['{NodeName1}' -> '{NodeName2}']"), Args);
	}

	Results.Add(Diff);
}

/** Diff result when a pin to relinked to a different pin on the same node */
static void DiffR_LinkedToPin(FDiffResults& Results, UEdGraphPin* OldPin, UEdGraphPin* NewPin, const UEdGraphPin* OldLinkedPin, const UEdGraphPin* NewLinkedPin)
{
	FDiffSingleResult Diff;
	Diff.Diff = EDiffType::PIN_LINKEDTO_NODE;
	Diff.Pin1 = OldPin;
	Diff.Pin2 = NewPin;
	Diff.Node1 = nullptr;
	Diff.Node2 = nullptr;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinNameForNode1"), FText::FromName(OldPin->PinName));
		Args.Add(TEXT("OldLinkedPinName"), FText::FromName(OldLinkedPin->PinName));
		Args.Add(TEXT("NewLinkedPinName"), FText::FromName(NewLinkedPin->PinName));
		Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinLinkMovedToPinToolTip", "Pin '{PinNameForNode1}' was linked to Pin '{OldLinkedPinName}', but is now linked to Pin '{NewLinkedPinName}'"), Args);

		Diff.Category = EDiffType::MODIFICATION;
		Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinLinkMovedToPin", "Link Moved  '{PinNameForNode1}' ['{OldLinkedPinName}' -> '{NewLinkedPinName}']"), Args);
	}

	Results.Add(Diff);
}

/** Diff result when a pin default value was changed, and is in use*/
static void DiffR_PinDefaultValueChanged(FDiffResults& Results, UEdGraphPin* NewPin, UEdGraphPin* OldPin)
{
	FDiffSingleResult Diff;
	Diff.Diff = EDiffType::PIN_DEFAULT_VALUE;
	Diff.Pin1 = OldPin;
	Diff.Pin2 = NewPin;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinNameForValue1"), FText::FromName(NewPin->PinName));
		Args.Add(TEXT("PinValue1"), FText::FromString(OldPin->GetDefaultAsString()));
		Args.Add(TEXT("PinValue2"), FText::FromString(NewPin->GetDefaultAsString()));
		Diff.ToolTip = FText::Format(LOCTEXT("DIF_PinDefaultValueToolTip", "Pin '{PinNameForValue1}' Default Value was '{PinValue1}', but is now '{PinValue2}'"), Args);
		Diff.Category = EDiffType::MODIFICATION;
		Diff.DisplayString = FText::Format(LOCTEXT("DIF_PinDefaultValue", "Pin Default '{PinNameForValue1}' ['{PinValue1}' -> '{PinValue2}']"), Args);
	}

	Results.Add(Diff);
}

/** Diff result when pin count is not the same */
static void DiffR_NodePinCount(FDiffResults& Results, UEdGraphNode* NewNode, UEdGraphNode* OldNode, const TArray<UEdGraphPin*>& NewPins, const TArray<UEdGraphPin*>& OldPins)
{
	FText NodeName = NewNode->GetNodeTitle(ENodeTitleType::ListView);
	int32 OriginalCount = OldPins.Num();
	int32 NewCount = NewPins.Num();
	FDiffSingleResult Diff;
	Diff.Diff = EDiffType::NODE_PIN_COUNT;
	Diff.Node1 = OldNode;
	Diff.Node2 = NewNode;

	// Only bother setting up the display data if we're storing the result
	if(Results.CanStoreResults())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeName"), NodeName);
		Args.Add(TEXT("OriginalCount"), OriginalCount);
		Args.Add(TEXT("NewCount"), NewCount);

		struct FMatchName
		{
			FMatchName(const FName InPinName)
				: PinName(InPinName)
			{
			}

			const FName PinName;

			bool operator()(const UEdGraphPin* Entry )
			{
				return PinName == Entry->PinName;
			}
		};

		FText ListOfPins;
		TArray< FText > RemovedPins;
		TArray< FText > AddedPins;
		
		for (UEdGraphPin* SearchPin : OldPins)
		{
			const UEdGraphPin* const* FoundPin = NewPins.FindByPredicate(FMatchName(SearchPin->PinName));
			if (FoundPin == nullptr)
			{
				RemovedPins.Add(SearchPin->GetDisplayName());
			}
		}

		for (UEdGraphPin* SearchPin : NewPins)
		{
			const UEdGraphPin* const* FoundPin = OldPins.FindByPredicate(FMatchName(SearchPin->PinName));
			if (FoundPin == nullptr)
			{
				AddedPins.Add(SearchPin->GetDisplayName());
			}
		}

		if (RemovedPins.Num() > 0 && AddedPins.Num() > 0)
		{
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_NodePinsAddedAndRemoved", "Added and removed Pins from '{NodeName}'"), Args);
			Diff.Category = EDiffType::ADDITION;
		}
		else if (AddedPins.Num() > 0)
		{
			if (AddedPins.Num() == 1)
			{
				Diff.DisplayString = FText::Format(LOCTEXT("DIF_NodePinCountIncreased", "Added Pin to '{NodeName}'"), Args);
				Diff.Category = EDiffType::ADDITION;
			}
			else
			{
				Diff.DisplayString = FText::Format(LOCTEXT("DIF_NodePinCountIncreasedSeveral", "Added Pins to '{NodeName}'"), Args);
				Diff.Category = EDiffType::ADDITION;
			}
		}
		else if (RemovedPins.Num() > 0)
		{
			if (RemovedPins.Num() == 1)
			{
				Diff.DisplayString = FText::Format(LOCTEXT("DIF_NodePinCountDecreased", "Removed Pin from '{NodeName}'"), Args);
				Diff.Category = EDiffType::SUBTRACTION;
			}
			else
			{
				Diff.DisplayString = FText::Format(LOCTEXT("DIF_NodePinCountDecreasedSeveral", "Removed Pins from '{NodeName}'"), Args);
				Diff.Category = EDiffType::SUBTRACTION;
			}
		}

		FTextBuilder Builder;
		Builder.AppendLine(FText::Format(LOCTEXT("DIF_NodePinCountChangedToolTip", "Node '{NodeName}' had {OriginalCount} Pins, now has {NewCount} Pins"), Args));
		if (AddedPins.Num() > 0)
		{
			Builder.AppendLine(LOCTEXT("DIF_PinsAddedList", "Pins Added:"));
			for (const FText& Added : AddedPins)
			{
				Builder.AppendLine(Added);
			}
		}

		if (RemovedPins.Num() > 0)
		{
			Builder.AppendLine(LOCTEXT("DIF_PinsRemovedList", "Pins Removed:"));
			for (const FText& Removed : RemovedPins)
			{
				Builder.AppendLine(Removed);
			}
		}
		Diff.ToolTip = Builder.ToText();
	}

	Results.Add(Diff);
}

/** 
 * Populate an array one of pins from another disregarding irrelevant ones (EG invisible). 
 * 
 * @param	InPins			Pins
 * @param	OutRelevanPins	Output of Relevant pins
 */
static void BuildArrayOfRelevantPins(const TArray<UEdGraphPin*>& InPins, TArray< UEdGraphPin* >& OutRelevantPins)
{
	for(int32 i = 0;i<InPins.Num();++i)
	{
		UEdGraphPin* EachPin = InPins[i];
		if( EachPin != nullptr )
		{
			if( EachPin->bHidden == false )
			{
				OutRelevantPins.Add( EachPin );
			}			
		}
	}
}

static bool IsPinTypeDifferent(const FEdGraphPinType& T1, const FEdGraphPinType& T2) 
{
	bool bIsDifferent =  (T1.PinCategory != T2.PinCategory) 
		|| (T1.PinSubCategory != T2.PinSubCategory) 
		|| (T1.ContainerType != T2.ContainerType) 
		|| (T1.bIsReference != T2.bIsReference);

	const UObject* T1Obj = T1.PinSubCategoryObject.Get();
	const UObject* T2Obj = T2.PinSubCategoryObject.Get();
	//TODO: fix, this code makes no sense
	if((T1Obj != T2Obj) && (T1Obj && T2Obj) && (T1Obj->GetFName() != T2Obj->GetFName())) 
	{
		bIsDifferent |= T1Obj->GetFName() == T2Obj->GetFName();
	}
	return bIsDifferent;
}

/** Find linked pin in array that matches pin */
static UEdGraphPin* FindOtherLink(TArray<UEdGraphPin*>& Links2, int32 OriginalIndex, UEdGraphPin* PinToFind)
{
	// Sometimes the order of the pins is different between revisions, although the pins themselves are unchanged, so we have to look at all of them
	UEdGraphNode* Node1 = PinToFind->GetOwningNode();
	UEdGraphPin* BestMatch = Links2[OriginalIndex];
	for (UEdGraphPin* Other : Links2)
	{
		UEdGraphNode* Node2 = Other->GetOwningNode();
		if(FGraphDiffControl::IsNodeMatch(Node1, Node2))
		{
			if (Other->PinId == PinToFind->PinId)
			{
				return Other;
			}
			else if (Other->GetName() == PinToFind->GetName())
			{
				BestMatch = Other;
			}
		}
	}
	return BestMatch;
}

static bool CheckGraphNodePin(UEdGraphPin* OldLinkedPin, UEdGraphPin* NewLinkedPin)
{
	if (!OldLinkedPin || !NewLinkedPin)
	{
		return false;
	}

	for (int32 i = 0; i < OldLinkedPin->LinkedTo.Num(); ++i)
	{
		UEdGraphPin* OldLinkedToPin = OldLinkedPin->LinkedTo[i];
		UEdGraphPin* NewLinkedToPin = nullptr;
		if(NewLinkedPin && OldLinkedToPin)
		{
			NewLinkedToPin = NewLinkedPin->LinkedTo.IsValidIndex(i) ? FindOtherLink(NewLinkedPin->LinkedTo, i, OldLinkedToPin) : nullptr;
		}

		if (!OldLinkedToPin || !NewLinkedToPin)
		{
			continue;
		}

		UEdGraphNode* OldLinkedToNode = OldLinkedToPin->GetOwningNode();
		UEdGraphNode* NewLinkedToNode = NewLinkedToPin->GetOwningNode();

		if (OldLinkedToNode && NewLinkedToNode && !FGraphDiffControl::IsNodeMatch(OldLinkedToNode, NewLinkedToNode))
		{
			return false;
		}
	}

	return true;
}

/** Determine if the LinkedTo pins are the same */
static bool LinkedToDifferent(UEdGraphPin* OldPin, UEdGraphPin* NewPin, const TArray<UEdGraphPin*>& OldLinks, TArray<UEdGraphPin*>& NewLinks, FDiffResults& Results)
{
	const UPackage* const TransientPackage = GetTransientPackage();
	const int32 Size = OldLinks.Num();
	bool bHasResult = false;
	for(int32 i = 0;i<Size;++i)
	{
		UEdGraphPin* OldLinkedPin = OldLinks[i];
		if (!ensureMsgf(OldLinkedPin, TEXT("Found unexpected nullptr in OldLinks. Node graph was either written with null links or pin resolution failed - see UEdGraphPin::ResolveAllPinReferences")))
		{
			continue;
		}
		UEdGraphPin* NewLinkedPin = FindOtherLink(NewLinks, i, OldLinkedPin);
		if (!ensureMsgf(NewLinkedPin, TEXT("Found unexpected nullptr in NewLinks. Node graph was either written with null links or pin resolution failed - see UEdGraphPin::ResolveAllPinReferences")))
		{
			continue;
		}

		UEdGraphNode* OldNode = OldLinkedPin->GetOwningNode();
		UEdGraphNode* NewNode = NewLinkedPin->GetOwningNode();
		if(!FGraphDiffControl::IsNodeMatch(OldNode, NewNode))
		{
			DiffR_LinkedToNode(Results, OldPin, NewPin, OldNode, NewNode);
			KEEP_GOING_IF_RESULTS()
		}
		// ignore PinId for generated graphs, like the material graphs - instead do a more expensive comparison
		// of the pin's node to detect changed links:
		else if (OldNode->GetPackage() == TransientPackage && NewNode->GetPackage() == TransientPackage
			&& CheckGraphNodePin(OldLinkedPin, NewLinkedPin))
		{
			continue;
		}
		else if (OldLinkedPin->PinId != NewLinkedPin->PinId)
		{
			DiffR_LinkedToPin(Results, OldPin, NewPin, OldLinkedPin, NewLinkedPin);
			KEEP_GOING_IF_RESULTS()
		}
	}
	return bHasResult;
}

/** 
 * Determine of two Arrays of Pins are different.
 *
 * @param	OldPins	First set of pins to compare.
 * @param	NewPins	Second set of pins to compare.
 * @param	Results Difference results.
 * 
 * returns true if any pins are different and populates the Results array
 */
static bool ArePinsDifferent(const TArray<UEdGraphPin*>& OldPins, TArray<UEdGraphPin*>& NewPins, FDiffResults& Results)
{
	const int32 Size = OldPins.Num();
	bool bHasResult = false;
	for(int32 i = 0;i<Size;++i)
	{
		UEdGraphPin* OldPin = OldPins[i];
		UEdGraphPin* NewPin = NewPins[i];
		if (!ensureMsgf(OldPin, TEXT("Found unexpected nullptr in OldPins. Node graph was either written with null links or pin resolution failed - see UEdGraphPin::ResolveAllPinReferences")))
		{
			continue;
		}
		if (!ensureMsgf(NewPin, TEXT("Found unexpected nullptr in NewPins. Node graph was either written with null links or pin resolution failed - see UEdGraphPin::ResolveAllPinReferences")))
		{
			continue;
		}

		const UEdGraphSchema* Schema = OldPin->GetSchema();

		if(IsPinTypeDifferent(OldPin->PinType, NewPin->PinType))
		{
			DiffR_PinTypeChanged(Results, NewPin, OldPin);
			KEEP_GOING_IF_RESULTS()
		}
		if(OldPin->LinkedTo.Num() != NewPin->LinkedTo.Num())
		{
			DiffR_PinLinkCountChanged(Results, NewPin, OldPin);
			KEEP_GOING_IF_RESULTS()
		}
		else if(LinkedToDifferent(OldPin, NewPin, OldPin->LinkedTo, NewPin->LinkedTo, Results))
		{
			KEEP_GOING_IF_RESULTS()
		}

		if(NewPin->LinkedTo.Num() == 0 && Schema && !Schema->DoesDefaultValueMatch(*OldPin, NewPin->GetDefaultAsString()))
		{
			DiffR_PinDefaultValueChanged(Results, NewPin, OldPin); //note: some issues with how floating point is stored as string format(0.0 vs 0.00) can cause false diffs
			KEEP_GOING_IF_RESULTS()
		}
	}
	return bHasResult;
}

/*******************************************************************************
* FGraphDiffControl::FNodeMatch
*******************************************************************************/

bool FGraphDiffControl::FNodeMatch::IsValid() const
{
	return ((NewNode != nullptr) && (OldNode != nullptr));
}

bool FGraphDiffControl::FNodeMatch::Diff(const FNodeDiffContext& DiffContext, TArray<FDiffSingleResult>* OptionalDiffsArray /* = nullptr*/) const
{
	FDiffResults DiffsOut(OptionalDiffsArray);
	return Diff(DiffContext, DiffsOut);
}

bool FGraphDiffControl::FNodeMatch::Diff(const FNodeDiffContext& DiffContext, FDiffResults& DiffsOut) const
{
	bool bIsDifferent = false;

	if (IsValid())
	{
		//has comment changed?
		if((DiffContext.DiffFlags & EDiffFlags::NodeComment) && NewNode->NodeComment != OldNode->NodeComment)
		{
			DiffR_NodeCommentChanged(DiffContext, DiffsOut, NewNode, OldNode);
			bIsDifferent = true;
		}

		//has it moved?
		if( (DiffContext.DiffFlags & EDiffFlags::NodeMovement) && ( (NewNode->NodePosX != OldNode->NodePosX) || (NewNode->NodePosY != OldNode->NodePosY) ) )
		{
			//same node, different position--
			DiffR_NodeMoved(DiffContext, DiffsOut, NewNode, OldNode);
			bIsDifferent = true;
		}

		if(DiffContext.DiffFlags & EDiffFlags::NodePins)
		{
			// Build arrays of pins that we care about
			TArray< UEdGraphPin* > OldRelevantPins;
			TArray< UEdGraphPin* > RelevantPins;
			BuildArrayOfRelevantPins(OldNode->Pins, OldRelevantPins);
			BuildArrayOfRelevantPins(NewNode->Pins, RelevantPins);

			if(OldRelevantPins.Num() == RelevantPins.Num())
			{
				//checks contents of pins
				bIsDifferent |= ArePinsDifferent(OldRelevantPins, RelevantPins, DiffsOut);
			}
			else//# of pins changed
			{
				DiffR_NodePinCount(DiffsOut, NewNode, OldNode, RelevantPins, OldRelevantPins);
				bIsDifferent = true;
			}
		}

		//Find internal node diffs; skip this if we don't need the result data
		if((DiffContext.DiffFlags & EDiffFlags::NodeSpecificDiffs) && (!bIsDifferent || DiffsOut.CanStoreResults()))
		{
			OldNode->FindDiffs(NewNode, DiffsOut);
			bIsDifferent |= DiffsOut.HasFoundDiffs();
		}
	}
	else if(DiffContext.DiffFlags & EDiffFlags::NodeExistance)
	// one of the nodes is nullptr
	{
		bIsDifferent = true;
		switch (DiffContext.DiffMode)
		{
		case EDiffMode::Additive:
			DiffR_NodeAdded(DiffContext, DiffsOut, NewNode);
			break;

		case EDiffMode::Subtractive:
			DiffR_NodeRemoved(DiffContext, DiffsOut, NewNode);
			break;

		default:
			break;
		}
	}

	return bIsDifferent;
}

/*******************************************************************************
* FGraphDiffControl
*******************************************************************************/

FGraphDiffControl::FNodeMatch FGraphDiffControl::FindNodeMatch(UEdGraph* OldGraph, UEdGraphNode* NewNode, TArray<FNodeMatch> const& PriorMatches)
{
	FNodeMatch Match;
	Match.NewNode = NewNode;

	if (OldGraph)
	{
		// Attempt to find a node matching 'NewNode', first try exact then try soft match
		for (UEdGraphNode* GraphNode : OldGraph->Nodes)
		{
			if (GraphNode && IsNodeMatch(NewNode, GraphNode, true, &PriorMatches))
			{
				Match.OldNode = GraphNode;
				return Match;
			}
		}

		for (UEdGraphNode* GraphNode : OldGraph->Nodes)
		{
			if (GraphNode && IsNodeMatch(NewNode, GraphNode, false, &PriorMatches))
			{
				Match.OldNode = GraphNode;
				return Match;
			}
		}
	}	

	return Match;
}

bool FGraphDiffControl::IsNodeMatch(UEdGraphNode* Node1, UEdGraphNode* Node2, bool bExactOnly, TArray<FGraphDiffControl::FNodeMatch> const* Exclusions)
{
	if(Node2->GetClass() != Node1->GetClass()) 
	{
		return false;
	}

	if(Node1->NodeGuid == Node2->NodeGuid)
	{
		return true;
	}

	if (bExactOnly)
	{
		return false;
	}

	if (Exclusions)
	{
		// Have to see if this node has already been matched with another, if so don't allow a soft match
		for (const FGraphDiffControl::FNodeMatch& PriorMatch : *Exclusions)
		{
			if (!PriorMatch.IsValid())
			{
				continue;
			}

			// if one of these nodes has already been matched to a different node
			if (((PriorMatch.OldNode == Node1) && (PriorMatch.NewNode != Node2)) ||
				((PriorMatch.OldNode == Node2) && (PriorMatch.NewNode != Node1)) ||
				((PriorMatch.NewNode == Node1) && (PriorMatch.OldNode != Node2)) ||
				((PriorMatch.NewNode == Node2) && (PriorMatch.OldNode != Node1)))
			{
				return false;
			}
		}
	}

	// For a soft match use the node title, which includes the function name and target usually
	FText Title1 = Node1->GetNodeTitle(ENodeTitleType::FullTitle);
	FText Title2 = Node2->GetNodeTitle(ENodeTitleType::FullTitle);

	return Title1.CompareTo(Title2) == 0;
}

bool FGraphDiffControl::DiffGraphs(UEdGraph* const LhsGraph, UEdGraph* const RhsGraph, TArray<FDiffSingleResult>& DiffsOut)
{
	bool bFoundDifferences = false;

	if (LhsGraph && RhsGraph)
	{
		TArray<FGraphDiffControl::FNodeMatch> NodeMatches;
		TSet<UEdGraphNode const*> MatchedRhsNodes;

		FGraphDiffControl::FNodeDiffContext AdditiveDiffContext;
		AdditiveDiffContext.NodeTypeDisplayName = LOCTEXT("NodeDiffDisplayName", "Node");

		// march through the all the nodes in the rhs graph and look for matches 
		for (UEdGraphNode* const RhsNode : RhsGraph->Nodes)
		{
			if (RhsNode)
			{
				FGraphDiffControl::FNodeMatch NodeMatch = FGraphDiffControl::FindNodeMatch(LhsGraph, RhsNode, NodeMatches);
				// if we found a corresponding node in the lhs graph, track it (so we
				// can prevent future matches with the same nodes)
				if (NodeMatch.IsValid())
				{
					NodeMatches.Add(NodeMatch);
					MatchedRhsNodes.Add(NodeMatch.OldNode);
				}

				bFoundDifferences |= NodeMatch.Diff(AdditiveDiffContext, &DiffsOut);
			}
		}

		FGraphDiffControl::FNodeDiffContext SubtractiveDiffContext = AdditiveDiffContext;
		SubtractiveDiffContext.DiffMode = EDiffMode::Subtractive;
		SubtractiveDiffContext.DiffFlags = EDiffFlags::NodeExistance;

		// go through the lhs nodes to catch ones that may have been missing from the rhs graph
		for (UEdGraphNode* const LhsNode : LhsGraph->Nodes)
		{
			// if this node has already been matched, move on
			if ((LhsNode == nullptr) || MatchedRhsNodes.Find(LhsNode))
			{
				continue;
			}

			// There can't be a matching node in RhsGraph because it would have been found above
			FGraphDiffControl::FNodeMatch NodeMatch;
			NodeMatch.NewNode = LhsNode;

			bFoundDifferences |= NodeMatch.Diff(SubtractiveDiffContext, &DiffsOut);
		}
	}

	// storing the graph name for all diff entries:
	const FString GraphPath = LhsGraph ? GetGraphPath(LhsGraph) : GetGraphPath(RhsGraph);
	for( FDiffSingleResult& Entry : DiffsOut )
	{
		Entry.OwningObjectPath = GraphPath;
	}

	return bFoundDifferences;
}

FString FGraphDiffControl::GetGraphPath(UEdGraph* Graph)
{
	if (Graph == nullptr)
	{
		return FString();
	}
	else if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph))
	{
		return Graph->GetPathName(Blueprint);
	}
	else if (UPackage* Package = Graph->GetOutermost())
	{
		return Graph->GetPathName(Package);
	}
	
	return Graph->GetName();
}

#undef LOCTEXT_NAMESPACE
