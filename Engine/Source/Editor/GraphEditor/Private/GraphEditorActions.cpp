// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphEditorActions.h"

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/UnrealString.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "GraphEditorCommands"

UE_DEFINE_TCOMMANDS(FGraphEditorCommandsImpl)

void FGraphEditorCommandsImpl::RegisterCommands()
{
	UI_COMMAND( ReconstructNodes, "Refresh Nodes", "Refreshes nodes", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( BreakNodeLinks, "Break Node Link(s)", "Breaks the selected node from all connected pins", EUserInterfaceActionType::Button, FInputChord() )
	
	UI_COMMAND( AddExecutionPin, "Add Execution Pin", "Adds another execution output pin to an execution sequence or switch node", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( InsertExecutionPinBefore, "Insert Execution Pin Before", "Adds another execution output pin before this one, to an execution sequence node", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( InsertExecutionPinAfter, "Insert Execution Pin After", "Adds another execution output pin after this one, to an execution sequence node", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( RemoveExecutionPin, "Remove Execution Pin", "Removes an execution output pin from an execution sequence or switch node", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( RemoveThisStructVarPin, "Remove This Struct Variable Pin", "Removes the selected input pin", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( RemoveOtherStructVarPins, "Remove All Other Pins", "Removes all variable input pins, except for the selected one", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( RestoreAllStructVarPins, "Restore All Structure Pins", "Restore all structure pins", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( AddOptionPin, "Add Option Pin", "Adds another option input pin to the node", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( RemoveOptionPin, "Remove Option Pin", "Removes the last option input pin from the node", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( ChangePinType, "Change Pin Type", "Changes the type of this pin (boolean, int, etc.)", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( DeleteAndReconnectNodes, "Delete and Reconnect Exec Pins", "Deletes the currently selected nodes and makes connections between all input pins to their output pins.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Delete), FInputChord(EModifierKey::Shift, EKeys::BackSpace))

	UI_COMMAND( ShowAllPins, "Show All Pins", "Shows all pins", EUserInterfaceActionType::RadioButton, FInputChord() )
	UI_COMMAND( HideNoConnectionPins, "Hide Unconnected Pins", "Hides all pins with no connections", EUserInterfaceActionType::RadioButton, FInputChord() )
	UI_COMMAND( HideNoConnectionNoDefaultPins, "Hide Unused Pins", "Hides all pins with no connections and no default value", EUserInterfaceActionType::RadioButton, FInputChord() )

	UI_COMMAND( AddParentNode, "Add Call to Parent Function", "Adds a node that calls this function's parent", EUserInterfaceActionType::Button, FInputChord() )
	
	UI_COMMAND( CreateMatchingFunction, "Create Matching Function", "Adds a function to the blueprint with a matching signature", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( ToggleBreakpoint, "Toggle Breakpoint", "Adds or removes a breakpoint on each selected node", EUserInterfaceActionType::Button, FInputChord(EKeys::F9) )
	UI_COMMAND( AddBreakpoint, "Add Breakpoint", "Adds a breakpoint to each selected node", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( RemoveBreakpoint, "Remove Breakpoint", "Removes any breakpoints on each selected node", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( EnableBreakpoint, "Enable Breakpoint", "Enables any breakpoints on each selected node", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( DisableBreakpoint, "Disable Breakpoint", "Disables any breakpoints on each selected node", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( CollapseNodes, "Collapse Nodes", "Collapses selected nodes into a single node", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( PromoteSelectionToFunction, "Promote to Function", "Promotes selected collapsed graphs to functions.", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( PromoteSelectionToMacro, "Promote to Macro", "Promotes selected collapsed graphs to macros.", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( ExpandNodes, "Expand Node", "Expands the node's internal graph into the current graph and removes this node.", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( CollapseSelectionToFunction, "Collapse to Function", "Collapses selected nodes into a single function node.", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( CollapseSelectionToMacro, "Collapse to Macro", "Collapses selected nodes into a single macro node.", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( ConvertFunctionToEvent, "Convert Function to Event", "Converts selected function to an event and removes the function definition.", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( ConvertEventToFunction, "Convert Event to Function", "Converts the selected event to a function graph and removes this event node.", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( AlignNodesTop, "Align Top", "Aligns the top edges of the selected nodes", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::W) )
	UI_COMMAND( AlignNodesMiddle, "Align Middle", "Aligns the vertical middles of the selected nodes", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift|EModifierKey::Alt, EKeys::W) )
	UI_COMMAND( AlignNodesBottom, "Align Bottom", "Aligns the bottom edges of the selected nodes", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::S) )
	UI_COMMAND( AlignNodesLeft, "Align Left", "Aligns the left edges of the selected nodes", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::A) )
	UI_COMMAND( AlignNodesCenter, "Align Center", "Aligns the horizontal centers of the selected nodes", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift | EModifierKey::Alt, EKeys::S) )
	UI_COMMAND( AlignNodesRight, "Align Right", "Aligns the right edges of the selected nodes", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::D) )

	UI_COMMAND( StraightenConnections, "Straighten Connection(s)", "Straightens connections between the selected nodes.", EUserInterfaceActionType::Button, FInputChord(EKeys::Q) )

	UI_COMMAND( DistributeNodesHorizontally, "Distribute Horizontally", "Evenly distributes the selected nodes horizontally", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( DistributeNodesVertically, "Distribute Vertically", "Evenly distributes the selected nodes vertically", EUserInterfaceActionType::Button, FInputChord() )
	
	UI_COMMAND( StackNodesHorizontally, "Stack Horizontally", "Stacks the selected nodes horizontally", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( StackNodesVertically, "Stack Vertically", "Stacks the selected nodes vertically", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( EnableNodes, "Enable Nodes", "Selected node(s) will be enabled.", EUserInterfaceActionType::Check, FInputChord() )
	UI_COMMAND( DisableNodes, "Disable Nodes", "Selected node(s) will be disabled.", EUserInterfaceActionType::Check, FInputChord() )
	UI_COMMAND( EnableNodes_Always, "Enable Nodes (Always)", "Selected node(s) will always be enabled.", EUserInterfaceActionType::RadioButton, FInputChord() )
	UI_COMMAND( EnableNodes_DevelopmentOnly, "Enable Nodes (Development Only)", "Selected node(s) will be enabled in development mode only.", EUserInterfaceActionType::RadioButton, FInputChord() )

	UI_COMMAND( SelectReferenceInLevel, "Find Actor in Level", "Select the actor referenced by this node in the level", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( AssignReferencedActor, "Assign Selected Actor", "Assign the selected actor to be this node's referenced object", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( FindReferences, "Find References", "Find references of this item", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Shift | EModifierKey::Alt) )
	UI_COMMAND( FindReferencesByNameLocal, "By Name", "Find references of this item by name", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( FindReferencesByNameGlobal, "By Name (All)", "Find references of this item by name in all blueprints", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( FindReferencesByClassMemberLocal, "By Class Member", "Find references of this item by class member", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( FindReferencesByClassMemberGlobal, "By Class Member (All)", "Find references of this item by class member in all blueprints", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( FindAndReplaceReferences, "Replace References", "Brings up a window to help find and replace all instances of this item", EUserInterfaceActionType::Button, FInputChord() )
	
	UI_COMMAND( GoToDefinition, "Goto Definition", "Jumps to the definition of the selected node if available, e.g., C++ code for a native function or the graph for a Blueprint function.", EUserInterfaceActionType::Button, FInputChord(EKeys::G, EModifierKey::Alt) )

	UI_COMMAND( OpenInNewTab, "Open in New Tab", "Opens this function, macro or collapsed graph in a new tab.", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( BreakThisLink, "Break This Link", "Breaks the selected pin's only link.", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( BreakPinLinks, "Break All Pin Links", "Breaks all of the selected pin's links", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( PromoteToVariable, "Promote to Variable", "Promotes something to a variable", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( PromoteToLocalVariable, "Promote to Local Variable", "Promotes something to a local variable of the current function", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( SplitStructPin, "Split Struct Pin", "Breaks a struct pin in to a separate pin per element", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( RecombineStructPin, "Recombine Struct Pin", "Takes struct pins that have been broken in to composite elements and combines them back to a single struct pin", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( StartWatchingPin, "Watch This Value", "Adds this pin or variable to the watch list", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( StopWatchingPin, "Stop Watching This Value", "Removes this pin or variable from the watch list ", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( ResetPinToDefaultValue, "Reset to Default Value", "Reset value of this pin to the default", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( SelectAllInputNodes, "Select All Input Nodes", "Adds all input Nodes linked to this Pin to selection", EUserInterfaceActionType::Button, FInputChord() )
	UI_COMMAND( SelectAllOutputNodes, "Select All Output Nodes", "Adds all output Nodes linked to this Pin to selection", EUserInterfaceActionType::Button, FInputChord() )

	UI_COMMAND( CreateComment, "Create Comment", "Create a comment box", EUserInterfaceActionType::Button, FInputChord(EKeys::C))
	UI_COMMAND( CreateCustomEvent, "Create Custom Event", "Create a new custom event node.", EUserInterfaceActionType::Button, FInputChord())

	UI_COMMAND( ZoomIn, "Zoom In", "Zoom in on the graph editor", EUserInterfaceActionType::Button, FInputChord(EKeys::Add))
	UI_COMMAND( ZoomOut, "Zoom Out", "Zoom out from the graph editor", EUserInterfaceActionType::Button, FInputChord(EKeys::Subtract))

	UI_COMMAND( GoToDocumentation, "View Documentation", "View documentation for this node.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( SummonCreateNodeMenu, "Open Create Node Menu", "Opens the create node menu at the last known mouse position.", EUserInterfaceActionType::Button, FInputChord(EKeys::Tab) );

	// Map quick jump index to command key bindings.
	TArray< FKey, TInlineAllocator<10> > NumberKeys;
	NumberKeys.Add( EKeys::Zero );
	NumberKeys.Add( EKeys::One );
	NumberKeys.Add( EKeys::Two );
	NumberKeys.Add( EKeys::Three );
	NumberKeys.Add( EKeys::Four );
	NumberKeys.Add( EKeys::Five );
	NumberKeys.Add( EKeys::Six );
	NumberKeys.Add( EKeys::Seven );
	NumberKeys.Add( EKeys::Eight );
	NumberKeys.Add( EKeys::Nine );

	const int32 NumQuickJumpCommands = NumberKeys.Num();
	QuickJumpCommands.Reserve(NumQuickJumpCommands);

	for (int32 QuickJumpIndex = 0; QuickJumpIndex < NumQuickJumpCommands; ++QuickJumpIndex)
	{
		const FText QuickJumpIndexText = FText::AsNumber(QuickJumpIndex);

		FQuickJumpCommandInfo QuickJumpCommandInfo;

		QuickJumpCommandInfo.QuickJump =
			FUICommandInfoDecl(
				this->AsShared(), //Command class
				FName(*FString::Printf(TEXT("QuickJump%i"), QuickJumpIndex)), //CommandName
				FText::Format(LOCTEXT("QuickJump", "Quick Jump {0}"), QuickJumpIndexText), //Localized label
				FText::Format(LOCTEXT("QuickJump_ToolTip", "Jump to the location and zoom level bound to {0}"), QuickJumpIndexText))//Localized tooltip
			.UserInterfaceType(EUserInterfaceActionType::Button) //interface type
			.DefaultChord(FInputChord(EModifierKey::Shift, NumberKeys[QuickJumpIndex])); //default chord

		QuickJumpCommandInfo.SetQuickJump =
			FUICommandInfoDecl(
				this->AsShared(), //Command class
				FName(*FString::Printf(TEXT("SetQuickJump%i"), QuickJumpIndex)), //CommandName
				FText::Format(LOCTEXT("SetQuickJump", "Set Quick Jump {0}"), QuickJumpIndexText), //Localized label
				FText::Format(LOCTEXT("SetQuickJump_ToolTip", "Save the graph's current location and zoom level as quick jump {0}"), QuickJumpIndexText))//Localized tooltip
			.UserInterfaceType(EUserInterfaceActionType::Button) //interface type
			.DefaultChord(FInputChord(EModifierKey::Control, NumberKeys[QuickJumpIndex])); //default chord

		QuickJumpCommandInfo.ClearQuickJump =
			FUICommandInfoDecl(
				this->AsShared(), //Command class
				FName(*FString::Printf(TEXT("ClearQuickJump%i"), QuickJumpIndex)), //CommandName
				FText::Format(LOCTEXT("ClearQuickJump", "Clear Quick Jump {0}"), QuickJumpIndexText), //Localized label
				FText::Format(LOCTEXT("ClearQuickJump_ToolTip", "Clear the saved location and zoom level at quick jump {0}"), QuickJumpIndexText))//Localized tooltip
			.UserInterfaceType(EUserInterfaceActionType::Button) //interface type
			.DefaultChord(FInputChord()); //default chord 

		QuickJumpCommands.Add(QuickJumpCommandInfo);
	}

	UI_COMMAND( ClearAllQuickJumps, "Clear All Quick Jumps", "Clear all quick jump bindings", EUserInterfaceActionType::Button, FInputChord() );
}

void FGraphEditorCommands::Register()
{
	return FGraphEditorCommandsImpl::Register();
}

const FGraphEditorCommandsImpl& FGraphEditorCommands::Get()
{
	return FGraphEditorCommandsImpl::Get();
}

void FGraphEditorCommands::BuildFindReferencesMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("FindReferences");
	{
		MenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().FindReferencesByNameLocal);
		MenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().FindReferencesByNameGlobal);
		MenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().FindReferencesByClassMemberLocal);
		MenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().FindReferencesByClassMemberGlobal);
	}
	MenuBuilder.EndSection();
}

void FGraphEditorCommands::Unregister()
{
	return FGraphEditorCommandsImpl::Unregister();
}

#undef LOCTEXT_NAMESPACE
