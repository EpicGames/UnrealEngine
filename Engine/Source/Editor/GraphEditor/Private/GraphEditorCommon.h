// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "UnrealEd.h"
#include "GraphEditor.h"

#include "SlateBasics.h"

#include "GraphEditorSettings.h"
#include "SNodePanel.h"
#include "SGraphPanel.h"
#include "SCommentBubble.h"
#include "SGraphNode.h"
#include "SGraphNodeResizable.h"
#include "SGraphPin.h"
#include "BlueprintUtilities.h"
#include "GraphEditorDragDropAction.h"
#include "DragConnection.h"
#include "DragNode.h"
