// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphEditorDragDropAction.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "SPinTypeSelector.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class SToolTip;

FGraphEditorDragDropAction::FGraphEditorDragDropAction()
	: bDropTargetValid(true)
{
}

UEdGraphPin* FGraphEditorDragDropAction::GetHoveredPin() const
{
	return HoveredPin.Get();
}

UEdGraphNode* FGraphEditorDragDropAction::GetHoveredNode() const
{
	return HoveredNode.Get();
}

UEdGraph* FGraphEditorDragDropAction::GetHoveredGraph() const
{
	// Note: We always want to report a graph even when hovering over a node or pin;
	// the same is not true for nodes when hovering over a pin (at least right now)
	if (HoveredGraph.IsValid())
	{
		return HoveredGraph->GetGraphObj();
	}
	else if (UEdGraphNode* Node = GetHoveredNode())
	{
		return Node->GetGraph();
	}
	else if (UEdGraphPin* Pin = GetHoveredPin())
	{
		return Pin->GetOwningNode()->GetGraph();
	}
	
	return NULL;
}

void FGraphEditorDragDropAction::SetHoveredPin(UEdGraphPin* InPin)
{
	if (HoveredPin.Get() != InPin)
	{
		HoveredPin = InPin;
		HoverTargetChanged();
	}
}

void FGraphEditorDragDropAction::SetHoveredNode(const TSharedPtr<SGraphNode>& InNode)
{
	SetHoveredNode(InNode.IsValid() ? InNode->GetNodeObj() : nullptr);
}

void FGraphEditorDragDropAction::SetHoveredNode(UEdGraphNode* InNode)
{
	if (HoveredNode != InNode)
	{
		HoveredNode = InNode;
		HoverTargetChanged();
	}
}

void FGraphEditorDragDropAction::SetHoveredGraph(const TSharedPtr<SGraphPanel>& InGraph)
{
	if (HoveredGraph != InGraph)
	{
		HoveredGraph = InGraph;
		HoverTargetChanged();
	}
}

void FGraphEditorDragDropAction::SetHoveredCategoryName(const FText& InHoverCategoryName)
{
	if(!HoveredCategoryName.EqualTo(InHoverCategoryName))
	{
		HoveredCategoryName = InHoverCategoryName;
		HoverTargetChanged();
	}
}

void FGraphEditorDragDropAction::SetHoveredAction(TSharedPtr<struct FEdGraphSchemaAction> Action)
{
	if(HoveredAction.Pin().Get() != Action.Get())
	{
		HoveredAction = Action;
		HoverTargetChanged();
	}
}

FReply FGraphEditorDragDropAction::DroppedOnPin(FVector2D ScreenPosition, FVector2D GraphPosition)
{
	return FReply::Unhandled();
}

FReply FGraphEditorDragDropAction::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return DroppedOnPin(FVector2D(ScreenPosition), FVector2D(GraphPosition));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FReply FGraphEditorDragDropAction::DroppedOnNode(FVector2D ScreenPosition, FVector2D GraphPosition)
{
	return FReply::Unhandled();
}

FReply FGraphEditorDragDropAction::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return DroppedOnNode(FVector2D(ScreenPosition), FVector2D(GraphPosition));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FReply FGraphEditorDragDropAction::DroppedOnPanel(const TSharedRef<SWidget>& Panel, FVector2D ScreenPosition, FVector2D GraphPosition, UEdGraph& Graph)
{
	return FReply::Unhandled();
}

FReply FGraphEditorDragDropAction::DroppedOnPanel(const TSharedRef<SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return DroppedOnPanel(Panel, FVector2D(ScreenPosition), FVector2D(GraphPosition), Graph);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FGraphEditorDragDropAction::Construct()
{
	// Create the drag-drop decorator window
	CursorDecoratorWindow = SWindow::MakeCursorDecorator();
	const bool bShowImmediately = false;
	FSlateApplication::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), bShowImmediately);

	HoverTargetChanged();
}

bool FGraphEditorDragDropAction::HasFeedbackMessage()
{
	return CursorDecoratorWindow->GetContent() != SNullWidget::NullWidget;
}

void FGraphEditorDragDropAction::SetFeedbackMessage(const TSharedPtr<SWidget>& Message)
{
	if (Message.IsValid())
	{
		CursorDecoratorWindow->ShowWindow();
		CursorDecoratorWindow->SetContent
		(
			SNew(SBorder)
			. BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			[
				Message.ToSharedRef()
			]				
		);
	}
	else
	{
		CursorDecoratorWindow->HideWindow();
		CursorDecoratorWindow->SetContent(SNullWidget::NullWidget);
	}
}

void FGraphEditorDragDropAction::SetSimpleFeedbackMessage(const FSlateBrush* Icon, const FSlateColor& IconColor, const FText& Message, const FSlateBrush* SecondaryIcon /*= nullptr*/, const FSlateColor SecondaryColor /*= FSlateColor()*/)
{
	// Let the user know the status of making this connection.

	// Use CreateRaw as we cannot using anything that will create a shared ptr from within an objects construction, this should be
	// safe though as we will destroy our window before we get destroyed.
	TAttribute<EVisibility> ErrorIconVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateRaw(this, &FGraphEditorDragDropAction::GetErrorIconVisible));
	TAttribute<EVisibility> IconVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateRaw(this, &FGraphEditorDragDropAction::GetIconVisible));
	
	TSharedRef<SWidget> TypeImage = SPinTypeSelector::ConstructPinTypeImage(Icon, IconColor, SecondaryIcon, SecondaryColor, TSharedPtr<SToolTip>());

	SetFeedbackMessage(
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.0f)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			.Visibility(ErrorIconVisibility)
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush( TEXT("Graph.ConnectorFeedback.Error") ))
				.ColorAndOpacity( FLinearColor::White )
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.0f)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			.Visibility(IconVisibility)
			[
				TypeImage
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.0f)
		.MaxWidth(500)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.WrapTextAt( 480 )
			.Text( Message )
		]
	);
}

EVisibility FGraphEditorDragDropAction::GetIconVisible() const
{
	return bDropTargetValid ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FGraphEditorDragDropAction::GetErrorIconVisible() const
{
	return bDropTargetValid ? EVisibility::Collapsed : EVisibility::Visible;
}

////////////////////////////////////////////////////////////

void FGraphSchemaActionDragDropAction::HoverTargetChanged()
{
	if (SourceAction.IsValid())
	{
		const FSlateBrush* PrimarySymbol;
		const FSlateBrush* SecondarySymbol;
		FSlateColor PrimaryColor;
		FSlateColor SecondaryColor;
		GetDefaultStatusSymbol(/*out*/ PrimarySymbol, /*out*/ PrimaryColor, /*out*/ SecondarySymbol, /*out*/ SecondaryColor);

		//Create feedback message with the function name.
		SetSimpleFeedbackMessage(PrimarySymbol, PrimaryColor, SourceAction->GetMenuDescription(), SecondarySymbol, SecondaryColor);
	}
}

void FGraphSchemaActionDragDropAction::GetDefaultStatusSymbol(const FSlateBrush*& PrimaryBrushOut, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut) const
{
	PrimaryBrushOut = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.NewNode"));
	IconColorOut = FLinearColor::White;
	SecondaryBrushOut = nullptr;
	SecondaryColorOut = FLinearColor::White;
}

FReply FGraphSchemaActionDragDropAction::DroppedOnPanel( const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	if (SourceAction.IsValid())
	{
		TArray<UEdGraphPin*> DummyPins;
		SourceAction->PerformAction(&Graph, DummyPins, FDeprecateSlateVector2D(GraphPosition));

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply FGraphSchemaActionDragDropAction::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	if (UEdGraph* Graph = GetHoveredGraph())
	{
		if (SourceAction.IsValid())
		{
			TArray<UEdGraphPin*> DummyPins;
			if (UEdGraphPin* Pin = GetHoveredPin())
			{
				DummyPins.Add(Pin);
			}

			SourceAction->PerformAction(Graph, DummyPins, FDeprecateSlateVector2D(GraphPosition));

			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}
