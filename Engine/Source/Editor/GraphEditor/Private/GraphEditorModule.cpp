// Copyright Epic Games, Inc. All Rights Reserved.


#include "GraphEditorModule.h"

#include "GraphEditorActions.h"
#include "Modules/ModuleManager.h"
#include "SGraphEditorImpl.h"
#include "ToolMenus.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FUICommandList;
class SWidget;
struct FDiffSingleResult;

IMPLEMENT_MODULE(FGraphEditorModule, GraphEditor);

#define LOCTEXT_NAMESPACE "GraphEditorModule"

/////////////////////////////////////////////////////
// FGraphEditorModule

void FGraphEditorModule::StartupModule()
{
	FGraphEditorCommands::Register();

	TArray< TWeakPtr<SGraphEditor> >& Instances = SGraphEditor::AllInstances;
	for (auto InstanceIt = SGraphEditor::AllInstances.CreateIterator(); InstanceIt; ++InstanceIt)
	{
		TWeakPtr<SGraphEditor>& Instance = *InstanceIt;
		if (Instance.IsValid())
		{
			Instance.Pin()->OnModuleReloaded();
		}		
	}
}

void FGraphEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	// Notify all the instances of GraphEditor that their code is about to be unloaded.
	for (auto InstanceIt = SGraphEditor::AllInstances.CreateIterator(); InstanceIt; ++InstanceIt)
	{
		TWeakPtr<SGraphEditor>& Instance = *InstanceIt;
		if (Instance.IsValid())
		{
			Instance.Pin()->OnModuleUnloading();
		}		
	}

	FGraphEditorCommands::Unregister();
}

/**
 * DO NOT CALL THIS METHOD. Use SNew(SGraphEditor) to make instances of SGraphEditor.
 *
 * @return A GraphEditor implementation.
 */
TSharedRef<SGraphEditor> FGraphEditorModule::PRIVATE_MakeGraphEditor( 
	const TSharedPtr<FUICommandList>& InAdditionalCommands, 
	const TAttribute<bool>& InIsEditable,
	const TAttribute<bool>& InDisplayAsReadOnly,
	const TAttribute<bool>& InIsEmpty,
	TAttribute<FGraphAppearanceInfo> Appearance,
	TSharedPtr<SWidget> InTitleBar,
	UEdGraph* InGraphToEdit,
	SGraphEditor::FGraphEditorEvents InGraphEvents,
	bool InAutoExpandActionMenu,
	TSharedPtr<TArray<FDiffSingleResult>> DiffResults,
	TAttribute<int32> FocusedDiffResult,
	FSimpleDelegate InOnNavigateHistoryBack,
	FSimpleDelegate InOnNavigateHistoryForward,
	TAttribute<bool> ShowGraphStateOverlay,
	TAttribute<bool> AllowConnectionSlicing,
	FOnGraphEditorTickDelegate InOnTickDelegate,
	bool bGraphSupportsInvalidation)
{
	return
		SNew(SGraphEditorImpl)
		.AdditionalCommands(InAdditionalCommands)
		.IsEditable(InIsEditable)
		.DisplayAsReadOnly(InDisplayAsReadOnly)
		.Appearance(Appearance)
		.TitleBar(InTitleBar)
		.GraphToEdit(InGraphToEdit)
		.GraphEvents(InGraphEvents)
		.AutoExpandActionMenu(InAutoExpandActionMenu)
		.DiffResults(DiffResults)
		.FocusedDiffResult(FocusedDiffResult)
		.OnNavigateHistoryBack(InOnNavigateHistoryBack)
		.OnNavigateHistoryForward(InOnNavigateHistoryForward)
		.OnTick(InOnTickDelegate)
		.ShowGraphStateOverlay(ShowGraphStateOverlay)
		.AllowConnectionSlicing(AllowConnectionSlicing)
		.GraphSupportsInvalidation(bGraphSupportsInvalidation);
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE 
