// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphEditorSettings.h"
#include "UObject/UnrealType.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GraphEditorSettings)


/* UGraphEditorSettings structors
 *****************************************************************************/

UGraphEditorSettings::UGraphEditorSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, PanningMouseButton(EGraphPanningMouseButton::Right)
	, bUseHighPrecisionMouseMovement(true)
	, bUseInterpolationWithManualPanning(false)
	, PaddingAbovePin(4.0f)
	, PaddingBelowPin(4.0f)
	, PaddingRightOfInput(10.0f)
	, PaddingLeftOfOutput(10.0f)
	, PaddingTowardsNodeEdge(10.0f)
	, bTreatSplinesLikePins(true)
	, SplineHoverTolerance(2.0f)
	, SplineCloseTolerance(6.0f)
	, ForwardSplineHorizontalDeltaRange(1000.0f)
	, ForwardSplineVerticalDeltaRange(1000.0f)
	, ForwardSplineTangentFromHorizontalDelta(1.0f, 0.0f)
	, ForwardSplineTangentFromVerticalDelta(1.0f, 0.0f)
	, BackwardSplineHorizontalDeltaRange(200.0f)
	, BackwardSplineVerticalDeltaRange(200.0f)
	, BackwardSplineTangentFromHorizontalDelta(2.0f, 0.0f)
	, BackwardSplineTangentFromVerticalDelta(1.5f, 0.0f)
	, PaddingAutoCollateIncrement(20.0f)
	, bOpenCreateMenuOnBlankGraphAreas(true)
{
	DataPinStyle = BPST_VariantA;

	// graph node pin type colors
	DefaultPinTypeColor = FLinearColor(0.750000f, 0.6f, 0.4f, 1.0f);			// light brown
	ExecutionPinTypeColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);				// white
	BooleanPinTypeColor = FLinearColor(0.300000f, 0.0f, 0.0f, 1.0f);			// maroon
	BytePinTypeColor = FLinearColor(0.0f, 0.160000f, 0.131270f, 1.0f);			// dark green
	ClassPinTypeColor = FLinearColor(0.1f, 0.0f, 0.5f, 1.0f);					// deep purple (violet)
	IntPinTypeColor = FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f);		// green-blue
	Int64PinTypeColor = FLinearColor(0.413575f, 0.770000f, 0.429609f, 1.0f);	
	FloatPinTypeColor = FLinearColor(0.357667f, 1.0f, 0.060000f, 1.0f);			// bright green
	DoublePinTypeColor = FLinearColor(0.039216f, 0.666667f, 0.0f, 1.0f);		// darker green
	RealPinTypeColor = FLinearColor(0.039216f, 0.666667f, 0.0f, 1.0f);			// darker green
	NamePinTypeColor = FLinearColor(0.607717f, 0.224984f, 1.0f, 1.0f);			// lilac
	DelegatePinTypeColor = FLinearColor(1.0f, 0.04f, 0.04f, 1.0f);				// bright red
	ObjectPinTypeColor = FLinearColor(0.0f, 0.4f, 0.910000f, 1.0f);				// sharp blue
	SoftObjectPinTypeColor = FLinearColor(0.3f, 1.0f, 1.0f, 1.0f);					
	SoftClassPinTypeColor = FLinearColor(1.0f, 0.3f, 1.0f, 1.0f);				
	InterfacePinTypeColor = FLinearColor(0.8784f, 1.0f, 0.4f, 1.0f);			// pale green
	StringPinTypeColor = FLinearColor(1.0f, 0.0f, 0.660537f, 1.0f);				// bright pink
	TextPinTypeColor = FLinearColor(0.8f, 0.2f, 0.4f, 1.0f);					// salmon (light pink)
	StructPinTypeColor = FLinearColor(0.0f, 0.1f, 0.6f, 1.0f);					// deep blue
	WildcardPinTypeColor = FLinearColor(0.220000f, 0.195800f, 0.195800f, 1.0f);	// dark gray
	VectorPinTypeColor = FLinearColor(1.0f, 0.591255f, 0.016512f, 1.0f);		// yellow
	RotatorPinTypeColor = FLinearColor(0.353393f, 0.454175f, 1.0f, 1.0f);		// periwinkle 
	TransformPinTypeColor = FLinearColor(1.0f, 0.172585f, 0.0f, 1.0f);			// orange
	IndexPinTypeColor = FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f);	// green-blue

	DefaultDataWireThickness = 1.5f;
	DefaultExecutionWireThickness = 2.5f;

	// graph debugging visuals
	TraceAttackColor = FLinearColor(1.0f, 0.05f, 0.0f, 1.0f);
	TraceAttackWireThickness = 6.0f;
	TraceSustainColor = FLinearColor(1.0f, 0.7f, 0.4f, 1.0f);
	TraceSustainWireThickness = 3.5f;
	TraceReleaseColor = FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
	TraceReleaseWireThickness = 1.5f;

	// graph debugging exec curve constants
	TracePositionBonusPeriod = 0.5f;
	TracePositionExponent = 5.0f;
	TraceAttackHoldPeriod = 0.3f;
	TraceDecayPeriod = 0.4f;
	TraceDecayExponent = 1.8f;
	TraceSustainHoldPeriod = 0.4f;
	TraceReleasePeriod = 1.5f;
	TraceReleaseExponent = 1.4f;

	// Comment nodes
	DefaultCommentNodeMoveMode = ECommentBoxMode::GroupMovement;
	bShowCommentBubbleWhenZoomedOut = false;

	// Blueprint editor graph node title colors
	EventNodeTitleColor = FLinearColor(1.f, 0.0f, 0.0f, 1.0f);
	FunctionCallNodeTitleColor = FLinearColor(0.190525f, 0.583898f, 1.0f, 1.0f);
	PureFunctionCallNodeTitleColor = FLinearColor(0.4f, 0.850000f, 0.350000f, 1.0f);
	ParentFunctionCallNodeTitleColor = FLinearColor(1.0f, 0.170000f, 0.0f, 1.0f);
	FunctionTerminatorNodeTitleColor = FLinearColor(0.6f, 0.0f, 1.0f, 1.0f);
	ExecBranchNodeTitleColor = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	ExecSequenceNodeTitleColor = FLinearColor(0.8f, 0.4f, 0.4f, 1.0f);
	ResultNodeTitleColor = FLinearColor(1.0f, 0.65f, 0.4f, 1.0f);
	DefaultCommentNodeTitleColor = FLinearColor(0.15f, 0.15f, 0.15f, 0.5f);
	PreviewNodeTitleColor = FLinearColor(0.0f, 0.0f, 1.0f, 1.0f);

	// Graph connection slicing visuals
	SliceLineColor = FLinearColor(1.0f, 0.9f, 0.1f, 1.f);
	SliceLineDashLength = 8.f;
	SliceLineThickness = 2.f;
}

#if WITH_EDITOR
void UGraphEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UGraphEditorSettings, DataPinStyle)) || (PropertyName == GET_MEMBER_NAME_CHECKED(UGraphEditorSettings, PaddingTowardsNodeEdge)))
	{
		// Invalidate all node graph editors
		//@TODO: That thing I said
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

FMargin UGraphEditorSettings::GetInputPinPadding() const
{
	const float HPad = FMath::Max(PaddingTowardsNodeEdge, 0.0f);
	return FMargin(HPad, PaddingAbovePin, PaddingRightOfInput, PaddingBelowPin);
}

FMargin UGraphEditorSettings::GetOutputPinPadding() const
{
	const float HPad = FMath::Max(PaddingTowardsNodeEdge, 0.0f);
	return FMargin(PaddingLeftOfOutput, PaddingAbovePin, HPad, PaddingBelowPin);
}

FMargin UGraphEditorSettings::GetNonPinNodeBodyPadding() const
{
	const float NegativeHPad = FMath::Max(-PaddingTowardsNodeEdge, 0.0f);
	return FMargin(NegativeHPad, 0.0f, NegativeHPad, 0.0f);
}

UE::Slate::FDeprecateVector2DResult UGraphEditorSettings::GetShadowDeltaSize() const
{
	FVector2f ShadowSize = FAppStyle::GetVector(TEXT("Graph.Node.ShadowSize"));
	ShadowSize.X += FMath::Min(PaddingTowardsNodeEdge, 0.0f);
	return ShadowSize;
}

UE::Slate::FDeprecateVector2DResult UGraphEditorSettings::ComputeSplineTangent(const UE::Slate::FDeprecateVector2DParameter& Start, const UE::Slate::FDeprecateVector2DParameter& End) const
{
	const FVector2f DeltaPos = End - Start;
	const bool bGoingForward = DeltaPos.X >= 0.0f;

	const float ClampedTensionX = FMath::Min(FMath::Abs(DeltaPos.X), bGoingForward ? ForwardSplineHorizontalDeltaRange : BackwardSplineHorizontalDeltaRange);
	const float ClampedTensionY = FMath::Min(FMath::Abs(DeltaPos.Y), bGoingForward ? ForwardSplineVerticalDeltaRange : BackwardSplineVerticalDeltaRange);

	if (bGoingForward)
	{
		return (ClampedTensionX * ForwardSplineTangentFromHorizontalDelta) + (ClampedTensionY * ForwardSplineTangentFromVerticalDelta);
	}
	else
	{
		return (ClampedTensionX * BackwardSplineTangentFromHorizontalDelta) + (ClampedTensionY * BackwardSplineTangentFromVerticalDelta);
	}
}
