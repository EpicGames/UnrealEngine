// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphEditorSnapping/NodeSnappingManager.h"

#include "EdGraphSchema_K2.h"
#include "EdGraph/EdGraph.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "SGraphPanel.h"

namespace UE::Private
{

template<typename Predicate>
UEdGraphPin* GetOnlyVisiblePinWithDirection( UEdGraphNode* Node, EEdGraphPinDirection PinDirection, Predicate Pred )
{
	// if predicate matches multiple visible pins then we return nullptr:
	const TArray<UEdGraphPin*>& Pins = Node->GetAllPins();
	UEdGraphPin* Result = nullptr;
	for(UEdGraphPin* Pin : Pins)
	{
		if(Pin->bHidden)
		{
			continue;
		}

		if(Pin->Direction == PinDirection && Pred(Pin))
		{
			if(Result)
			{
				Result = nullptr;
				break;
			}
			else
			{
				Result = Pin;
			}
		}

	}
	return Result;
}

UEdGraphPin* GetFirstAndOnlyVisibleOutputPin(UEdGraphNode* ForNode)
{
	return GetOnlyVisiblePinWithDirection(
		ForNode, 
		EEdGraphPinDirection::EGPD_Output, 
		[](UEdGraphPin* Pin) { return true; }
	);
}

UEdGraphPin* GetFirstAndOnlyVisibleExecInPin(UEdGraphNode* OnNode)
{
	return GetOnlyVisiblePinWithDirection(
		OnNode, 
		EEdGraphPinDirection::EGPD_Input, 
		[](UEdGraphPin* Pin) { return UEdGraphSchema_K2::IsExecPin(*Pin); }
	);
}

UEdGraphPin* GetFirstAndOnlyVisibleExecOutPin(UEdGraphNode* OnNode)
{
	return GetOnlyVisiblePinWithDirection(
		OnNode, 
		EEdGraphPinDirection::EGPD_Output, 
		[](UEdGraphPin* Pin) { return UEdGraphSchema_K2::IsExecPin(*Pin); }
	);
}

UEdGraphPin* FindPinConnectedTo(UEdGraphNode* Node, UEdGraphPin* Pin)
{
	check(Node && Pin);
	for(UEdGraphPin* LinkedPin : Pin->LinkedTo)
	{
		if(LinkedPin->GetOwningNode() == Node)
		{
			return LinkedPin;
		}
	}
	return nullptr;
}

UEdGraphPin* FindFreeCompatiblePin(UEdGraphNode* OnNode, const UEdGraphPin* WithPin)
{
	if(WithPin->LinkedTo.Num() > 0)
	{
		return nullptr;
	}
	
	const UEdGraphSchema* Schema = OnNode->GetSchema();

	for(UEdGraphPin* PinOnNode : OnNode->Pins)
	{		
		if(PinOnNode->bHidden)
		{
			continue;
		}

		if(PinOnNode->LinkedTo.Num() > 0)
		{
			continue;
		}

		const FPinConnectionResponse Response = Schema->CanCreateConnection(
			PinOnNode, WithPin);
		if( Response.Response == ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE )
		{
			return PinOnNode;
		}
	}

	return nullptr;
}

bool HasAnyVisibleExecPins(UEdGraphNode* Node)
{
	for(UEdGraphPin* Pin : Node->Pins)
	{
		if(Pin->bHidden)
		{
			continue;
		}
		
		if(UEdGraphSchema_K2::IsExecPin(*Pin))
		{
			return true;
		}
	}
	return false;
}
//Slate<->Object helpers:
UEdGraphNode* GetEdGraphNode(TSharedRef<SGraphNode> FromSGraphNode)
{
	return CastChecked<UEdGraphNode>(FromSGraphNode->GetObjectBeingDisplayed());
}
bool AreCompatibleDirection(TSharedRef<SGraphPin> Pin, UEdGraphPin* OtherPin)
{
	return Pin->GetDirection() != OtherPin->Direction;
}

bool ConnectAndSnap(UEdGraphPin* PinA, UEdGraphPin* PinB)
{
	check(PinA && PinB);

	// if the pins are not connected then connect them:
	bool bMadeNewConnection = false;
	if(!PinA->LinkedTo.Contains(PinB))
	{
		bMadeNewConnection = GetDefault<UEdGraphSchema_K2>()->TryCreateConnection(PinA, PinB);
	}
	
	if(bMadeNewConnection && !PinA->LinkedTo.Contains(PinB))
	{
		ensure(false); // TryCreateConnection lied - relatively complicated function so lets be defensive
		bMadeNewConnection = false;
	}

	// execution pins should snap in execution order - data pins
	// should snap with consumer owning the producer:
	UEdGraphPin* FirstPin = PinA;
	UEdGraphPin* SecondPin = PinB;
	if(UEdGraphSchema_K2::IsExecPin(*FirstPin))
	{
		if(FirstPin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			::Swap(FirstPin, SecondPin);
		}
	}
	else
	{
		// @todo: may need to prefer to snap 'within' pins that are on nodes
		// with execution pins
		if(FirstPin->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			::Swap(FirstPin, SecondPin);
		}
	}

	if(FirstPin->GetSnappedChildPin() == SecondPin)
	{
		// already connected, nothing more to do
		return bMadeNewConnection;
	}

	FirstPin->SetSnappedChild(SecondPin);
	return bMadeNewConnection;
}

void ForEachDirectlySnappedNode(UEdGraphNode* Node, TFunctionRef<void(UEdGraphNode*)> FTor)
{
	for (const UEdGraphPin* Pin : Node->Pins)
	{
		UEdGraphPin* SnappedChildPin = Pin->GetSnappedChildPin();
		if(SnappedChildPin)
		{
			FTor(SnappedChildPin->GetOwningNode());
		}
		
		UEdGraphPin* SnappedParentPin = Pin->GetSnappedParentPin();
		if(SnappedParentPin)
		{
			FTor(SnappedParentPin->GetOwningNode());
		}
	}
}

}

FNodeSnappingManager::FNodeSnappingManager(TWeakPtr<SGraphPanel> InOwningPanel)
	: OwningPanel(InOwningPanel)
	, bIsActivelyDragging(false)
{
	check(InOwningPanel.Pin());
}

void FNodeSnappingManager::RemoveNode(const UEdGraphNode* Node)
{
	TSharedPtr<SGraphNode> GraphWidget = TryGetGraphNode(Node);
	if(GraphWidget)
	{
		ClearAllSnaps(GraphWidget.ToSharedRef());
	}
}

bool FNodeSnappingManager::IsDragging() const
{
	return bIsActivelyDragging;
}

bool FNodeSnappingManager::BeginDragWithSnap(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
{	
	TSharedPtr<SGraphPanel> OwningPanelPinned = OwningPanel.Pin();
	if(!OwningPanelPinned)
	{
		return false;
	}

	int32 NodeUnderMouseIndex = INDEX_NONE;
	FVector2f MousePositionInNode = FVector2f::ZeroVector;
	TSharedPtr<SGraphNode> NodeStartingDrag = FindNodeForMouseInteraction(MyGeometry, PointerEvent, MousePositionInNode).Widget;
		
	if( !NodeStartingDrag || !NodeStartingDrag->CanBeSelected(MousePositionInNode) )
	{
		return false;
	}

	OwningPanelPinned->NodeGrabOffset = MousePositionInNode;

	OwningPanelPinned->TotalMouseDelta = 0.f;
	bIsActivelyDragging = true;
	
	UEdGraphNode* ClickedNode = CastChecked<UEdGraphNode>(NodeStartingDrag->GetObjectBeingDisplayed());
	// don't update selection here, the selection manager is updated on mouse up/mouse move
	// so we have to project the selection. If selection is updated it will toggle in the
	// mouse up logic:
	TSet<TObjectPtr<UObject>> SelectedNodes;
	if(	PointerEvent.IsControlDown() || PointerEvent.IsShiftDown() || 
		OwningPanelPinned->SelectionManager.SelectedNodes.Contains(ClickedNode))
	{
		SelectedNodes = OwningPanelPinned->SelectionManager.SelectedNodes;
	}
	SelectedNodes.Add(ClickedNode);
	// prune any nodes from the drag set if their outer node is also selected, we can move
	// them by simply moving their inner - also override the root node with its outermost
	// selection:
	{
		TSet<TObjectPtr<UObject>> SelectedNodesPruned;
		for(TObjectPtr<UObject> Object : SelectedNodes)
		{
			UEdGraphNode* AsNode = CastChecked<UEdGraphNode>(Object);
			bool bShouldBePruned = AsNode->IsSnappedToAny(SelectedNodes);
			if(bShouldBePruned)
			{
				// don't prune exec level snaps - those are peers and must remain selected:
				bShouldBePruned = !UE::Private::HasAnyVisibleExecPins(AsNode);
			}
			if(!bShouldBePruned)
			{
				SelectedNodesPruned.Add(Object);
			}
		}
		SelectedNodes = MoveTemp(SelectedNodesPruned);

		if(!SelectedNodes.Contains(ClickedNode))
		{
			// use the root selected node as the node starting drag:
			UEdGraphNode* SelectionRoot = ClickedNode->GetRootSnappedParent(SelectedNodes);
			check(SelectionRoot);
			
			TSharedRef<SNodePanel::SNode>* Widget = OwningPanelPinned->NodeToWidgetLookup.Find(SelectionRoot);
			check(Widget != nullptr);
			NodeStartingDrag = StaticCastSharedRef<SGraphNode>(*Widget);
		}
	}
	
	MoveNodeTransaction = MakeUnique<FScopedTransaction>(
		SelectedNodes.Num() > 1 
		? NSLOCTEXT("GraphEditor", "MoveNodesAction", "Move Nodes")
		: NSLOCTEXT("GraphEditor", "MoveNodeAction", "Move Node"));
	
	for(TObjectPtr<UObject> Node : SelectedNodes)
	{
		UEdGraphNode* AsEdGraphNode = CastChecked<UEdGraphNode>(Node);
		ModifiedNodes.Add(AsEdGraphNode);
		MovingNodes.Add(AsEdGraphNode);
		Node->Modify();
	}
		
	for(TObjectPtr<UObject> Node : SelectedNodes)
	{
		UEdGraphNode* AsEdGraphNode = CastChecked<UEdGraphNode>(Node);
		// we'll also need to modify any nodes that are snapped to this node
		// they may be implicitly mutated by mutating these nodes
		FNodeSnappingManager* Self = this;
		UE::Private::ForEachDirectlySnappedNode(
			AsEdGraphNode,
			[Self](UEdGraphNode* Node)
			{
				if(!Self->ModifiedNodes.Contains(Node))
				{
					Self->ModifiedNodes.Add(Node);
					Node->Modify();
				}
			}
		);
	}

	PendingSnaps.SetNum(SelectedNodes.Num());
	int32 Idx = 0;
	for(TObjectPtr<UObject> Object : SelectedNodes)
	{
		FPendingSnap& MoveData = PendingSnaps[Idx];
		
		TSharedRef<SNodePanel::SNode>* Widget = OwningPanelPinned->NodeToWidgetLookup.Find(Object);
		check(Widget != nullptr);
		MoveData.NodeBeingDragged = StaticCastSharedRef<SGraphNode>(*Widget);

		++Idx;
	}
	RootDragNode = StaticCastSharedRef<SGraphNode>(NodeStartingDrag.ToSharedRef());
	OwningPanelPinned->NodeUnderMousePtr = RootDragNode;

	return true;
}

bool FNodeSnappingManager::FinalizeNodeMovements()
{
	if(!IsDragging())
	{
		return false; // no customization - fall back to marquee selection or pin commit or whatever
	}

	CommitDrag();
	return true;
}

void FNodeSnappingManager::UpdateDrag(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	ensure(bIsActivelyDragging);
	TSharedPtr<SGraphPanel> OwningPanelPinned = OwningPanel.Pin();
	if(!OwningPanelPinned)
	{
		// believed unreachable, hard to confirm with reference counts:
		ResetDragManager();
		return;
	}

	TSharedPtr<SGraphNode> NodeBeingDragged = RootDragNode;
	if(!ensure(NodeBeingDragged))
	{
		return;
	}
	
	const FVector2f CursorDelta = MouseEvent.GetCursorDelta();
	OwningPanelPinned->TotalMouseDelta += CursorDelta.Size();

	// Update the amount to pan panel
	OwningPanelPinned->UpdateViewOffset(MyGeometry, MouseEvent.GetScreenSpacePosition());

	const bool bCursorInDeadZone = OwningPanelPinned->TotalMouseDelta <= FSlateApplication::Get().GetDragTriggerDistance();
	
	// reset snap data for the nodes we're moving, we want to recalculate it:
	for(const FPendingConnection& UncommittedConnection : UncommittedConnections)
	{
		UEdGraphPin* DraggingPinResolved = UncommittedConnection.DraggingPin.Get();
		UEdGraphPin* StablePinResolved = UncommittedConnection.StablePin.Get();

		if(DraggingPinResolved && StablePinResolved)
		{
			GetSchema()->BreakSinglePinLink(DraggingPinResolved, StablePinResolved);
		}
	}
	UncommittedConnections.Reset();

	for(UEdGraphNode* Node : MovingNodes)
	{
		// reset snapped state - a moving node
		// should unsnap itself from any unselected node:
		for(UEdGraphPin* Pin : Node->Pins)
		{
			if(Pin->GetSnappedChildNode() && UEdGraphSchema_K2::IsExecPin(*Pin) && !MovingNodes.Contains(Pin->GetSnappedChildNode()))
			{
				Pin->ClearSnappedChild();
			}
			else if(Pin->LinkedTo.Num() > 0 )
			{
				UEdGraphPin* OwningPin = Pin->LinkedTo[0];
				UEdGraphNode* SnappedParentNode = OwningPin->GetSnappedChildNode();
				if(MovingNodes.Contains(SnappedParentNode) && !MovingNodes.Contains(OwningPin->GetOwningNode()))
				{
					Pin->LinkedTo[0]->ClearSnappedChild();
				}
			}
		}
	}
	
	for(FPendingSnap& ExistingPendingSnap : PendingSnaps)
	{
		if(TryToSnapDataflow(ExistingPendingSnap))
		{
			continue;
		}
		else
		{
			TryToSnapControlflow(ExistingPendingSnap);
		}
	}
	
	// move the nodes, this must be done after snapping or we'll get
	// incoherence in attached node position!
	if ( NodeBeingDragged.IsValid() )
	{
		if ( !bCursorInDeadZone )
		{
			// Note, NodeGrabOffset() comes from the node itself, so it's already scaled correctly.
			FVector2f AnchorNodeNewPos = OwningPanelPinned->PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) ) - OwningPanelPinned->NodeGrabOffset;

			// Snap to grid
			const float SnapSize = static_cast<float>(OwningPanelPinned->GetSnapGridSize());
			AnchorNodeNewPos.X = SnapSize * FMath::RoundToFloat( AnchorNodeNewPos.X / SnapSize );
			AnchorNodeNewPos.Y = SnapSize * FMath::RoundToFloat( AnchorNodeNewPos.Y / SnapSize );

			// Dragging an unselected node automatically selects it.
			OwningPanelPinned->SelectionManager.StartDraggingNode(NodeBeingDragged->GetObjectBeingDisplayed(), MouseEvent);

			// Move all the selected nodes.
			const FVector2f AnchorNodeOldPos = NodeBeingDragged->GetPosition2f();
			const FVector2f DeltaPos = AnchorNodeNewPos - AnchorNodeOldPos;
			SNodePanel::SNode::FNodeSet NodeFilter;

			for (const FPendingSnap& DraggingNode : PendingSnaps)
			{
				DraggingNode.NodeBeingDragged->MoveTo(
					DraggingNode.NodeBeingDragged->GetPosition2f() + DeltaPos,
					NodeFilter,
					false);
			}
		}
	}
	
	// first stage of refresh - move nodes back to root level:
	for(UEdGraphNode* Node : ModifiedNodes)
	{
		TSharedRef<SNodePanel::SNode>* Widget = OwningPanelPinned->NodeToWidgetLookup.Find(Node);
		check(Widget != nullptr);
		TSharedRef<SGraphNode> NodeTyped = StaticCastSharedRef<SGraphNode>(*Widget);
		ClearAllSnaps(NodeTyped);
	}
	
	// with the data model updated, update snap state for all display nodes:
	for(UEdGraphNode* Node : ModifiedNodes)
	{
		TSharedRef<SNodePanel::SNode>* Widget = OwningPanelPinned->NodeToWidgetLookup.Find(Node);
		check(Widget != nullptr);
		TSharedRef<SGraphNode> NodeTyped = StaticCastSharedRef<SGraphNode>(*Widget);
		UpdateSnapsFromModel(NodeTyped);
	}
}

void FNodeSnappingManager::HandleGraphChanged()
{
	TSharedRef<SGraphPanel> OwningPanelPinned = OwningPanel.Pin().ToSharedRef();
	UEdGraph* DisplayedGraph = OwningPanelPinned->GraphObj;
	if(!DisplayedGraph)
	{
		return;
	}
	
	// first stage of refresh - move nodes back to root level
	// this makes sure we are in a consistent state if a partial
	// refresh happens
	for(UEdGraphNode* Node : DisplayedGraph->Nodes)
	{
		TSharedRef<SNodePanel::SNode>* Widget = OwningPanelPinned->NodeToWidgetLookup.Find(Node);
		check(Widget != nullptr);
		TSharedRef<SGraphNode> NodeTyped = StaticCastSharedRef<SGraphNode>(*Widget);
		ClearAllSnaps(NodeTyped);
	}

	for(UEdGraphNode* Node : DisplayedGraph->Nodes)
	{
		TSharedRef<SNodePanel::SNode>* Widget = OwningPanelPinned->NodeToWidgetLookup.Find(Node);
		check(Widget != nullptr);
		TSharedRef<SGraphNode> NodeTyped = StaticCastSharedRef<SGraphNode>(*Widget);
		UpdateSnapsFromModel(NodeTyped);
	}
}

void FNodeSnappingManager::ForEachNode(TFunctionRef<void(const TSharedRef<SNodePanel::SNode>&)> Ftor) const
{
	TSharedRef<SGraphPanel> OwningPanelPinned = OwningPanel.Pin().ToSharedRef();
	
	const auto SGraphNodeAdapter = [&Ftor](const TSharedRef<SGraphNode>& GraphNode)
	{
		Ftor(GraphNode);
	};

	for (int32 ChildIndex = 0; ChildIndex < OwningPanelPinned->Children.Num(); ++ChildIndex)
	{
		TSharedRef<SGraphNode> AsSgraphNode = StaticCastSharedRef<SGraphNode>(OwningPanelPinned->Children[ChildIndex]);
		AsSgraphNode->ForEachNodeIncludingChildren(SGraphNodeAdapter);
	}
}

FArrangedGraphNode FNodeSnappingManager::GetGraphNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2f Discard;
	return FindNodeForMouseInteraction(MyGeometry, MouseEvent, Discard);
}

bool FNodeSnappingManager::IsNodeSnappingEnabled()
{
	return false;
}

void FNodeSnappingManager::CommitDrag()
{
	// no finalization work to do atm - we scoped our transaction
	// correctly so we don't have to do the whacky 'restore to original 
	// node positions/call modify' stuff that the normal code path did
	ResetDragManager();
}

void FNodeSnappingManager::ResetDragManager()
{
	MoveNodeTransaction.Reset();
	PendingSnaps.Reset();
	RootDragNode = nullptr;
	MovingNodes.Reset();
	ModifiedNodes.Reset();
	UncommittedConnections.Reset();
	bIsActivelyDragging = false;
}

const UEdGraphSchema* FNodeSnappingManager::GetSchema() const
{
	TSharedPtr<SGraphPanel> OwningPanelPinned = OwningPanel.Pin();
	check(OwningPanelPinned);
	return OwningPanelPinned->GraphObj->GetSchema();
}

bool FNodeSnappingManager::TryToSnapDataflow(FPendingSnap& PendingSnap)
{
	TSharedPtr<SGraphNode> NodeBeingSnapped = PendingSnap.NodeBeingDragged;
	TSharedPtr<SGraphPanel> OwningPanelPinned = OwningPanel.Pin();
	check(OwningPanelPinned);
	UEdGraphPin* PinToMatchForSnap = UE::Private::GetFirstAndOnlyVisibleOutputPin(NodeBeingSnapped->GetNodeObj());
	if(PinToMatchForSnap && PinToMatchForSnap->LinkedTo.Num() > 1)
	{
		PinToMatchForSnap = nullptr;
	}

	TSharedPtr<SGraphNode> DestGraphNode = FindNodeForSnap(NodeBeingSnapped);
	if(!PinToMatchForSnap || !DestGraphNode)
	{
		return false;
	}

	UEdGraphPin* PinToSnapTo = nullptr;
	if(PinToMatchForSnap->LinkedTo.Num() == 1)
	{
		PinToSnapTo = PinToMatchForSnap->LinkedTo[0];
	}

	TSharedPtr<SGraphPin> DestPin = DestGraphNode->GetHoveredPin(DestGraphNode->GetCachedGeometry(), OwningPanelPinned->LastPointerEvent);

	bool bIsDestPinValid = DestPin.IsValid();
	if(PinToSnapTo && bIsDestPinValid)
	{
		bIsDestPinValid = (DestPin->GetPinObj() == PinToSnapTo);
	}

	if(!bIsDestPinValid )
	{
		return false;
	}
	UEdGraphPin* DestGraphPin = DestPin->GetPinObj();
	const FPinConnectionResponse Response = GetSchema()->CanCreateConnection(
		DestGraphPin, PinToMatchForSnap);
	if( Response.Response != ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE && 
		!DestGraphPin->LinkedTo.Contains(PinToMatchForSnap))
	{
		return false;
	}

	bool bCreatedConnection = TrySnapTo(DestPin.ToSharedRef()->GetPinObj(), PinToMatchForSnap);
	if(bCreatedConnection)
	{
		UncommittedConnections.Add({DestGraphPin, PinToMatchForSnap});
	}

	return true;
}

bool FNodeSnappingManager::TryToSnapControlflow(FPendingSnap& PendingSnap)
{
	TSharedPtr<SGraphPanel> OwningPanelPinned = OwningPanel.Pin();
	check(OwningPanelPinned);
	TSharedPtr<SGraphNode> NodeBeingSnapped = PendingSnap.NodeBeingDragged;
	check(NodeBeingSnapped);
	UEdGraphNode* EdGraphNodeBeingSnapped = NodeBeingSnapped->GetNodeObj();

	UEdGraphPin* ExecutionPinIn = UE::Private::GetFirstAndOnlyVisibleExecInPin(EdGraphNodeBeingSnapped);
	UEdGraphPin* ExecutionPinOut = UE::Private::GetFirstAndOnlyVisibleExecOutPin(EdGraphNodeBeingSnapped);

	if( !ExecutionPinOut && !ExecutionPinIn )
	{
		return false; // no control flow to snap - bail
	}

	TSharedPtr<SGraphPin> ExecPinOutToSnapTo = FindBestControlFlowForSnap(NodeBeingSnapped, ExecutionPinIn);
	TSharedPtr<SGraphPin> ExecPinInToSnapTo = FindBestControlFlowForSnap(NodeBeingSnapped, ExecutionPinOut);
	TSharedPtr<SGraphPin> PreferredPin = GetPreferredControlFlowSnapTarget(ExecPinOutToSnapTo, ExecPinInToSnapTo);
	UEdGraphPin* PinOnDraggingNode = PreferredPin == ExecPinOutToSnapTo ? ExecutionPinIn : ExecutionPinOut;
	if(!PreferredPin)
	{
		return false;
	}

	// this doesn't snap hierarchically, instead the pending snap is merely positioned
	// and restyled such that it looks as if it is connected to its peer. If either
	// node is moved the nodes will restyle:
	SnapControlFlowPeer(PendingSnap, PreferredPin.ToSharedRef(), PinOnDraggingNode);
	return true;
}

bool FNodeSnappingManager::TrySnapTo(UEdGraphPin* OtherPin, UEdGraphPin* PinOnMovingNode)
{
	check(OtherPin && OtherPin != PinOnMovingNode);
	ModifySafe(OtherPin);
	return UE::Private::ConnectAndSnap(PinOnMovingNode, OtherPin);
}

void FNodeSnappingManager::SnapControlFlowPeer(FPendingSnap& SnapState, TSharedRef<SGraphPin> PinToSnapTo, UEdGraphPin* PinOnMovingNode)
{
	UEdGraphPin* OtherPin = PinToSnapTo->GetPinObj();
	check(OtherPin && OtherPin != PinOnMovingNode);

	TSharedRef<SGraphNode> NodeBeingDragged = SnapState.NodeBeingDragged.ToSharedRef();
	TSharedRef<SGraphNode> NodeDroppedOn = PinToSnapTo->OwnerNodePtr.Pin().ToSharedRef();

	TSharedRef<SGraphNode> StartingNode = (PinToSnapTo->GetDirection() ==  EEdGraphPinDirection::EGPD_Input) ? 
		NodeBeingDragged : NodeDroppedOn;
	
	// update the data model.. this is all we should really do here - but I have made some mistakes
	const bool bCreatedConnection = TrySnapTo(PinOnMovingNode, OtherPin);
	if(bCreatedConnection)
	{
		UncommittedConnections.Add({PinOnMovingNode, OtherPin});
	}

	SnapState.PinSnappedTo = PinToSnapTo;
}

void FNodeSnappingManager::ClearAllSnaps(TSharedRef<SGraphNode> OnNode)
{
	ClearControlFlowSnap(OnNode);
	ClearDataSnap(OnNode);
}

void FNodeSnappingManager::ClearControlFlowSnap(TSharedRef<SGraphNode> OnNode)
{
	const auto IsAlreadySnapped = [](TSharedRef<SGraphNode> NodeSnappingTo, TSharedRef<SGraphNode> NodeToSnap)
	{
		const FChildren* Children = NodeSnappingTo->ControlFlowSequence->GetChildren();

		for (int32 I = 0; I < Children->Num(); ++I)
		{
			TSharedRef<const SWidget> ChildWidget = Children->GetChildAt(I);
			if (ChildWidget == NodeToSnap)
			{
				return true;
			}
		}
		return false;
	};

	TSharedPtr<SGraphNode> NodeParent = OnNode->SnapParent.Pin();
	if(NodeParent)
	{
		TSharedRef<SGraphNode> NodeParentRef = NodeParent.ToSharedRef();
		if(NodeParentRef->ControlFlowSequence && IsAlreadySnapped(NodeParentRef, OnNode))
		{
			NodeParentRef->ControlFlowSequence->RemoveSlot(OnNode);
			NodeParentRef->GetOwnerPanel()->Children.Add(OnNode);
			OnNode->SnapParent = nullptr;
		}
	}

	if(OnNode->ControlFlowSequence)
	{
		// unsnap subsequent children:
		FChildren* Children = OnNode->ControlFlowSequence->GetChildren();

		for (int32 I = Children->Num() - 1; I >= 1 && Children->Num(); --I)
		{
			TSharedRef<SGraphNode> ChildRef = StaticCastSharedRef<SGraphNode>(Children->GetChildAt(I));
			OnNode->GetOwnerPanel()->Children.Add(ChildRef);
			OnNode->ControlFlowSequence->RemoveSlot(ChildRef);
			ChildRef->SnapParent = nullptr;
		}
	}
}

void FNodeSnappingManager::ClearDataSnap(TSharedRef<SGraphNode> OnNode)
{
	// clear snap to our parent node:
	TSharedRef<SGraphPanel> OwningPanelPinned = OnNode->GetOwnerPanel().ToSharedRef();
	TSharedPtr<SGraphNode> NodeParent = OnNode->SnapParent.Pin();
	if(NodeParent)
	{
		for(TSharedRef<SGraphPin> Pin : NodeParent->InputPins)
		{
			TSharedPtr<SGraphNode> SnappedNode = Pin->SnappedNode.Pin();
			if(SnappedNode == OnNode)
			{
				Pin->ClearSnappedSGraphNode();
			}
		}
		for(TSharedRef<SGraphPin> Pin : NodeParent->OutputPins)
		{
			TSharedPtr<SGraphNode> SnappedNode = Pin->SnappedNode.Pin();
			if(SnappedNode== OnNode)
			{
				Pin->ClearSnappedSGraphNode();
			}
		}
		OwningPanelPinned->Children.Add(OnNode);
		OnNode->SnapParent = nullptr;
	}

	// clear snap to our children node:
	for(TSharedRef<SGraphPin> Pin : OnNode->InputPins)
	{
		TSharedPtr<SGraphNode> SnappedNode = Pin->SnappedNode.Pin();
		if(SnappedNode)
		{
			Pin->ClearSnappedSGraphNode();
			OwningPanelPinned->Children.Add(SnappedNode.ToSharedRef());
			SnappedNode->SnapParent = nullptr;
		}
	}
	for(TSharedRef<SGraphPin> Pin : OnNode->OutputPins)
	{
		TSharedPtr<SGraphNode> SnappedNode = Pin->SnappedNode.Pin();
		if(SnappedNode)
		{
			Pin->ClearSnappedSGraphNode();
			OwningPanelPinned->Children.Add(SnappedNode.ToSharedRef());
			SnappedNode->SnapParent = nullptr;
		}
	}
}

void FNodeSnappingManager::UpdateSnapsFromModel(TSharedRef<SGraphNode> OnNode)
{
	TSharedRef<SGraphPanel> OwningPanelPinned = OnNode->GetOwnerPanel().ToSharedRef();
	const UEdGraphNode* NodeObj = OnNode->GetNodeObj();
	// if we have a snapped input pin to after it:
	for(UEdGraphPin* Pin : NodeObj->Pins)
	{
		// if no snap, nothing to do:
		FSnappedEdGraphPin SnappedPins = Pin->GetSnapDescription();
		if(SnappedPins == FSnappedEdGraphPin())
		{
			continue;
		}
		
		// continue if the ui is already snapped - this handles cycles in the graph - 
		// e.g. two nodes are being modified at once - multiple snaps reworking, etc:
		TSharedRef<SGraphNode> ParentNode = GetSGraphNode(OwningPanelPinned, SnappedPins.ParentPin->GetOwningNode());
		TSharedRef<SGraphNode> ChildNode = GetSGraphNode(OwningPanelPinned, SnappedPins.ChildPin->GetOwningNode());
		if(ChildNode->SnapParent.Pin() != nullptr)
		{
			ensure(ChildNode->SnapParent == ParentNode);
			continue;
		}

		const bool bPinIsParent = (SnappedPins.ParentPin == Pin);
		const bool bPinIsChild = !bPinIsParent;

		if(UEdGraphSchema_K2::IsExecPin(*Pin))
		{
			// control flow snap - @todo: move to a function
			// if the node is being moved, snap it into position before the unmoving node:
			if(MovingNodes.Contains(NodeObj) && !MovingNodes.Contains(Pin->GetSnappedChildNode()) && bPinIsParent)
			{
				// set the moving node into the correct position:
				FVector2f NodePosition = Pin->GetSnappedChildNode()->GetPosition();
				NodePosition -= FVector2f(0.f, OnNode->GetDesiredSizeForMarquee2f().Y); // does my marquee size grow? need to cache it? 😷 back to vsnap objects..
				SNodePanel::SNode::FNodeSet NodeFilter;
				OnNode->MoveTo(NodePosition, NodeFilter, false);
			}
			else if(MovingNodes.Contains(NodeObj) && bPinIsChild)
			{
				// moving node needs to be moved beneath parent, so that selection
				// etc can occur in the corrrect position - subtle problems will
				// be created if the node is snapped but not in the right position
				// (e.g. failure to select pins because their notional position
				// is different than their painted position):
				FVector2f NodePosition = ParentNode->GetPosition2f();
				NodePosition += FVector2f(0.f, ParentNode->GetDesiredSizeForMarquee2f().Y);
				SNodePanel::SNode::FNodeSet NodeFilter;
				ChildNode->MoveTo(NodePosition, NodeFilter, false);
			}

			SNodePanel::SNode::FNodeSlot* CenterSlot = ParentNode->GetSlot(ENodeZone::Center);
			if(!ParentNode->ControlFlowSequence.IsValid())
			{
				ParentNode->ControlFlowSequence = SNew(SVerticalBox)
				 +SVerticalBox::Slot()
				.AutoHeight()
				[
					CenterSlot->GetWidget()
				];
				ParentNode->GetOrAddSlot(ENodeZone::Center)
					[
						ParentNode->ControlFlowSequence.ToSharedRef()
					];
			}

			ParentNode->ControlFlowSequence->AddSlot()
				.AutoHeight()
				[
					ChildNode
				];
			ChildNode->SnapParent = ParentNode;
			
			OwningPanelPinned->Children.Remove(ChildNode);
			OwningPanelPinned->VisibleChildren.Remove(ChildNode);
		}
		else
		{
			// data snap - @todo: move to a function
			TSharedRef<SGraphPin> PinToSnapTo = ParentNode->FindWidgetForPin(SnappedPins.ParentPin).ToSharedRef();
			if(PinToSnapTo->SnapSGraphNode(ChildNode))
			{
				ChildNode->SnapParent = ParentNode;
				OwningPanelPinned->Children.Remove(ChildNode);
				OwningPanelPinned->VisibleChildren.Remove(ChildNode);
			}
		}
	}
}

FArrangedGraphNode FNodeSnappingManager::FindNodeForMouseInteraction( const FGeometry& MyGeometry, const FPointerEvent& PointerEvent, FVector2f& OutPositionInNode ) const
{
	TSharedPtr<SGraphPanel> OwningPanelPinned = OwningPanel.Pin();
	if(!OwningPanelPinned)
	{
		return FArrangedGraphNode();
	}
	FVector2f ArrangedSpacePosition = PointerEvent.GetScreenSpacePosition();
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	OwningPanelPinned->ArrangeChildNodes(MyGeometry, ArrangedChildren);

	// loop over children, preferring nodes at leaves of snap hierarchy to roots:
	const int32 NumChildren = ArrangedChildren.Num();
	for( int32 ChildIndex=NumChildren-1; ChildIndex >= 0; --ChildIndex )
	{
		const FArrangedWidget& Candidate = ArrangedChildren[ChildIndex];
		const bool bCandidateUnderCursor = 
			// Candidate is physically under the cursor
			Candidate.Geometry.IsUnderLocation( ArrangedSpacePosition );
		
		if (bCandidateUnderCursor)
		{
			OutPositionInNode = Candidate.Geometry.AbsoluteToLocal(ArrangedSpacePosition);
			TSharedRef<SGraphNode> NodeWidgetUnderMouse = 
				StaticCastSharedRef<SGraphNode>( Candidate.Widget );

			const bool bIsSelected = (PointerEvent.IsControlDown() || PointerEvent.IsShiftDown()) && 
				OwningPanelPinned->SelectionManager.SelectedNodes.Contains(NodeWidgetUnderMouse->GetObjectBeingDisplayed());

			if(!bIsSelected)
			{
				TSharedPtr<SGraphNode> InnerNode = NodeWidgetUnderMouse->GetSnappedChildForPosition(
					Candidate.Geometry, ArrangedSpacePosition, OutPositionInNode);
				if(InnerNode)
				{
					NodeWidgetUnderMouse = InnerNode.ToSharedRef();
				}
			}

			return {Candidate.Geometry, NodeWidgetUnderMouse};
		}
	}

	return FArrangedGraphNode();
}

TSharedPtr<SGraphNode> FNodeSnappingManager::FindNodeForSnap(TSharedPtr<SGraphNode> NodeBeingMoved)
{
	TSharedPtr<SGraphPanel> OwningPanelPinned = OwningPanel.Pin();
	if(!OwningPanelPinned)
	{
		return nullptr;
	}

	const FGeometry& MyGeometry = OwningPanelPinned->LastPointerGeometry;
	const FPointerEvent& MouseEvent = OwningPanelPinned->LastPointerEvent;

	FArrangedChildren ArrangedNodes(EVisibility::Visible);
	
	for (int32 ChildIndex = 0; ChildIndex < OwningPanelPinned->Children.Num(); ++ChildIndex)
	{
		const TSharedRef<SNodePanel::SNode>& SomeChild = OwningPanelPinned->Children[ChildIndex];

		FArrangedWidget ArrangedWidget = MyGeometry.MakeChild(
			SomeChild, 
			SomeChild->GetPosition2f() - OwningPanelPinned->GetViewOffset(),
			SomeChild->GetDesiredSize(), 
			OwningPanelPinned->GetZoomAmount()
		);
		ArrangedNodes.AddWidget( ArrangedWidget );
	}

	const auto FindChildrenUnderMouse = [](const FArrangedChildren& Children, const FPointerEvent& PointerEvent) -> FArrangedChildren
	{
		FVector2f AbsoluteCursorLocation = PointerEvent.GetScreenSpacePosition();
		FArrangedChildren WidgetsUnderMouse(EVisibility::Visible);
		const int32 NumChildren = Children.Num();
		for( int32 ChildIndex=NumChildren-1; ChildIndex >= 0; --ChildIndex )
		{
			const FArrangedWidget& Candidate = Children[ChildIndex];
			const bool bCandidateUnderCursor = 
				// Candidate is physically under the cursor
				Candidate.Geometry.IsUnderLocation( AbsoluteCursorLocation );
	
			if (bCandidateUnderCursor)
			{
				WidgetsUnderMouse.AddWidget(Candidate);
			}
		}
		return WidgetsUnderMouse;
	};
	
	TSharedPtr<SGraphNode> GraphNode;
	FArrangedChildren WidgetsUnderMouse = FindChildrenUnderMouse(ArrangedNodes, MouseEvent);
	for(const FArrangedWidget& ArrangedWidget : WidgetsUnderMouse.GetInternalArray() )
	{
		TSharedPtr<SGraphNode> PotentialGraphNode = StaticCastSharedRef<SGraphNode>(ArrangedWidget.Widget);
		FVector2f Discard;
		TSharedPtr<SGraphNode> GraphSubNode =  PotentialGraphNode->GetSnappedChildForPosition(ArrangedWidget.Geometry, MouseEvent.GetScreenSpacePosition(), Discard);
		PotentialGraphNode = GraphSubNode.IsValid() ? GraphSubNode.ToSharedRef() : PotentialGraphNode;
		if(PotentialGraphNode != NodeBeingMoved)
		{
			GraphNode = PotentialGraphNode;
			//break;
		}
	}

	return GraphNode;
}

TSharedPtr<SGraphPin> FNodeSnappingManager::FindBestControlFlowForSnap(TSharedPtr<SGraphNode> NodeBeingMoved, UEdGraphPin* Pin)
{
	if(!Pin)
	{
		return nullptr;
	}

	TSharedPtr<SGraphNode> Node = FindNodeForSnap(NodeBeingMoved);
	if(!Node)
	{
		return nullptr;
	}

	// Lets snap to the sgraph node if:
	//  1. we're already attached
	// or
	//  2. we're not attached to anything and the node's execution pin also isn't: (@todo)
	UEdGraphNode* EdGraphNodeBeingSnappedTo = UE::Private::GetEdGraphNode(Node.ToSharedRef());
	UEdGraphPin* PinToSnap = UE::Private::FindPinConnectedTo(EdGraphNodeBeingSnappedTo, Pin);

	if(!PinToSnap)
	{
		// we're not connected yet, but lets see if EdGraphNodeBeingSnappedTo has
		// a compatible pin that is free:
		PinToSnap = UE::Private::FindFreeCompatiblePin(EdGraphNodeBeingSnappedTo, Pin);
	}

	if(!PinToSnap)
	{
		return nullptr;
	}
	
	return Node->FindWidgetForPin(PinToSnap);
}

TSharedPtr<SGraphPin> FNodeSnappingManager::GetPreferredControlFlowSnapTarget(TSharedPtr<SGraphPin> A, TSharedPtr<SGraphPin> B) const
{
	if(!A)
	{
		return B;
	}
	else if(!B)
	{
		return A;
	}

	// @todo: tie break.. closest node?
	return nullptr;
}

TSharedRef<SGraphNode> FNodeSnappingManager::GetSGraphNode(TSharedRef<SGraphPanel> FromPanel, const UEdGraphNode* Node)
{
	const TSharedRef<SNodePanel::SNode>* Widget = FromPanel->NodeToWidgetLookup.Find(Node);
	check(Widget != nullptr); // this will return null for some delete operations - which run redundant removals - consider TryGetGraphNode
	return StaticCastSharedRef<SGraphNode>(*Widget);
}

TSharedPtr<SGraphNode> FNodeSnappingManager::TryGetGraphNode(const UEdGraphNode* Node)
{
	TSharedRef<SGraphPanel> OwningPanelPinned = OwningPanel.Pin().ToSharedRef();
	const TSharedRef<SNodePanel::SNode>* Widget = OwningPanelPinned->NodeToWidgetLookup.Find(Node);
	if(Widget)
	{
		return StaticCastSharedRef<SGraphNode>(*Widget);
	}
	return TSharedPtr<SGraphNode>();
}

void FNodeSnappingManager::ModifySafe(UEdGraphPin* Pin)
{
	UEdGraphNode* Node = Pin->GetOwningNode();
	if(!ModifiedNodes.Contains(Node))
	{
		ModifiedNodes.Add(Node);
		Node->Modify();
	}
}

