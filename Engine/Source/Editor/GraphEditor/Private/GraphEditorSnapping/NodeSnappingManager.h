// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "SGraphNode.h"

struct FArrangedGraphNode
{
	FGeometry Geometry;
	TSharedPtr<SGraphNode> Widget;
};

struct FNodeSnappingManager
{
	FNodeSnappingManager(TWeakPtr<SGraphPanel> InOwningPanel);

	void RemoveNode(const UEdGraphNode* Node);
	bool IsDragging() const;
	bool BeginDragWithSnap(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent);
	bool FinalizeNodeMovements();
	void UpdateDrag(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void HandleGraphChanged();
	void ForEachNode(TFunctionRef<void(const TSharedRef<SNodePanel::SNode>&)> Ftor) const;
	FArrangedGraphNode GetGraphNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	static bool IsNodeSnappingEnabled();
private:
	struct FPendingSnap
	{
		TSharedPtr<SGraphNode> NodeBeingDragged;
		TSharedPtr<SGraphPin> PinSnappedTo;

		bool operator==(const FPendingSnap& RHS) const = default;
	};

	void CommitDrag();
	void ResetDragManager();
	const UEdGraphSchema* GetSchema() const;
	bool TryToSnapDataflow(FPendingSnap& PendingSnap);
	bool TryToSnapControlflow(FPendingSnap& PendingSnap);
	bool TrySnapTo(UEdGraphPin* OtherPin, UEdGraphPin* PinOnMovingNode);
	void SnapControlFlowPeer(FPendingSnap& SnapState, TSharedRef<SGraphPin> PinToSnapTo, UEdGraphPin* PinOnMovingNode);
	
	void ClearAllSnaps(TSharedRef<SGraphNode> OnNode);
	void ClearControlFlowSnap(TSharedRef<SGraphNode> OnNode);
	void ClearDataSnap(TSharedRef<SGraphNode> OnNode);
	void UpdateSnapsFromModel(TSharedRef<SGraphNode> OnNode);
	static TArray<TSharedRef<SGraphNode>> GetControlFlowInputNodes(TSharedRef<SGraphNode> ForNode);
	static TArray<TSharedRef<SGraphNode>> GetControlFlowOutputNodes(TSharedRef<SGraphNode> ForNode);
	
	FArrangedGraphNode FindNodeForMouseInteraction( const FGeometry& MyGeometry, const FPointerEvent& PointerEvent, FVector2f& OutPositionInNode ) const;
	TSharedPtr<SGraphNode> FindNodeForSnap(TSharedPtr<SGraphNode> NodeBeingMoved);
	TSharedPtr<SGraphPin> FindBestControlFlowForSnap(TSharedPtr<SGraphNode> NodeBeingMoved, UEdGraphPin* Pin);
	TSharedPtr<SGraphPin> GetPreferredControlFlowSnapTarget(TSharedPtr<SGraphPin> A, TSharedPtr<SGraphPin> B) const;
	static TSharedRef<SGraphNode> GetSGraphNode(TSharedRef<SGraphPanel> FromPanel, const UEdGraphNode* Node);
	TSharedPtr<SGraphNode> TryGetGraphNode(const UEdGraphNode* Node);
	void ModifySafe(UEdGraphPin* Pin);

	TWeakPtr<SGraphPanel> OwningPanel; // must be weak, otherwise we'll have a cycle, never meant to be null

	TUniquePtr<FScopedTransaction> MoveNodeTransaction;
	// @todo: rename
	TArray<FPendingSnap> PendingSnaps;
	TSharedPtr<SGraphNode> RootDragNode; // actual node that is being dragged, used to calculate deltas
	TSet<UEdGraphNode*> MovingNodes;
	TSet<UEdGraphNode*> ModifiedNodes; // track modification to make sure we don't modify after mutating

	// A list of uncommitted connections, which will be created only if the final state of the 
	// drag operation leaves them in connected positions. This is meant to facilitate connecting
	// nodes by dragging them on top of eachother/compatible pins. We use FEdGraphPinReference
	// for persistent references to visible pins across node refreshes:
	struct FPendingConnection
	{
		FEdGraphPinReference DraggingPin;
		FEdGraphPinReference StablePin;
	};
	TArray<FPendingConnection> UncommittedConnections;

	bool bIsActivelyDragging;
};