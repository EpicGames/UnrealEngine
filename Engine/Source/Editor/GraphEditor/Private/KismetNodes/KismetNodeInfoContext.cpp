// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/KismetNodeInfoContext.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/LatentActionManager.h"
#include "Engine/World.h"
#include "HAL/PlatformCrt.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"

class UEdGraphNode;

//////////////////////////////////////////////////////////////////////////
// FKismetNodeInfoContext

// Context used to aid debugging displays for nodes
FKismetNodeInfoContext::FKismetNodeInfoContext(UEdGraph* SourceGraph)
	: ActiveObjectBeingDebugged(NULL)
{
	// Only show pending latent actions in PIE/SIE mode
	SourceBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(SourceGraph);

	if (SourceBlueprint != NULL)
	{
		ActiveObjectBeingDebugged = SourceBlueprint->GetObjectBeingDebugged();

		// Run thru debugged objects to see if any are objects with pending latent actions
		if (ActiveObjectBeingDebugged != NULL)
		{
			UBlueprintGeneratedClass* Class = CastChecked<UBlueprintGeneratedClass>((UObject*)(ActiveObjectBeingDebugged->GetClass()));
			FBlueprintDebugData const& ClassDebugData = Class->GetDebugData();

			TSet<UObject*> LatentContextObjects;

			TArray<UK2Node_CallFunction*> FunctionNodes;
			SourceGraph->GetNodesOfClass<UK2Node_CallFunction>(FunctionNodes);
			// collect all the world context objects for all of the graph's latent nodes
			for (UK2Node_CallFunction const* FunctionNode : FunctionNodes)
			{
				UFunction* Function = FunctionNode->GetTargetFunction();
				if ((Function == NULL) || !Function->HasMetaData(FBlueprintMetadata::MD_Latent))
				{
					continue;
				}

				UObject* NodeWorldContext = ActiveObjectBeingDebugged;
				// if the node has a specific "world context" pin, attempt to get the value set for that first
				if (Function->HasMetaData(FBlueprintMetadata::MD_WorldContext))
				{
					const FString& WorldContextPinName = Function->GetMetaData(FBlueprintMetadata::MD_WorldContext);
					if (UEdGraphPin* ContextPin = FunctionNode->FindPin(WorldContextPinName))
					{
						if (FObjectPropertyBase* ContextProperty = CastField<FObjectPropertyBase>(ClassDebugData.FindClassPropertyForPin(ContextPin)))
						{
							UObject* PropertyValue = ContextProperty->GetObjectPropertyValue_InContainer(ActiveObjectBeingDebugged);
							if (PropertyValue != NULL)
							{
								NodeWorldContext = PropertyValue;
							}
						}
					}
				}
				
				LatentContextObjects.Add(NodeWorldContext);
			}

			for (UObject* ContextObject : LatentContextObjects)
			{
				if (UWorld* World = GEngine->GetWorldFromContextObject(ContextObject, EGetWorldErrorMode::ReturnNull))
				{
					FLatentActionManager& Manager = World->GetLatentActionManager();

					TSet<int32> UUIDSet;
					Manager.GetActiveUUIDs(ActiveObjectBeingDebugged, /*out*/ UUIDSet);

					for (TSet<int32>::TConstIterator IterUUID(UUIDSet); IterUUID; ++IterUUID)
					{
						const int32 UUID = *IterUUID;

						if (UEdGraphNode* ParentNode = ClassDebugData.FindNodeFromUUID(UUID))
						{
 							TArray<FObjectUUIDPair>& Pairs = NodesWithActiveLatentActions.FindOrAdd(ParentNode);
 							new (Pairs) FObjectUUIDPair(ContextObject, UUID);
						}
					}
				}
			}
		}

		// Covert the watched pin array into a set
		FKismetDebugUtilities::ForeachPinWatch(
			SourceBlueprint,
			[&WatchedPinSet = WatchedPinSet, &WatchedNodeSet = WatchedNodeSet]
				(UEdGraphPin* WatchedPin)
				{
					if (!ensure(WatchedPin))
					{
						return; // ~continue
					}

					UEdGraphNode* OwningNode = WatchedPin->GetOuter();
					if (!ensure(OwningNode != NULL)) // shouldn't happen, but just in case a dead pin was added to the WatchedPins array
					{
						return; // ~continue
					}
					check(OwningNode == WatchedPin->GetOwningNode());

					WatchedPinSet.Add(WatchedPin);
					WatchedNodeSet.Add(OwningNode);
				}
		);
	}
}
