// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeCallParameterCollectionFunction.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "K2Node_CallMaterialParameterCollectionFunction.h"
#include "Materials/MaterialParameterCollection.h"
#include "NodeFactory.h"
#include "SGraphPinNameList.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

//////////////////////////////////////////////////////////////////////////
// SGraphNodeCallParameterCollectionFunction

TSharedPtr<SGraphPin> SGraphNodeCallParameterCollectionFunction::CreatePinWidget(UEdGraphPin* Pin) const
{
	UK2Node_CallMaterialParameterCollectionFunction* CallFunctionNode = Cast<UK2Node_CallMaterialParameterCollectionFunction>(GraphNode);

	// Create a special pin class for the ParameterName pin
	if (CallFunctionNode
		&& Pin->PinName == TEXT("ParameterName") 
		&& Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		TArray<FName> NameList;

		UEdGraphPin* CollectionPin = GraphNode->FindPin(TEXT("Collection"));

		if (CollectionPin)
		{
			UMaterialParameterCollection* Collection = Cast<UMaterialParameterCollection>(CollectionPin->DefaultObject);

			if (Collection)
			{
				// Populate the ParameterName pin combobox with valid options from the Collection
				const bool bVectorParameters = CallFunctionNode->FunctionReference.GetMemberName().ToString().Contains(TEXT("Vector"));
				Collection->GetParameterNames(NameList, bVectorParameters);
			}

			NameList.Sort([](const FName& A, const FName& B)
			{
				return A.LexicalLess(B);
			});
		}

		TArray<TSharedPtr<FName>> NamePtrList;

		for (FName NameItem : NameList)
		{
			NamePtrList.Add(MakeShareable( new FName(NameItem)));
		}

		TSharedPtr<SGraphPin> NewPin = SNew(SGraphPinNameList, Pin, NamePtrList);
		return NewPin;
	}
	else
	{
		return FNodeFactory::CreatePinWidget(Pin);
	}
}

