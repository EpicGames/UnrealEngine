// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeFormatText.h"

#include "Containers/Array.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_FormatText.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "NodeFactory.h"
#include "SGraphNode.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Widgets/SBoxPanel.h"

class SGraphPin;
class SWidget;


//////////////////////////////////////////////////////////////////////////
// SGraphNodeFormatText

void SGraphNodeFormatText::Construct(const FArguments& InArgs, UK2Node_FormatText* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor( EMouseCursor::CardinalCross );

	this->UpdateGraphNode();
}

void SGraphNodeFormatText::CreatePinWidgets()
{
	// Create Pin widgets for each of the pins, except for the default pin
	for (auto PinIt = GraphNode->Pins.CreateConstIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* CurrentPin = *PinIt;
		if ((!CurrentPin->bHidden))
		{
			TSharedPtr<SGraphPin> NewPin = FNodeFactory::CreatePinWidget(CurrentPin);
			check(NewPin.IsValid());

			this->AddPin(NewPin.ToSharedRef());
		}
	}
}

void SGraphNodeFormatText::CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox)
{
	TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
										NSLOCTEXT("FormatTextNode", "FormatTextNodeAddPinButton", "Add pin"),
										NSLOCTEXT("FormatTextNode", "FormatTextNodeAddPinButton_Tooltip", "Adds an argument to the node"),
										false);

	FMargin AddPinPadding = Settings->GetInputPinPadding();
	AddPinPadding.Top += 6.0f;

	InputBox->AddSlot()
	.AutoHeight()
	.VAlign(VAlign_Center)
	.Padding(AddPinPadding)
	[
		AddPinButton
	];
}

EVisibility SGraphNodeFormatText::IsAddPinButtonVisible() const
{
	EVisibility VisibilityState = EVisibility::Collapsed;
	if(NULL != Cast<UK2Node_FormatText>(GraphNode))
	{
		VisibilityState = SGraphNode::IsAddPinButtonVisible();
		if(VisibilityState == EVisibility::Visible)
		{
			UK2Node_FormatText* FormatNode = CastChecked<UK2Node_FormatText>(GraphNode);
			VisibilityState = FormatNode->CanEditArguments()? EVisibility::Visible : EVisibility::Collapsed;
		}
	}
	return VisibilityState;
}

FReply SGraphNodeFormatText::OnAddPin()
{
	if(UK2Node_FormatText* FormatText = Cast<UK2Node_FormatText>(GraphNode))
	{
		FormatText->AddArgumentPin();
	}
	return FReply::Handled();
}
