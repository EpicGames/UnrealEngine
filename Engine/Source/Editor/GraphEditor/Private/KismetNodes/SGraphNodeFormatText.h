// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SVerticalBox;
class UK2Node_FormatText;

class SGraphNodeFormatText : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeFormatText){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UK2Node_FormatText* InNode);

	// SGraphNode interface
	virtual void CreatePinWidgets() override;

protected:
	virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox) override;
	virtual EVisibility IsAddPinButtonVisible() const override;
	virtual FReply OnAddPin() override;
	// End of SGraphNode interface
};
