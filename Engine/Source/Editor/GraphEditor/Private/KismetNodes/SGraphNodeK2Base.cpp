// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetNodes/SGraphNodeK2Base.h"

#include "BlueprintEditorSettings.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "Engine/LatentActionManager.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "IDocumentation.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_Timeline.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Breakpoint.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/WatchedPin.h"
#include "KismetNodes/KismetNodeInfoContext.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "SCommentBubble.h"
#include "SGraphPin.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "TutorialMetaData.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/FieldPath.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "SGraphNodeK2Base"

//////////////////////////////////////////////////////////////////////////
// SGraphNodeK2Base

const FLinearColor SGraphNodeK2Base::BreakpointHitColor(0.7f, 0.0f, 0.0f);
const FLinearColor SGraphNodeK2Base::LatentBubbleColor(1.f, 0.5f, 0.25f);
const FLinearColor SGraphNodeK2Base::TimelineBubbleColor(0.7f, 0.5f, 0.5f);
const FLinearColor SGraphNodeK2Base::PinnedWatchColor(0.35f, 0.25f, 0.25f);

void SGraphNodeK2Base::UpdateStandardNode()
{
	SGraphNode::UpdateGraphNode();
	// clear the default tooltip, to make room for our custom "complex" tooltip
	SetToolTip(nullptr);
}

void SGraphNodeK2Base::UpdateCompactNode()
{
	InputPins.Empty();
	OutputPins.Empty();

	// error handling set-up
	SetupErrorReporting();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

 	TSharedPtr< SToolTip > NodeToolTip = SNew( SToolTip );
	if (!GraphNode->GetTooltipText().IsEmpty())
	{
		NodeToolTip = IDocumentation::Get()->CreateToolTip( TAttribute< FText >( this, &SGraphNode::GetNodeTooltip ), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName() );
	}

	// Setup a meta tag for this node
	FGraphNodeMetaData TagMeta(TEXT("Graphnode"));
	PopulateMetaTag(&TagMeta);
	
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode)
		.Text(this, &SGraphNodeK2Base::GetNodeCompactTitle);

	TSharedRef<SOverlay> NodeOverlay = SNew(SOverlay);
	
	// add optional node specific widget to the overlay:
	TSharedPtr<SWidget> OverlayWidget = GraphNode->CreateNodeImage();
	if(OverlayWidget.IsValid())
	{
		NodeOverlay->AddSlot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew( SBox )
			.WidthOverride( 70.f )
			.HeightOverride( 70.f )
			[
				OverlayWidget.ToSharedRef()
			]
		];
	}

	NodeOverlay->AddSlot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(45.f, 0.f, 45.f, 0.f)
		[
			// MIDDLE
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoHeight()
			[
				SNew(STextBlock)
					.TextStyle( GetStyleSet(), "Graph.CompactNode.Title" )
					.Text( NodeTitle.Get(), &SNodeTitle::GetHeadTitle )
					.WrapTextAt(128.0f)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				NodeTitle.ToSharedRef()
			]
		];
	
	// Default to "pure" styling, where we can just center the pins vertically
	// since don't need to worry about alignment with other nodes
	float PinPaddingTop = 0.f;

	static float MinNodePadding = 55.f;
	// Calculate a padding amount clamping to the min/max settings
	float PinPaddingRight = MinNodePadding;

	EVerticalAlignment PinVerticalAlignment = VAlign_Center;

	// But if this is an impure node, we'll align the pins to the top, 
	// and add some padding so that the exec pins line up with the exec pins of other nodes
	if (UK2Node* K2Node = Cast<UK2Node>(GraphNode))
	{
		const bool bIsPure = K2Node->IsNodePure();
		if (!bIsPure)
		{
			PinPaddingTop += 8.0f;
			PinVerticalAlignment = VAlign_Top;
		}

		if (K2Node->ShouldDrawCompact() && bIsPure)
		{
			// If the center node title is 2 or more, then make the node bigger
			// so that the text box isn't over top of the label
			static float MaxNodePadding = 180.0f;
			static float PaddingIncrementSize = 20.0f;

			int32 HeadTitleLength = NodeTitle.Get() ? NodeTitle.Get()->GetHeadTitle().ToString().Len() : 0;

			PinPaddingRight = FMath::Clamp(MinNodePadding + ((float)HeadTitleLength * PaddingIncrementSize), MinNodePadding, MaxNodePadding);
		}
	}

	NodeOverlay->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(PinVerticalAlignment)
		.Padding(/* left */ 0.f, PinPaddingTop, PinPaddingRight, /* bottom */ 0.f)
		[
			// LEFT
			SAssignNew(LeftNodeBox, SVerticalBox)
		];

	NodeOverlay->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(PinVerticalAlignment)
		.Padding(55.f, PinPaddingTop, 0.f, 0.f)
		[
			// RIGHT
			SAssignNew(RightNodeBox, SVerticalBox)
		];

	//
	//             ______________________
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |   +  | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	TSharedRef<SVerticalBox> InnerVerticalBox =
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			// NODE CONTENT AREA
			SNew( SOverlay)
			+SOverlay::Slot()
			[
				SNew(SImage)
				.Image( GetStyleSet().GetBrush("Graph.VarNode.Body") )
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( GetStyleSet().GetBrush("Graph.VarNode.Gloss") )
			]
			+SOverlay::Slot()
			.Padding( FMargin(0,3) )
			[
				NodeOverlay
			]
		];
	
	TSharedPtr<SWidget> EnabledStateWidget = GetEnabledStateWidget();
	if (EnabledStateWidget.IsValid())
	{
		InnerVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.Padding(FMargin(2, 0))
			[
				EnabledStateWidget.ToSharedRef()
			];
	}

	InnerVerticalBox->AddSlot()
		.AutoHeight()
		.Padding( FMargin(5.0f, 1.0f) )
		[
			ErrorReporting->AsWidget()
		];

	this->GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		InnerVerticalBox
	];

	CreatePinWidgets();

	// Hide pin labels
	for (auto InputPin: this->InputPins)
	{
		if (InputPin->GetPinObj()->ParentPin == nullptr)
		{
			InputPin->SetShowLabel(false);
		}
	}

	for (auto OutputPin : this->OutputPins)
	{
		if (OutputPin->GetPinObj()->ParentPin == nullptr)
		{
			OutputPin->SetShowLabel(false);
		}
	}

	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew( CommentBubble, SCommentBubble )
	.GraphNode( GraphNode )
	.Text( this, &SGraphNode::GetNodeComment )
	.OnTextCommitted( this, &SGraphNode::OnCommentTextCommitted )
	.ColorAndOpacity( CommentColor )
	.AllowPinning( true )
	.EnableTitleBarBubble( true )
	.EnableBubbleCtrls( true )
	.GraphLOD( this, &SGraphNode::GetCurrentLOD )
	.IsGraphNodeHovered( this, &SGraphNodeK2Base::IsHovered );

	GetOrAddSlot( ENodeZone::TopCenter )
	.SlotOffset2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetOffset2f ))
	.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f ))
	.AllowScaling( TAttribute<bool>( CommentBubble.Get(), &SCommentBubble::IsScalingAllowed ))
	.VAlign( VAlign_Top )
	[
		CommentBubble.ToSharedRef()
	];

	CreateInputSideAddButton(LeftNodeBox);
	CreateOutputSideAddButton(RightNodeBox);
}

TSharedPtr<SToolTip> SGraphNodeK2Base::GetComplexTooltip()
{
	TSharedPtr<SToolTip> NodeToolTip;
	TSharedRef<SToolTip> DefaultToolTip = IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &SGraphNode::GetNodeTooltip), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName());

	struct LocalUtils
	{
		static EVisibility IsToolTipVisible(TWeakPtr<SGraphNodeK2Base> const NodeWidget)
		{
			TSharedPtr<SGraphNodeK2Base> NodeWidgetPinned = NodeWidget.Pin();
			return NodeWidgetPinned && NodeWidgetPinned->GetNodeTooltip().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
		}

		static EVisibility IsToolTipHeadingVisible(TWeakPtr<SGraphNodeK2Base> const NodeWidget)
		{
			TSharedPtr<SGraphNodeK2Base> NodeWidgetPinned = NodeWidget.Pin();
			return NodeWidgetPinned && NodeWidgetPinned->GetToolTipHeading().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
		}

		static bool IsInteractive()
		{
			const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
			return ( ModifierKeys.IsAltDown() && ModifierKeys.IsControlDown() );
		}
	};
	TWeakPtr<SGraphNodeK2Base> ThisWeak = SharedThis(this);

	TSharedPtr< SVerticalBox > VerticalBoxWidget;
	SAssignNew(NodeToolTip, SToolTip)
		.Visibility_Static(&LocalUtils::IsToolTipVisible, ThisWeak)
		.IsInteractive_Static(&LocalUtils::IsInteractive)

		// Emulate text-only tool-tip styling that SToolTip uses when no custom content is supplied.  We want node tool-tips to 
		// be styled just like text-only tool-tips
		.BorderImage( FCoreStyle::Get().GetBrush("ToolTip.BrightBackground") )
		.TextMargin(FMargin(11.0f))
	[
		SAssignNew(VerticalBoxWidget, SVerticalBox)
		// heading container
		+SVerticalBox::Slot()
		[
			SNew(SVerticalBox)
				.Visibility_Static(&LocalUtils::IsToolTipHeadingVisible, ThisWeak)
			+SVerticalBox::Slot()
				.AutoHeight()
			[
				SNew(STextBlock)
					.TextStyle( GetStyleSet(), "Documentation.SDocumentationTooltipSubdued")
					.Text(this, &SGraphNodeK2Base::GetToolTipHeading)
			]
			+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 2.f, 0.f, 5.f)
			[
				SNew(SBorder)
				// use the border's padding to actually create the horizontal line
				.Padding(1.f)
				.BorderImage(GetStyleSet().GetBrush(TEXT("Menu.Separator")))
			]
		]
		// tooltip body
		+SVerticalBox::Slot()
			.AutoHeight()
		[
			DefaultToolTip->GetContentWidget()
		]
	];

	// English speakers have no real need to know this exists.
	if(FInternationalization::Get().GetCurrentCulture()->GetTwoLetterISOLanguageName() != TEXT("en"))
	{
		struct Local
		{
			static EVisibility GetNativeNodeNameVisibility()
			{
				return FSlateApplication::Get().GetModifierKeys().IsAltDown()? EVisibility::Collapsed : EVisibility::Visible;
			}
		};

		VerticalBoxWidget->AddSlot()
			.AutoHeight()
			.HAlign( HAlign_Right )
			[
				SNew( STextBlock )
				.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
				.Text( LOCTEXT( "NativeNodeName", "hold (Alt) for native node name" ) )
				.TextStyle( &GetStyleSet().GetWidgetStyle<FTextBlockStyle>(TEXT("Documentation.SDocumentationTooltip")) )
				.Visibility_Static(&Local::GetNativeNodeNameVisibility)
			];
	}
	return NodeToolTip;
}

FText SGraphNodeK2Base::GetToolTipHeading() const
{
	if (UK2Node const* K2Node = CastChecked<UK2Node>(GraphNode))
	{
		return K2Node->GetToolTipHeading();
	}
	return FText::GetEmpty();
}

const ISlateStyle& SGraphNodeK2Base::GetStyleSet() const
{
	return Style ? *Style : FAppStyle::Get();
}

void SGraphNodeK2Base::Construct(const FArguments& InArgs)
{
	Style = InArgs._Style;
}

/**
 * Update this GraphNode to match the data that it is observing
 */
void SGraphNodeK2Base::UpdateGraphNode()
{
	UK2Node* K2Node = CastChecked<UK2Node>(GraphNode);
	const bool bCompactMode = K2Node->ShouldDrawCompact();

	if (bCompactMode)	
	{
		UpdateCompactNode();
	}
	else
	{
		UpdateStandardNode();
	}
}

FText SGraphNodeK2Base::GetNodeCompactTitle() const
{
	UK2Node* K2Node = CastChecked<UK2Node>(GraphNode);
	return K2Node->GetCompactNodeTitle();
}

/** Populate the brushes array with any overlay brushes to render */
void SGraphNodeK2Base::GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const
{
	UBlueprint* OwnerBlueprint = FBlueprintEditorUtils::FindBlueprintForNode(GraphNode);

	// Search for an enabled or disabled breakpoint on this node
	FBlueprintBreakpoint* Breakpoint = OwnerBlueprint ? FKismetDebugUtilities::FindBreakpointForNode(GraphNode, OwnerBlueprint) : nullptr;
	if (Breakpoint != NULL)
	{
		FOverlayBrushInfo BreakpointOverlayInfo;

		if (Breakpoint->IsEnabledByUser())
		{
			BreakpointOverlayInfo.Brush = GetStyleSet().GetBrush(FKismetDebugUtilities::IsBreakpointValid(*Breakpoint) ? TEXT("Kismet.DebuggerOverlay.Breakpoint.EnabledAndValid") : TEXT("Kismet.DebuggerOverlay.Breakpoint.EnabledAndInvalid"));
		}
		else
		{
			BreakpointOverlayInfo.Brush = GetStyleSet().GetBrush(TEXT("Kismet.DebuggerOverlay.Breakpoint.Disabled"));
		}

		if(BreakpointOverlayInfo.Brush != NULL)
		{
			BreakpointOverlayInfo.OverlayOffset -= BreakpointOverlayInfo.Brush->ImageSize/2.f;
		}

		Brushes.Add(BreakpointOverlayInfo);
	}

	// Is this the current instruction?
	if (FKismetDebugUtilities::GetCurrentInstruction() == GraphNode)
	{
		FOverlayBrushInfo IPOverlayInfo;

		// Pick icon depending on whether we are on a hit breakpoint
		const bool bIsOnHitBreakpoint = FKismetDebugUtilities::GetMostRecentBreakpointHit() == GraphNode;
		
		IPOverlayInfo.Brush = GetStyleSet().GetBrush( bIsOnHitBreakpoint ? TEXT("Kismet.DebuggerOverlay.InstructionPointerBreakpoint") : TEXT("Kismet.DebuggerOverlay.InstructionPointer") );

		if (IPOverlayInfo.Brush != NULL)
		{
			float Overlap = 10.f;
			IPOverlayInfo.OverlayOffset.X = (WidgetSize.X/2.f) - (IPOverlayInfo.Brush->ImageSize.X/2.f);
			IPOverlayInfo.OverlayOffset.Y = (Overlap - IPOverlayInfo.Brush->ImageSize.Y);
		}

		IPOverlayInfo.AnimationEnvelope = FVector2f(0.f, 10.f);

		Brushes.Add(IPOverlayInfo);
	}

	// @todo remove if Timeline nodes are rendered in their own slate widget
	if (const UK2Node_Timeline* Timeline = Cast<const UK2Node_Timeline>(GraphNode))
	{
		float Offset = 0.0f;
		if (Timeline && Timeline->bAutoPlay)
		{
			FOverlayBrushInfo IPOverlayInfo;
			IPOverlayInfo.Brush = GetStyleSet().GetBrush( TEXT("Graph.Node.Autoplay") );

			if (IPOverlayInfo.Brush != NULL)
			{
				const float Padding = 2.5f;
				IPOverlayInfo.OverlayOffset.X = WidgetSize.X - IPOverlayInfo.Brush->ImageSize.X - Padding;
				IPOverlayInfo.OverlayOffset.Y = Padding;
				Offset = IPOverlayInfo.Brush->ImageSize.X;
			}
			Brushes.Add(IPOverlayInfo);
		}
		if (Timeline && Timeline->bLoop)
		{
			FOverlayBrushInfo IPOverlayInfo;
			IPOverlayInfo.Brush = GetStyleSet().GetBrush( TEXT("Graph.Node.Loop") );

			if (IPOverlayInfo.Brush != NULL)
			{
				const float Padding = 2.5f;
				IPOverlayInfo.OverlayOffset.X = WidgetSize.X - IPOverlayInfo.Brush->ImageSize.X - Padding - Offset;
				IPOverlayInfo.OverlayOffset.Y = Padding;
			}
			Brushes.Add(IPOverlayInfo);
		}
	}

	// Display an  icon depending on the type of node and it's settings
	if (const UK2Node* K2Node = Cast<const UK2Node>(GraphNode))
	{
		FName ClientIcon = K2Node->GetCornerIcon();
		if ( ClientIcon != NAME_None )
		{
			FOverlayBrushInfo IPOverlayInfo;

			IPOverlayInfo.Brush = GetStyleSet().GetBrush( ClientIcon );

			if (IPOverlayInfo.Brush != NULL)
			{
				IPOverlayInfo.OverlayOffset.X = (WidgetSize.X - (IPOverlayInfo.Brush->ImageSize.X/2.f))-3.f;
				IPOverlayInfo.OverlayOffset.Y = (IPOverlayInfo.Brush->ImageSize.Y/-2.f)+2.f;
			}

			Brushes.Add(IPOverlayInfo);
		}
	}
}

void SGraphNodeK2Base::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
	FKismetNodeInfoContext* K2Context = (FKismetNodeInfoContext*)Context;

	// Display any pending latent actions
	if (UObject* ActiveObject = K2Context->ActiveObjectBeingDebugged)
	{
		TArray<FKismetNodeInfoContext::FObjectUUIDPair>* Pairs = K2Context->NodesWithActiveLatentActions.Find(GraphNode);
		if (Pairs != NULL)
		{
			for (int32 Index = 0; Index < Pairs->Num(); ++Index)
			{
				FKismetNodeInfoContext::FObjectUUIDPair Action = (*Pairs)[Index];

				if (Action.Object == ActiveObject)
				{
					if (UWorld* World = GEngine->GetWorldFromContextObject(Action.Object, EGetWorldErrorMode::ReturnNull))
					{
						FLatentActionManager& LatentActionManager = World->GetLatentActionManager();

						const FString LatentDesc = LatentActionManager.GetDescription(Action.Object, Action.UUID);
						const FString& ActorLabel = Action.GetDisplayName();

						new (Popups) FGraphInformationPopupInfo(NULL, LatentBubbleColor, LatentDesc);
					}
				}
			}
		}

		// Display pinned watches
		if (K2Context->WatchedNodeSet.Contains(GraphNode))
		{
			const UBlueprintEditorSettings* BlueprintEditorSettings = GetDefault<UBlueprintEditorSettings>();
			UBlueprint* Blueprint = K2Context->SourceBlueprint;
			const UEdGraphSchema* Schema = GraphNode->GetSchema();

			const FPerBlueprintSettings* FoundSettings = BlueprintEditorSettings->PerBlueprintSettings.Find(Blueprint->GetPathName());
			if (FoundSettings)
			{
				FString PinnedWatchText;
				int32 ValidWatchCount = 0;
				TMap<const UEdGraphPin*, TSharedPtr<FPropertyInstanceInfo>> CachedPinInfo;
				for (const FBlueprintWatchedPin& WatchedPin : FoundSettings->WatchedPins)
				{
					const UEdGraphPin* WatchPin = WatchedPin.Get();
					if (WatchPin && WatchPin->GetOwningNode() == GraphNode)
					{
						if (ValidWatchCount > 0)
						{
							PinnedWatchText += TEXT("\n");
						}
						TSharedPtr<FPropertyInstanceInfo> PinInfo;
						if (CachedPinInfo.Find(WatchPin))
						{
							PinInfo = CachedPinInfo[WatchPin];
						}
						else
						{
							const FKismetDebugUtilities::EWatchTextResult WatchStatus = FKismetDebugUtilities::GetDebugInfo(CachedPinInfo.Add(WatchPin), Blueprint, ActiveObject, WatchPin);

							if (WatchStatus == FKismetDebugUtilities::EWTR_Valid)
							{
								PinInfo = CachedPinInfo[WatchPin];
							}
							else
							{
								FString PinName = UEdGraphSchema_K2::TypeToText(WatchPin->PinType).ToString();
								PinName += TEXT(" ");
								PinName += Schema->GetPinDisplayName(WatchPin).ToString();

								switch (WatchStatus)
								{
								case FKismetDebugUtilities::EWTR_Valid:
									break;

								case FKismetDebugUtilities::EWTR_NotInScope:
									PinnedWatchText += FText::Format(LOCTEXT("WatchingWhenNotInScopeFmt", "Watching {0}\n\t(not in scope)"), FText::FromString(PinName)).ToString();
									break;

								case FKismetDebugUtilities::EWTR_NoProperty:
									PinnedWatchText += FText::Format(LOCTEXT("WatchingUnknownPropertyFmt", "Watching {0}\n\t(no debug data)"), FText::FromString(PinName)).ToString();
									break;

								default:
								case FKismetDebugUtilities::EWTR_NoDebugObject:
									PinnedWatchText += FText::Format(LOCTEXT("WatchingNoDebugObjectFmt", "Watching {0}"), FText::FromString(PinName)).ToString();
									break;
								}
							}
						}

						if (PinInfo.IsValid())
						{
							FString WatchName;
							FString WatchText;
							if (WatchedPin.GetPathToProperty().IsEmpty())
							{
								WatchName = UEdGraphSchema_K2::TypeToText(WatchPin->PinType).ToString();
								WatchName += TEXT(" ");
								WatchName += Schema->GetPinDisplayName(WatchPin).ToString();

								WatchText = PinInfo->GetWatchText();
							}
							else
							{
								TSharedPtr<FPropertyInstanceInfo> PropWatch = PinInfo->ResolvePathToProperty(WatchedPin.GetPathToProperty());
								if (PropWatch.IsValid())
								{
									WatchName = UEdGraphSchema_K2::TypeToText(PropWatch->Property.Get()).ToString();
									WatchName += TEXT(" ");

									WatchText = PropWatch->GetWatchText();
								}
								else
								{
									WatchText = LOCTEXT("NoDebugData", "(no debug data)").ToString();
								}
								
								WatchName += Schema->GetPinDisplayName(WatchPin).ToString();

								for (const FName& PathName : WatchedPin.GetPathToProperty())
								{
									if (!PathName.ToString().StartsWith("["))
									{
										WatchName += TEXT("/");
									}

									WatchName += PathName.ToString();
								}
							}

							PinnedWatchText += FText::Format(LOCTEXT("WatchingAndValidFmt", "Watching {0}\n\t{1}"), FText::FromString(WatchName), FText::FromString(WatchText)).ToString();
						}

						ValidWatchCount++;
					}
				}

				if (ValidWatchCount)
				{
					new (Popups) FGraphInformationPopupInfo(nullptr, PinnedWatchColor, PinnedWatchText);
				}
			}
		}
	}
}

const FSlateBrush* SGraphNodeK2Base::GetShadowBrush(bool bSelected) const
{
	const UK2Node* K2Node = CastChecked<UK2Node>(GraphNode);
	const bool bCompactMode = K2Node->ShouldDrawCompact();

	if (bSelected && bCompactMode)
	{
		return GetStyleSet().GetBrush( "Graph.VarNode.ShadowSelected" );
	}
	else
	{
		return SGraphNode::GetShadowBrush(bSelected);
	}
}

void SGraphNodeK2Base::GetDiffHighlightBrushes(const FSlateBrush*& BackgroundOut, const FSlateBrush*& ForegroundOut) const
{
	const UK2Node* K2Node = CastChecked<UK2Node>(GraphNode);
	
	if (K2Node->ShouldDrawCompact())
	{
		BackgroundOut = GetStyleSet().GetBrush(TEXT("Graph.VarNode.DiffHighlight"));
		ForegroundOut = GetStyleSet().GetBrush(TEXT("Graph.VarNode.DiffHighlightShading"));
	}
	else
	{
		BackgroundOut = GetStyleSet().GetBrush(TEXT("Graph.Node.DiffHighlight"));
		ForegroundOut = GetStyleSet().GetBrush(TEXT("Graph.Node.DiffHighlightShading"));
	}
}

void SGraphNodeK2Base::PerformSecondPassLayout(const TMap< UObject*, TSharedRef<SNode> >& NodeToWidgetLookup) const
{
	TSet<UEdGraphNode*> PrevNodes;
	TSet<UEdGraphNode*> NextNodes;

	// Gather predecessor/successor nodes
	for (int32 PinIndex = 0; PinIndex < GraphNode->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = GraphNode->Pins[PinIndex];

		if (Pin->LinkedTo.Num() > 0)
		{
			if (Pin->Direction == EGPD_Input)
			{
				for (int32 LinkIndex = 0; LinkIndex < Pin->LinkedTo.Num(); ++LinkIndex)
				{
					PrevNodes.Add(Pin->LinkedTo[LinkIndex]->GetOwningNode());
				}
			}
			
			if (Pin->Direction == EGPD_Output)
			{
				for (int32 LinkIndex = 0; LinkIndex < Pin->LinkedTo.Num(); ++LinkIndex)
				{
					NextNodes.Add(Pin->LinkedTo[LinkIndex]->GetOwningNode());
				}
			}
		}
	}

	// Place this node smack between them
	const float Height = 0.0f;
	PositionThisNodeBetweenOtherNodes(NodeToWidgetLookup, PrevNodes, NextNodes, Height);
}



#undef LOCTEXT_NAMESPACE
