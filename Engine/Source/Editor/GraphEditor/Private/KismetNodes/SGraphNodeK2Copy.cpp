// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeK2Copy.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SGraphPin.h"
// Pin stuff
#include "SLevelOfDetailBranchNode.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"

class SWidget;
class UEdGraphSchema;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SGraphNodeK2Copy"

class SCopyNodeGraphPin : public SGraphPin
{
public:

	SLATE_BEGIN_ARGS(SCopyNodeGraphPin)
		: _PinLabelStyle(NAME_DefaultPinLabelStyle)
		, _UsePinColorForText(false)
		, _SideToSideMargin(0.0f)
	{}
		SLATE_ARGUMENT(FName, PinLabelStyle)
		SLATE_ARGUMENT(bool, UsePinColorForText)
		SLATE_ARGUMENT(float, SideToSideMargin)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin)
	{
		bUsePinColorForText = InArgs._UsePinColorForText;
		this->SetCursor(EMouseCursor::Default);

		SetVisibility(MakeAttributeSP(this, &SCopyNodeGraphPin::GetPinVisiblity));

		GraphPinObj = InPin;
		check(GraphPinObj != NULL);

		const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
		checkf(
			Schema,
			TEXT("Missing schema for pin: %s with outer: %s of type %s"),
			*(GraphPinObj->GetName()),
			GraphPinObj->GetOuter() ? *(GraphPinObj->GetOuter()->GetName()) : TEXT("NULL OUTER"),
			GraphPinObj->GetOuter() ? *(GraphPinObj->GetOuter()->GetClass()->GetName()) : TEXT("NULL OUTER")
			);

		const bool bCanConnectToPin = !GraphPinObj->bNotConnectable;
		const bool bIsInput = (GetDirection() == EGPD_Input);
		
		// Create the pin icon widget
		TSharedRef<SWidget> ActualPinWidget =
			SAssignNew(PinImage, SImage)
			.Image(this, &SCopyNodeGraphPin::GetPinIcon)
			.IsEnabled(bCanConnectToPin)
			.ColorAndOpacity(this, &SCopyNodeGraphPin::GetPinColor)
			.OnMouseButtonDown(this, &SCopyNodeGraphPin::OnPinMouseDown)
			.Cursor(this, &SCopyNodeGraphPin::GetPinCursor);

		// Create the pin indicator widget (used for watched values)
		static const FName NAME_NoBorder("NoBorder");
		TSharedRef<SWidget> PinStatusIndicator =
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), NAME_NoBorder)
			.Visibility(this, &SCopyNodeGraphPin::GetPinStatusIconVisibility)
			.ContentPadding(0.0f)
			.OnClicked(this, &SCopyNodeGraphPin::ClickedOnPinStatusIcon)
			[
				SNew(SImage)
				.Image(this, &SCopyNodeGraphPin::GetPinStatusIcon)
			];

		// Create the widget used for the pin body (status indicator, label, and value)
		TSharedRef<SWrapBox> LocalLabelAndValue =
			SNew(SWrapBox)
			.PreferredSize(150.f);

		LocalLabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				PinStatusIndicator
			];

		TSharedPtr<SWidget> PinContent;
		if (bIsInput)
		{
			// Input pin
			PinContent = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, InArgs._SideToSideMargin, 0.0f)
				[
					ActualPinWidget
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					LocalLabelAndValue
				];
		}
		else
		{
			// Output pin
			PinContent = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					LocalLabelAndValue
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(InArgs._SideToSideMargin, 0.0f, 0.0f, 0.0f)
				[
					ActualPinWidget
				];
		}

		// Set up a hover for pins that is tinted the color of the pin.
		SBorder::Construct(SBorder::FArguments()
			.BorderImage(this, &SCopyNodeGraphPin::GetPinBorder)
			.BorderBackgroundColor(this, &SCopyNodeGraphPin::GetPinColor)
			.OnMouseButtonDown(this, &SCopyNodeGraphPin::OnPinNameMouseDown)
			[
				SNew(SLevelOfDetailBranchNode)
					.UseLowDetailSlot(this, &SCopyNodeGraphPin::UseLowDetailPinNames)
					.LowDetail()
					[
						//@TODO: Try creating a pin-colored line replacement that doesn't measure text / call delegates but still renders
						ActualPinWidget
					]
					.HighDetail()
					[
						PinContent.ToSharedRef()
					]
			]
		);

		TAttribute<FText> ToolTipAttribute = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SCopyNodeGraphPin::GetTooltipText));
		SetToolTipText(ToolTipAttribute);
	}

	virtual const FSlateBrush* GetPinIcon() const
	{
		if (GetDirection() == EGPD_Input)
		{
			if (IsConnected())
			{
				return FAppStyle::GetBrush(TEXT("Graph.Pin.CopyNodePinLeft_Connected"));
			}
			else
			{
				return FAppStyle::GetBrush(TEXT("Graph.Pin.CopyNodePinLeft_Disconnected"));
			}
		}
		else
		{
			if (IsConnected())
			{
				return FAppStyle::GetBrush(TEXT("Graph.Pin.CopyNodePinRight_Connected"));
			}
			else
			{
				return FAppStyle::GetBrush(TEXT("Graph.Pin.CopyNodePinRight_Disconnected"));
			}
		}
	}

protected:
	FSlateBrush* CachedImg_Left_Pin;
	FSlateBrush* CachedImg_Right_Pin;
};

//////////////////////////////////////////////////////////////////////////
// SGraphNodeK2Copy

void SGraphNodeK2Copy::Construct(const FArguments& InArgs, UK2Node* InNode)
{
	GraphNode = InNode;

	SetCursor(EMouseCursor::CardinalCross);

	UpdateGraphNode();
}

void SGraphNodeK2Copy::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	SetupErrorReporting();

	GetOrAddSlot(ENodeZone::Center)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Top)
			[
				CreateNodeContentArea()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(Settings->GetNonPinNodeBodyPadding())
			[
				ErrorReporting->AsWidget()
			]
		];
	CreatePinWidgets();
}

TSharedRef<SWidget> SGraphNodeK2Copy::CreateNodeContentArea()
{
	// NODE CONTENT AREA
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(5.0f, 5.0f, 0.0f, 5.0f)
		[
			// LEFT
			SAssignNew(LeftNodeBox, SVerticalBox)
		]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(0.0f, 5.0f, 5.0f, 5.0f)
		[
			// RIGHT
			SAssignNew(RightNodeBox, SVerticalBox)
		];
}

void SGraphNodeK2Copy::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = (PinObj != nullptr) && PinObj->bAdvancedView;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility(TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced));
	}

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				PinToAdd
			];
		InputPins.Add(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		RightNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				PinToAdd
			];
		OutputPins.Add(PinToAdd);
	}
}

TSharedPtr<SGraphPin> SGraphNodeK2Copy::CreatePinWidget(UEdGraphPin* Pin) const
{
	return SNew(SCopyNodeGraphPin, Pin);
}

const FSlateBrush* SGraphNodeK2Copy::GetShadowBrush(bool bSelected) const
{
	return bSelected ? FAppStyle::GetBrush(TEXT("Graph.Node.ShadowSelected")) : FAppStyle::GetNoBrush();
}

#undef LOCTEXT_NAMESPACE


