// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetNodes/SGraphNodeK2Base.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class SGraphPin;
class SWidget;
class UEdGraphPin;
class UK2Node;
struct FSlateBrush;

class SGraphNodeK2Copy : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2Copy) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UK2Node* InNode);

	//~ Begin SGraphNode Interface
	UE_API virtual void UpdateGraphNode() override;
	UE_API virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	UE_API virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	UE_API virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	UE_API virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	//~ End SGraphNode Interface
};

#undef UE_API
