// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeK2CreateDelegate.h"

#include "BlueprintEventNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CustomEvent.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "KismetCompilerMisc.h"
#include "K2Node.h"
#include "K2Node_CreateDelegate.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

namespace UE::SGraphNodeK2::Private
{
	bool MatchEventGraphPredicate(const TObjectPtr<UEdGraph>& Entry)
	{
		check(Entry);
		return Entry->GetFName() == UEdGraphSchema_K2::GN_EventGraph;
	};
}

FText SGraphNodeK2CreateDelegate::FunctionDescription(const UFunction* Function, const bool bOnlyDescribeSignature /*= false*/, const int32 CharacterLimit /*= 32*/)
{
	if (!Function || !Function->GetOuter())
	{
		return NSLOCTEXT("GraphNodeK2Create", "Error", "Error");
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	FString Result;
	
	// Show function name.
	if (!bOnlyDescribeSignature)
	{
		Result = Function->GetName();
	}

	Result += TEXT("(");

	// Describe input parameters.
	{
		bool bFirst = true;
		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* const Param = *PropIt;
			const bool bIsFunctionInput = Param && (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm));
			if (bIsFunctionInput)
			{
				if (!bFirst)
				{
					Result += TEXT(", ");
				}
				if (CharacterLimit > INDEX_NONE && Result.Len() > CharacterLimit)
				{
					Result += TEXT("...");
					break;
				}
				Result += bOnlyDescribeSignature ? UEdGraphSchema_K2::TypeToText(Param).ToString() : Param->GetName();
				bFirst = false;
			}
		}
	}

	Result += TEXT(")");

	// Describe outputs.
	{
		TArray<FString> Outputs;

		FProperty* const FunctionReturnProperty = Function->GetReturnProperty();
		if (FunctionReturnProperty)
		{
			Outputs.Add(UEdGraphSchema_K2::TypeToText(FunctionReturnProperty).ToString());
		}

		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* const Param = *PropIt;
			const bool bIsFunctionOutput = Param && Param->HasAnyPropertyFlags(CPF_OutParm);
			if (bIsFunctionOutput)
			{
				Outputs.Add(bOnlyDescribeSignature ? UEdGraphSchema_K2::TypeToText(Param).ToString() : Param->GetName());
			}
		}

		if (Outputs.Num() > 0)
		{
			Result += TEXT(" -> ");
		}

		if (Outputs.Num() > 1)
		{
			Result += TEXT("[");
		}

		bool bFirst = true;
		for (const FString& Output : Outputs)
		{
			if (!bFirst)
			{
				Result += TEXT(", ");
			}
			if (CharacterLimit > INDEX_NONE && Result.Len() > CharacterLimit)
			{
				Result += TEXT("...");
				break;
			}
			Result += Output;
			bFirst = false;
		}

		if (Outputs.Num() > 1)
		{
			Result += TEXT("]");
		}
	}

	return FText::FromString(Result);
}

void SGraphNodeK2CreateDelegate::Construct(const FArguments& InArgs, UK2Node* InNode)
{
	GraphNode = InNode;
	UpdateGraphNode();
}

FText SGraphNodeK2CreateDelegate::GetCurrentFunctionDescription() const
{
	UK2Node_CreateDelegate* Node = Cast<UK2Node_CreateDelegate>(GraphNode);
	UFunction* FunctionSignature = Node ? Node->GetDelegateSignature() : nullptr;
	UClass* ScopeClass = Node ? Node->GetScopeClass() : nullptr;

	if (!FunctionSignature || !ScopeClass)
	{
		return FText::GetEmpty();
	}

	if (const UFunction* Func = FindUField<UFunction>(ScopeClass, Node->GetFunctionName()))
	{
		return FunctionDescription(Func);
	}

	if (Node->GetFunctionName() != NAME_None)
	{
		return FText::Format(NSLOCTEXT("GraphNodeK2Create", "ErrorLabelFmt", "Error? {0}"), FText::FromName(Node->GetFunctionName()));
	}

	return NSLOCTEXT("GraphNodeK2Create", "SelectFunctionLabel", "Select Function...");
}

void SGraphNodeK2CreateDelegate::OnFunctionSelected(TSharedPtr<FString> FunctionItemData, ESelectInfo::Type SelectInfo)
{
	const FScopedTransaction Transaction(NSLOCTEXT("GraphNodeK2Create", "CreateMatchingSignature", "Create matching signature"));

	if (FunctionItemData.IsValid())
	{
		if (UK2Node_CreateDelegate* Node = Cast<UK2Node_CreateDelegate>(GraphNode))
		{
			UBlueprint* NodeBP = Node->GetBlueprint();
			UEdGraph* const SourceGraph = Node->GetGraph();
			check(NodeBP && SourceGraph);
			SourceGraph->Modify();
			NodeBP->Modify();
			Node->Modify();

			if (FunctionItemData == CreateMatchingFunctionData)
			{
				// Get a valid name for the function graph
				FString ProposedFuncName = NodeBP->GetName() + "_AutoGenFunc";
				FName NewFuncName = FBlueprintEditorUtils::GenerateUniqueGraphName(NodeBP, ProposedFuncName);
				
				UEdGraph* NewGraph = nullptr;
				NewGraph = FBlueprintEditorUtils::CreateNewGraph(NodeBP, NewFuncName, SourceGraph->GetClass(), SourceGraph->GetSchema() ? SourceGraph->GetSchema()->GetClass() : GetDefault<UEdGraphSchema_K2>()->GetClass());

				if (NewGraph != nullptr)
				{
					FBlueprintEditorUtils::AddFunctionGraph<UFunction>(NodeBP, NewGraph, true, Node->GetDelegateSignature());
					FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewGraph);
				}
				
				Node->SetFunction(NewFuncName);
			}
			else if (FunctionItemData == CreateMatchingEventData)
			{
				TObjectPtr<UEdGraph>* EventGraphPtr = NodeBP->UbergraphPages.FindByPredicate(UE::SGraphNodeK2::Private::MatchEventGraphPredicate);
				check(EventGraphPtr);
				UEdGraph* EventGraph = *EventGraphPtr;
				check(EventGraph);

				FName NewEventName = FBlueprintEditorUtils::FindUniqueCustomEventName(NodeBP);
				UBlueprintEventNodeSpawner* Spawner = UBlueprintEventNodeSpawner::Create(UK2Node_CustomEvent::StaticClass(), NewEventName);
				FVector2f SpawnPos = EventGraph->GetGoodPlaceForNewNode();

				UEdGraphNode* NewNode = Spawner->Invoke(EventGraph, IBlueprintNodeBinder::FBindingSet(), FDeprecateSlateVector2D(SpawnPos));

				UK2Node_CustomEvent* NewEventNode = CastChecked<UK2Node_CustomEvent>(NewNode);
				NewEventNode->SetDelegateSignature(Node->GetDelegateSignature());
				// Reconstruct to get the new parameters to show in the editor
				NewEventNode->ReconstructNode();
				NewEventNode->bIsEditable = true;
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewEventNode);

				Node->SetFunction(NewEventName);
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(NodeBP);
			}
			else
			{
				FName FuncName( **FunctionItemData.Get());
				Node->SetFunction(FuncName);
			}
			
			Node->HandleAnyChange(true);

			TSharedPtr<SSearchableComboBox> SelectFunctionWidgetPtr = FunctionOptionComboBox.Pin();
			if (SelectFunctionWidgetPtr.IsValid())
			{
				SelectFunctionWidgetPtr->SetIsOpen(false);
			}
		}
	}
}

TSharedPtr<FString> SGraphNodeK2CreateDelegate::AddDefaultFunctionDataOption(const FText& DisplayName)
{
	TSharedPtr<FString> Res = MakeShareable(new FString(DisplayName.ToString()));
	FunctionOptionList.Add(Res);
	return Res;
}

void SGraphNodeK2CreateDelegate::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	if (UK2Node_CreateDelegate* Node = Cast<UK2Node_CreateDelegate>(GraphNode))
	{
		const UBlueprint* NodeBP = Node->GetBlueprint();
		check(NodeBP);

		UFunction* FunctionSignature = Node->GetDelegateSignature();
		UClass* ScopeClass = Node->GetScopeClass();

		if (FunctionSignature && ScopeClass)
		{
			FText FunctionSignaturePrompt;
			{
				FFormatNamedArguments FormatArguments;
				FormatArguments.Add(TEXT("FunctionSignature"), FunctionDescription(FunctionSignature, true));
				FunctionSignaturePrompt = FText::Format(NSLOCTEXT("GraphNodeK2Create", "FunctionSignaturePrompt", "Signature: {FunctionSignature}"), FormatArguments);
			}

			FText FunctionSignatureToolTipText;
			{
				FFormatNamedArguments FormatArguments;
				FormatArguments.Add(TEXT("FullFunctionSignature"), FunctionDescription(FunctionSignature, true, INDEX_NONE));
				FunctionSignatureToolTipText = FText::Format(NSLOCTEXT("GraphNodeK2Create", "FunctionSignatureToolTip", "Signature Syntax: (Inputs) -> [Outputs]\nFull Signature:{FullFunctionSignature}"), FormatArguments);
			}

			MainBox->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Fill)
				.Padding(4.0f)
				[
					SNew(STextBlock)
					.Text(FunctionSignaturePrompt)
					.ToolTipText(FunctionSignatureToolTipText)
				];

			FunctionOptionList.Empty();

			// add an empty row, so the user can clear the selection if they want
			AddDefaultFunctionDataOption(NSLOCTEXT("GraphNodeK2Create", "EmptyFunctionOption", "[None]"));

			// Option to create a function based on the event parameters
			CreateMatchingFunctionData = AddDefaultFunctionDataOption(NSLOCTEXT("GraphNodeK2Create", "CreateMatchingFunctionOption", "[Create a matching function]"));

			// Only signatures with no output parameters can be events
			// There must also be an event graph to add the event to
			const bool bCanCreateEvent =
				!UEdGraphSchema_K2::HasFunctionAnyOutputParameter(FunctionSignature) &&
				NodeBP->UbergraphPages.ContainsByPredicate(UE::SGraphNodeK2::Private::MatchEventGraphPredicate)
			;

			if (bCanCreateEvent)
			{
				CreateMatchingEventData = AddDefaultFunctionDataOption(NSLOCTEXT("GraphNodeK2Create", "CreateMatchingEventOption", "[Create a matching event]"));
			}

			struct FFunctionItemData
			{
				FName Name;
				FText Description;
			};

			TArray<FFunctionItemData> ClassFunctions;

			for (TFieldIterator<UFunction> It(ScopeClass); It; ++It)
			{
				UFunction* Func = *It;
				if (Func && 
					(FKismetCompilerUtilities::DoSignaturesHaveConvertibleFloatTypes(Func, FunctionSignature) != ConvertibleSignatureMatchResult::Different) &&
					UEdGraphSchema_K2::FunctionCanBeUsedInDelegate(Func))
				{
					FFunctionItemData ItemData;
					ItemData.Name = Func->GetFName();
					ItemData.Description = FunctionDescription(Func);
					ClassFunctions.Emplace(MoveTemp(ItemData));
				}
			}

			ClassFunctions.Sort([](const FFunctionItemData& A, const FFunctionItemData& B) {
				return A.Description.CompareTo(B.Description) < 0;
			});

			for (const FFunctionItemData& ItemData : ClassFunctions)
			{
				// Add this to the searchable text box as an FString so users can type and find it
				FunctionOptionList.Add(MakeShareable(new FString(ItemData.Name.ToString())));
			}

			TSharedRef<SSearchableComboBox> SelectFunctionWidgetRef =
				SNew(SSearchableComboBox)
				.OptionsSource(&FunctionOptionList)
				.OnGenerateWidget(this, &SGraphNodeK2CreateDelegate::MakeFunctionOptionComboWidget)
				.OnSelectionChanged(this, &SGraphNodeK2CreateDelegate::OnFunctionSelected)
				.ContentPadding(2.0f)
				.MaxListHeight(200.0f)
				.Content()
				[
					SNew(STextBlock)
					.Text(GetCurrentFunctionDescription())
				];
			
			MainBox->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Fill)
				.Padding(4.0f)
				[
					SelectFunctionWidgetRef
				];

			FunctionOptionComboBox = SelectFunctionWidgetRef;
		}
	}
}

TSharedRef<SWidget> SGraphNodeK2CreateDelegate::MakeFunctionOptionComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock).Text(FText::FromString(*InItem));
}

SGraphNodeK2CreateDelegate::~SGraphNodeK2CreateDelegate()
{
	TSharedPtr<SSearchableComboBox> SelectFunctionWidgetPtr = FunctionOptionComboBox.Pin();
	if (SelectFunctionWidgetPtr.IsValid())
	{
		SelectFunctionWidgetPtr->SetIsOpen(false);
	}
}
