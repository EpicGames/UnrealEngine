// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FString;
class SSearchableComboBox;
class SVerticalBox;
class SWidget;
class UFunction;
class UK2Node;

/**
* @brief	The CreateDelegate node will allow users to bind to event dispatchers 
*			based off of appropriate function signatures, or create a matching one.
*/
class SGraphNodeK2CreateDelegate : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2CreateDelegate) {}
	SLATE_END_ARGS()

	/** Data that can be used to create a matching function based on the parameters of a create event node */
	TSharedPtr<FString> CreateMatchingFunctionData;
	
	/** Data that can be used to create a matching event based on based on the parameters of a create event node */
	TSharedPtr<FString> CreateMatchingEventData;

public:
	virtual ~SGraphNodeK2CreateDelegate();
	void Construct(const FArguments& InArgs, UK2Node* InNode);
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;

protected:
	static FText FunctionDescription(const UFunction* Function, const bool bOnlyDescribeSignature = false, const int32 CharacterLimit = 32);

	FText GetCurrentFunctionDescription() const;

private:

	//~ Begin Searchable combo box options
	TWeakPtr<SSearchableComboBox>	FunctionOptionComboBox;
	TArray< TSharedPtr< FString > >	FunctionOptionList;

	TSharedRef<SWidget> MakeFunctionOptionComboWidget(TSharedPtr<FString> InItem);

	/** Callback for when the function selection has changed from the dropdown */
	void OnFunctionSelected(TSharedPtr<FString> FunctionItemData, ESelectInfo::Type SelectInfo);
	//~ End Searchable combo box options

	/**
	* Adds a FunctionItemData with a given description to the array of FunctionDataItems. 
	* 
	* @param	DescriptionName		Description of the option to give the user
	* @return	Shared pointer to the FunctionItemData
	*/
	TSharedPtr<FString> AddDefaultFunctionDataOption(const FText& DescriptionName);
};