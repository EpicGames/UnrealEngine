// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetNodes/SGraphNodeK2Default.h"

#include "GenericPlatform/ICursor.h"
#include "K2Node.h"
#include "Misc/Optional.h"


void SGraphNodeK2Default::Construct( const FArguments& InArgs, UK2Node* InNode )
{
	this->GraphNode = InNode;

	this->SetCursor( EMouseCursor::CardinalCross );

	this->UpdateGraphNode();
}
