// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeK2Event.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/PlatformCrt.h"
#include "K2Node_Event.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "SGraphNode.h"
#include "SGraphPin.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"

void SGraphNodeK2Event::AddPin( const TSharedRef<SGraphPin>& PinToAdd ) 
{
	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bDelegateOutput = (PinObj != nullptr) && (UK2Node_Event::DelegateOutputName == PinObj->PinName);

	if (bDelegateOutput && TitleAreaWidget.IsValid())
	{
		PinToAdd->SetOwner(SharedThis(this));

		bHasDelegateOutputPin = true;
		PinToAdd->SetShowLabel(false);
		TitleAreaWidget->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
		[
			PinToAdd
		];

		OutputPins.Add(PinToAdd);
	}
	else
	{
		SGraphNodeK2Default::AddPin(PinToAdd);
	}
}

bool SGraphNodeK2Event::UseLowDetailNodeTitles() const
{
	return (!bHasDelegateOutputPin) && ParentUseLowDetailNodeTitles();
}

EVisibility SGraphNodeK2Event::GetTitleVisibility() const
{
	return ParentUseLowDetailNodeTitles() ? EVisibility::Hidden : EVisibility::Visible;
}

TSharedRef<SWidget> SGraphNodeK2Event::CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle)
{
	TSharedRef<SWidget> WidgetRef = SGraphNodeK2Default::CreateTitleWidget(NodeTitle);
	WidgetRef->SetVisibility(MakeAttributeSP(this, &SGraphNodeK2Event::GetTitleVisibility));
	if (NodeTitle.IsValid())
	{
		NodeTitle->SetVisibility(MakeAttributeSP(this, &SGraphNodeK2Event::GetTitleVisibility));
	}

	return WidgetRef;
}
