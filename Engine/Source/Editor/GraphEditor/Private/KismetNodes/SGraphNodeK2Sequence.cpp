// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeK2Sequence.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_AddPinInterface.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "Widgets/SBoxPanel.h"

class SWidget;


void SGraphNodeK2Sequence::Construct( const FArguments& InArgs, UK2Node* InNode )
{
	ensure(InNode == nullptr || InNode->GetClass()->ImplementsInterface(UK2Node_AddPinInterface::StaticClass()));
	GraphNode = InNode;

	SetCursor( EMouseCursor::CardinalCross );

	UpdateGraphNode();
}

void SGraphNodeK2Sequence::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
		NSLOCTEXT("SequencerNode", "SequencerNodeAddPinButton", "Add pin"),
		NSLOCTEXT("SequencerNode", "SequencerNodeAddPinButton_ToolTip", "Add new pin"));

	FMargin AddPinPadding = Settings->GetOutputPinPadding();
	AddPinPadding.Top += 6.0f;

	OutputBox->AddSlot()
	.AutoHeight()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Right)
	.Padding(AddPinPadding)
	[
		AddPinButton
	];
}

FReply SGraphNodeK2Sequence::OnAddPin()
{
	IK2Node_AddPinInterface* AddPinNode = Cast<IK2Node_AddPinInterface>(GraphNode);
	ensure(AddPinNode);
	if (AddPinNode && AddPinNode->CanAddPin())
	{
		FScopedTransaction Transaction(NSLOCTEXT("SequencerNode", "AddPinTransaction", "Add Pin"));

		AddPinNode->AddInputPin();
		UpdateGraphNode();
		GraphNode->GetGraph()->NotifyNodeChanged(GraphNode);
	}
	
	return FReply::Handled();
}

EVisibility SGraphNodeK2Sequence::IsAddPinButtonVisible() const
{
	IK2Node_AddPinInterface* AddPinNode = Cast<IK2Node_AddPinInterface>(GraphNode);
	ensure(AddPinNode);
	return ((AddPinNode && AddPinNode->CanAddPin()) ? EVisibility::Visible : EVisibility::Collapsed);
}