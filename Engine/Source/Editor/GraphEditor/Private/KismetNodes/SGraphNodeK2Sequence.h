// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SVerticalBox;
class UK2Node;

class SGraphNodeK2Sequence : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2Sequence){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, UK2Node* InNode );

protected:
	// SGraphNode interface
	virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
	virtual FReply OnAddPin() override;
	virtual EVisibility IsAddPinButtonVisible() const override;
	// End of SGraphNode interface
};
