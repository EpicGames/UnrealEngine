// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetNodes/SGraphNodeK2Terminator.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "K2Node.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SGraphNode.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

struct FSlateBrush;


void SGraphNodeK2Terminator::Construct( const FArguments& InArgs, UK2Node* InNode )
{
	this->GraphNode = InNode;

	this->SetCursor( EMouseCursor::CardinalCross );

	this->UpdateGraphNode();
}

void SGraphNodeK2Terminator::UpdateGraphNode()
{
	check(GraphNode != NULL);

	InputPins.Empty();
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();
	
	UK2Node* K2Node = CastChecked<UK2Node>(GraphNode);
	const FSlateBrush* TopBrush = NULL;
	const FSlateBrush* BottomBrush = NULL;

	if(K2Node->DrawNodeAsEntry())
	{
		TopBrush = FAppStyle::GetBrush(TEXT("Graph.Node.NodeEntryTop"));
		BottomBrush = FAppStyle::GetBrush(TEXT("Graph.Node.NodeEntryBottom"));
	}
	else
	{
		TopBrush = FAppStyle::GetBrush(TEXT("Graph.Node.NodeExitTop"));
		BottomBrush = FAppStyle::GetBrush(TEXT("Graph.Node.NodeExitBottom"));
	}


	//
	//             ______________________
	//            |      TITLE AREA      |
	//            +-------+------+-------+
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |      | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	this->GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SImage)
			.ColorAndOpacity( this, &SGraphNode::GetNodeTitleColor )
			.Image(TopBrush)
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SBorder)
			. Padding( 3 )
			. BorderImage( FAppStyle::GetBrush(TEXT("WhiteTexture")) )
			. HAlign(HAlign_Center)
			. BorderBackgroundColor( this, &SGraphNode::GetNodeTitleColor )
			[
				SNew(SNodeTitle, GraphNode)
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			// NODE CONTENT AREA
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush(TEXT("Graph.Node.NodeBackground")) )
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding( FMargin(0,3) )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					// LEFT
					SAssignNew(LeftNodeBox, SVerticalBox)
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Fill)
				.FillWidth(1.0f)
				[
					// MIDDLE
					SNew(SImage)
					.Image( FAppStyle::GetBrush(TEXT("WhiteTexture")) )
					.ColorAndOpacity(FLinearColor(1.f,1.f,1.f,0.f))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					// RIGHT
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SImage)
			.ColorAndOpacity( this, &SGraphNode::GetNodeTitleColor )
			.Image(BottomBrush)
		]
	];

	CreatePinWidgets();
}

const FSlateBrush* SGraphNodeK2Terminator::GetShadowBrush(bool bSelected) const
{
	UK2Node* K2Node = CastChecked<UK2Node>(GraphNode);

	if(K2Node->DrawNodeAsEntry())
	{
		return bSelected ? FAppStyle::GetBrush(TEXT("Graph.Node.NodeEntryShadowSelected")) : FAppStyle::GetBrush(TEXT("Graph.Node.NodeEntryShadow"));
	}
	else
	{
		return bSelected ? FAppStyle::GetBrush(TEXT("Graph.Node.NodeExitShadowSelected")) : FAppStyle::GetBrush(TEXT("Graph.Node.NodeExitShadow"));
	}
}
