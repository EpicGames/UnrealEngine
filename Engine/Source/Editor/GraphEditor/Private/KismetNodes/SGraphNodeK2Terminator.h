// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetNodes/SGraphNodeK2Base.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UK2Node;
struct FSlateBrush;

class SGraphNodeK2Terminator : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2Terminator){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, UK2Node* InNode );

	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;

protected:
	virtual void UpdateGraphNode() override;
};
