// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeK2Timeline.h"

#include "Components/TimelineComponent.h"
#include "Containers/UnrealString.h"
#include "Editor/EditorEngine.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_Timeline.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "KismetNodes/KismetNodeInfoContext.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "SNodePanel.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "SGraphNodeK2Base"

void SGraphNodeK2Timeline::Construct(const FArguments& InArgs, UK2Node_Timeline* InNode)
{
	SGraphNodeK2Default::Construct(SGraphNodeK2Default::FArguments(), InNode);
}

void SGraphNodeK2Timeline::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
 	FKismetNodeInfoContext* K2Context = (FKismetNodeInfoContext*)Context;
 
 	// Display the timeline status bubble
 	if (UObject* ActiveObject = K2Context->ActiveObjectBeingDebugged)
 	{
 		FProperty* NodeProperty = FKismetDebugUtilities::FindClassPropertyForNode(K2Context->SourceBlueprint, GraphNode);
 		if (FObjectProperty* TimelineProperty = CastField<FObjectProperty>(NodeProperty))
 		{
			UClass* ContainingClass = TimelineProperty->GetTypedOwner<UClass>();
			if (!ActiveObject->IsA(ContainingClass))
			{
				const FString ErrorText = FText::Format(LOCTEXT("StaleDebugDataFmt", "Stale debug data\nProperty is on {0}\nDebugging a {1}"), FText::FromString(ContainingClass->GetName()), FText::FromString(ActiveObject->GetClass()->GetName())).ToString();
				new (Popups) FGraphInformationPopupInfo(NULL, TimelineBubbleColor, ErrorText);
			}
			else if (UTimelineComponent* Timeline = Cast<UTimelineComponent>(TimelineProperty->GetObjectPropertyValue(TimelineProperty->ContainerPtrToValuePtr<void>(ActiveObject))))
 			{
 				// Current state
 				const FString State = Timeline->IsPlaying() ? LOCTEXT("Playing", "Playing").ToString() : LOCTEXT("Paused", "Paused").ToString();
 
 				// Play direction, only shown if playing
 				FString Direction;
 				if (Timeline->IsReversing())
 				{
 					Direction = LOCTEXT("InReverse", " (in reverse)").ToString();
 				}
 
 				// Position
				const float Percentage = Timeline->GetTimelineLength() > 0.0f ? Timeline->GetPlaybackPosition() / Timeline->GetTimelineLength() * 100.0f : 0.0f;
 				const FString Position = FString::Printf(TEXT(" @ %.2f s (%.1f %%)"), Timeline->GetPlaybackPosition(), Percentage);
 
 				// Looping status, only shown if playing
 				FString Looping;
 				if (Timeline->IsPlaying() && Timeline->IsLooping())
 				{
 					Looping = LOCTEXT("Looping", " (looping)").ToString();
 				}
 
 				// Putting it all together
 				FString TimelineText = FString::Printf(TEXT("%s\n%s%s%s%s"), *(UEditorEngine::GetFriendlyName(TimelineProperty)), *State, *Direction, *Position, *Looping);
 
 				new (Popups) FGraphInformationPopupInfo(NULL, TimelineBubbleColor, TimelineText);
 			}
 		}
 	}

	SGraphNodeK2Default::GetNodeInfoPopups(Context, Popups);
}

#undef LOCTEXT_NAMESPACE
