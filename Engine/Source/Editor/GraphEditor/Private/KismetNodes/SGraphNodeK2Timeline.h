// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "KismetNodes/SGraphNodeK2Default.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UK2Node_Timeline;
struct FGraphInformationPopupInfo;
struct FNodeInfoContext;

class SGraphNodeK2Timeline : public SGraphNodeK2Default
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2Timeline){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UK2Node_Timeline* InNode);

	// SNodePanel::SNode interface
	void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	// End of SNodePanel::SNode interface
};
