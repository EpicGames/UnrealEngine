// Copyright Epic Games, Inc. All Rights Reserved.



#include "KismetNodes/SGraphNodeK2Var.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_Literal.h"
#include "K2Node_MakeStruct.h"
#include "K2Node_StructMemberGet.h"
#include "K2Node_StructMemberSet.h"
#include "K2Node_StructOperation.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "SCommentBubble.h"
#include "SGraphNode.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Casts.h"
#include "TutorialMetaData.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FSlateBrush;

void SGraphNodeK2Var::Construct( const FArguments& InArgs, UK2Node* InNode )
{
	this->GraphNode = InNode;

	this->SetCursor( EMouseCursor::CardinalCross );

	this->UpdateGraphNode();
}

FSlateColor SGraphNodeK2Var::GetVariableColor() const
{
	return GraphNode->GetNodeTitleColor();
}

TSharedRef<SWidget> SGraphNodeK2Var::UpdateTitleWidget(FText InTitleText, TSharedPtr<SWidget> InTitleWidget, EHorizontalAlignment& InOutTitleHAlign, FMargin& InOutTitleMargin) const
{
	TSharedPtr<SWidget> TitleWidget = InTitleWidget;
	
	if (InTitleText.IsEmpty())
	{
		TitleWidget = SNullWidget::NullWidget;
	}

	if(!TitleWidget.IsValid())
	{
		TitleWidget = SNew(STextBlock)
		.TextStyle( FAppStyle::Get(), "Graph.Node.NodeTitle" )
		.Text(InTitleText);
	}

	return TitleWidget.ToSharedRef();
}

void SGraphNodeK2Var::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	FText TitleText;
	bool bPadTitle = false;
	float HorizontalTitleMargin = 0.0f;
	float VerticalTitleMargin = 8.0f;
	FMargin TitleMargin = FMargin(0.0f, 8.0f);
	FMargin ContentAreaMargin = FMargin(0.0f, 4.0f);
	EHorizontalAlignment TitleHAlign = HAlign_Center;
	TSharedPtr<SWidget> TitleWidget;

	if (GraphNode->IsA(UK2Node_VariableSet::StaticClass()))
	{
		UK2Node_VariableSet *SetNode = Cast<UK2Node_VariableSet>(GraphNode);
		const bool bHasLocalRepNotify = SetNode->HasLocalRepNotify();
		const bool bIsFieldNotify = SetNode->IsFieldNotifyProperty();
		if (bHasLocalRepNotify && bIsFieldNotify)
		{
			TitleText = NSLOCTEXT("GraphEditor", "VariableSetWithNotifyWithBroadcast", "SET w/ Notify&Broadcast");
		}
		else if (bHasLocalRepNotify)
		{
			TitleText = NSLOCTEXT("GraphEditor", "VariableSetWithNotify", "SET w/ Notify");
		}
		else if (bIsFieldNotify)
		{
			TitleText = NSLOCTEXT("GraphEditor", "VariableSetWithBroadcast", "SET w/ Broadcast");
		}
		else
		{
			TitleText = NSLOCTEXT("GraphEditor", "VariableSet", "SET");
		}
	}
	else if (UK2Node_StructOperation* StructOp = Cast<UK2Node_StructOperation>(GraphNode))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableName"), StructOp->GetVarNameText());
		if (GraphNode->IsA(UK2Node_StructMemberGet::StaticClass()))
		{
			TitleText = FText::Format(NSLOCTEXT("GraphEditor", "StructMemberGet", "Get in {VariableName}"), Args);
		}
		else if (GraphNode->IsA(UK2Node_StructMemberSet::StaticClass()))
		{
			TitleText = FText::Format(NSLOCTEXT("GraphEditor", "StructMemberSet", "Set in {VariableName}"), Args);
		}
		else if (GraphNode->IsA(UK2Node_MakeStruct::StaticClass()))
		{
			TitleText = FText::Format(NSLOCTEXT("GraphEditor", "MakeStruct", "Make {VariableName}"), Args);
		}
		else
		{
			check(false);
		}
		bPadTitle = true;
		HorizontalTitleMargin = 12.0f;
		TitleMargin = FMargin(12.0f, VerticalTitleMargin);
	}
	else if (UK2Node_Literal* LiteralRef = Cast<UK2Node_Literal>(GraphNode))
	{
		FText SubTitleText;

		// Get the name of the level the object is in.
		if(AActor* Actor = Cast<AActor>(LiteralRef->GetObjectRef()))
		{
			FText LevelName;

			if(ULevel* Level = Actor->GetLevel())
			{
				if ( Level->IsPersistentLevel() )
				{
					LevelName = NSLOCTEXT("GraphEditor", "PersistentTag", "Persistent Level");
				}
				else
				{
					LevelName = FText::FromString(FPaths::GetCleanFilename(Actor->GetOutermost()->GetName()));
				}
			}
			SubTitleText = FText::Format(NSLOCTEXT("GraphEditor", "ActorRef", "from {0}"), LevelName);
		}

		TitleText = GraphNode->GetNodeTitle(ENodeTitleType::FullTitle);

		TitleHAlign = HAlign_Left;
		TitleMargin = FMargin(12.0f, VerticalTitleMargin, 32.0f, 2.0f);

		TitleWidget = 
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.AutoWidth()
			[
				SNew(SImage)
				.Image(FSlateIconFinder::FindIconBrushForClass(LiteralRef->GetObjectRef() ? LiteralRef->GetObjectRef()->GetClass() : NULL))
			]

			+SHorizontalBox::Slot()
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.AutoWidth()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					.AutoHeight()
				[
					SNew(STextBlock)
						.WrapTextAt(128.0f)
						.TextStyle( FAppStyle::Get(), "Graph.Node.NodeTitle" )
						.Text(TitleText)
				]

				+SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					.AutoHeight()
				[
					SNew(STextBlock)
						.Visibility(TitleText.IsEmpty()? EVisibility::Collapsed : EVisibility::Visible)
						.WrapTextAt(128.0f)
						.TextStyle( FAppStyle::Get(), "Graph.Node.NodeTitleExtraLines" )
						.Text(SubTitleText)
				]
			];
	}
	else if (UK2Node_VariableGet* VariableGet = Cast<UK2Node_VariableGet>(GraphNode))
	{
		if (!VariableGet->IsNodePure())
		{
			TitleText = NSLOCTEXT("GraphEditor", "VariableGet", "GET");
			ContentAreaMargin.Top += 16;
		}
	}

	TitleWidget = UpdateTitleWidget(TitleText, TitleWidget, TitleHAlign, TitleMargin);

	SetupErrorReporting();

	// Setup a meta tag for this node
	FGraphNodeMetaData TagMeta(TEXT("Graphnode"));
	PopulateMetaTag(&TagMeta);

	//             ________________
	//            | (>) L |  R (>) |
	//            | (>) E |  I (>) |
	//            | (>) F |  G (>) |
	//            | (>) T |  H (>) |
	//            |       |  T (>) |
	//            |_______|________|
	//
	this->GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(SOverlay)
			.AddMetaData<FGraphNodeMetaData>(TagMeta)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("Graph.VarNode.Body") )
			]
			+ SOverlay::Slot()
			.VAlign(VAlign_Top)
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("Graph.VarNode.ColorSpill") )
				.ColorAndOpacity( this, &SGraphNodeK2Var::GetVariableColor )
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("Graph.VarNode.Gloss") )
			]
			+SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(TitleHAlign)
			.Padding( TitleMargin )
			[
				TitleWidget.ToSharedRef()
			]
			+ SOverlay::Slot()
			.Padding( ContentAreaMargin )
			[
				// NODE CONTENT AREA
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1.0f)
				.Padding( FMargin(2,0) )
				[
					// LEFT
					SAssignNew(LeftNodeBox, SVerticalBox)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.Padding( FMargin(2,0) )
				[
					// RIGHT
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			]
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight() 
		.Padding( FMargin(5.0f, 1.0f) )
		[
			ErrorReporting->AsWidget()
		]
	];

	float VerticalPaddingAmount = 0.0f;

	// Add padding widgets at the top of the pin boxes if it's a struct operation with a long title
	if (bPadTitle)
	{
		VerticalPaddingAmount += 16.0f;
	}

	// Add padding to offset the exec pin so that it will align with other nodes
	if (UK2Node* K2Node = Cast<UK2Node>(GraphNode))
	{
		if (!K2Node->IsNodePure())
		{
			VerticalPaddingAmount += 7.0f;
		}
	}

	if (VerticalPaddingAmount > 0.0f)
	{
		LeftNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SSpacer).Size(FVector2D(0.0f, VerticalPaddingAmount))
			];
		RightNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SSpacer).Size(FVector2D(0.0f, VerticalPaddingAmount))
			];
	}
	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew( CommentBubble, SCommentBubble )
	.GraphNode( GraphNode )
	.Text( this, &SGraphNode::GetNodeComment )
	.OnTextCommitted( this, &SGraphNode::OnCommentTextCommitted )
	.ColorAndOpacity( CommentColor )
	.AllowPinning( true )
	.EnableTitleBarBubble( true )
	.EnableBubbleCtrls( true )
	.GraphLOD( this, &SGraphNode::GetCurrentLOD )
	.IsGraphNodeHovered( this, &SGraphNode::IsHovered );

	GetOrAddSlot( ENodeZone::TopCenter )
	.SlotOffset2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetOffset2f ))
	.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f ))
	.AllowScaling( TAttribute<bool>( CommentBubble.Get(), &SCommentBubble::IsScalingAllowed ))
	.VAlign( VAlign_Top )
	[
		CommentBubble.ToSharedRef()
	];

	// Create widgets for each of the real pins
	CreatePinWidgets();
}

const FSlateBrush* SGraphNodeK2Var::GetShadowBrush(bool bSelected) const
{
	return bSelected ? FAppStyle::GetBrush(TEXT("Graph.VarNode.ShadowSelected")) : FAppStyle::GetBrush(TEXT("Graph.VarNode.Shadow"));
}

void SGraphNodeK2Var::GetDiffHighlightBrushes(const FSlateBrush*& BackgroundOut, const FSlateBrush*& ForegroundOut) const
{
	BackgroundOut = FAppStyle::GetBrush(TEXT("Graph.VarNode.DiffHighlight"));
	ForegroundOut = FAppStyle::GetBrush(TEXT("Graph.VarNode.DiffHighlightShading"));
}

