// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeMakeStruct.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "SGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "K2Node.h"
#include "K2Node_MakeStruct.h"
#include "NodeFactory.h"

#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "SGraphNodeMakeStruct"

// Enable this option to allow for the deprecated states to be selectable options in the dropdown.
// Warning: These states are no longer being supported but were possible states for optional pins prior to 4.11
#define ALLOW_DEPRECATED_STATES 0

class SOptionalPinStateView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOptionalPinStateView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin, FOptionalPinFromProperty& InPropertyEntry);

protected:
	/** Generates items for the ComboBox's list */
	TSharedRef<SWidget> OnGenerateWidget(TSharedPtr<FOptionalPinOverrideState> InItem);

	/** Callback for display of the current icon for the pin's status */
	const FSlateBrush* GetPinOverrideIcon(FOptionalPinFromProperty* InPropertyEntry) const;

	/** Callback for display of the current icon color for the pin's status */
	FLinearColor GetPinOverrideIconColor(FOptionalPinFromProperty* InPropertyEntry) const;

	/**
	 * Updates the optional pin's data with the current selection (cached)
	 *
	 * @param InGraphPin			The pin being updated
	 * @param InPropertyEntry		The optional pin data associated with the pin
	 */
	void UpdateOptionalPin(UEdGraphPin* InGraphPin, FOptionalPinFromProperty* InPropertyEntry);

	/**
	 * Callback after selecting a new item in the ComboBox
	 *
	 * @param InItem				Selected item
	 * @param InSelectInfo			Info about the method of selection
	 * @param InGraphPin			The pin being updated
	 * @param InPropertyEntry		The optional pin data associated with the pin
	 */
	void OnOverrideStateSelected(TSharedPtr<FOptionalPinOverrideState> InItem, ESelectInfo::Type InSelectInfo, UEdGraphPin* InGraphPin, FOptionalPinFromProperty* InPropertyEntry);

	/**
	 * Toggle button's selection event.
	 *
	 * @param InGraphPin			The pin being updated
	 * @param InPropertyEntry		The optional pin data associated with the pin
	 */
	FReply OnOverrideStateToggled(UEdGraphPin* InGraphPin, FOptionalPinFromProperty* InPropertyEntry);

	/**
	 * Retrieves the icon for display of the current status of the optional pin
	 *
	 * @param bInIsOverrideEnabled		TRUE if the optional pin's override is enabled
	 * @param bInIsSetValuePinVisible	TRUE if the optional pin's value pin is visible
	 * @param bInIsOverridePinVisible	TRUE if the optional pin's override is visible
	 * @param OutBrush					Returns the brush
	 */
	void GetIconForOptionalPinOverrideState(bool bInIsOverrideEnabled, bool bIsSetValuePinVisible, bool bInIsOverridePinVisible, const FSlateBrush*& OutBrush) const;

	/**
	 * Retrieves the icon's color for display of the current status of the optional pin
	 *
	 * @param bInIsOverrideEnabled		TRUE if the optional pin's override is enabled
	 * @param bInIsSetValuePinVisible	TRUE if the optional pin's value pin is visible
	 * @param bInIsOverridePinVisible	TRUE if the optional pin's override is visible
	 * @param OutColor					Returns the color
	 */
	void GetIconColorForOptionalPinOverrideState(bool bInIsOverrideEnabled, bool bIsSetValuePinVisible, bool bInIsOverridePinVisible, FLinearColor& OutColor) const;

	/** Generates the selection widget, either a SComboBox or an SButton, used to select valid states for the pin */
	TSharedRef<SWidget> CreateSelectionWidget(UEdGraphPin* InPin, FOptionalPinFromProperty& InPropertyEntry, TSharedPtr<FOptionalPinOverrideState> InInitiallySelectedItem);

	/** Returns the current tooltip for the status of the pin */
	FText GetCurrentTooltip() const;


protected:
	/** Wrapper widget that holds either the SComboBox or the SButton based on the number of items allowed for selection */
	TWeakPtr<SBox> WrapperWidget;

	/** The Blueprint that is being targetted by this optional pin change */
	UBlueprint* TargetBlueprint;

	/** The current item selected for this optional pin, used for displaying tooltips */
	TSharedPtr<FOptionalPinOverrideState> CurrentSelection;

	/** List of items available for selection by this pin */
	TArray<TSharedPtr<FOptionalPinOverrideState>> ListItems;
};

void SOptionalPinStateView::Construct(const FArguments& InArgs, UEdGraphPin* InPin, FOptionalPinFromProperty& InPropertyEntry)
{
	UK2Node_MakeStruct* SetFieldsNode = CastChecked<UK2Node_MakeStruct>(InPin->GetOwningNode());
	TargetBlueprint = SetFieldsNode->GetBlueprint();

	TArray<TSharedPtr<FOptionalPinOverrideState>> AllListItems;
	FOptionalPinOverrideState CurrentState(FText::GetEmpty(), FText::GetEmpty(), InPropertyEntry.bIsOverridePinVisible, InPropertyEntry.bIsOverrideEnabled, InPropertyEntry.bIsSetValuePinVisible);

	TSharedPtr<FOptionalPinOverrideState> NoOverrideNoValue = MakeShareable(new FOptionalPinOverrideState(
		LOCTEXT("DisableOverride", "Disable Override"),
		LOCTEXT("DisableOverride_Tooltip", "Disables the override value from being used without modifying the value stored in the struct."),
		true, false, false));

	if (SetFieldsNode->GetClass() != UK2Node_MakeStruct::StaticClass() || ALLOW_DEPRECATED_STATES)
	{
		ListItems.Add(NoOverrideNoValue);
	}
	else
	{
		AllListItems.Add(NoOverrideNoValue);
	}
	ListItems.Add(MakeShareable(new FOptionalPinOverrideState(
		LOCTEXT("EnableOverride", "Override"),
		LOCTEXT("EnableOverride_Tooltip", "Enables the override to use the value currently stored in the struct."),
		true, true, false)));
	ListItems.Add(MakeShareable(new FOptionalPinOverrideState(
		LOCTEXT("EnableOverrideSetValue", "Override and Set Value"),
		LOCTEXT("EnableOverrideSetValue_Tooltip", "Overrides and updates the value in the struct."),
		true, true, true)));

#if ALLOW_DEPRECATED_STATES == 0
	AllListItems.Append(ListItems);
#endif

	// Can enable the extra two states that were previously supported when the override and value were two separate pins. This functionality is not officially supported anymore and will incur warnings.
	AllListItems.Add(MakeShareable(new FOptionalPinOverrideState(
		LOCTEXT("SetValue", "Set Value"),
		LOCTEXT("SetValue_Tooltip", "Updates only the value inside the struct without changing whether the override is enabled or not.\nWarning: This setting is no longer a supported workflow and it is advised that you refactor your Blueprint to not use it!"),
		false, false, true)));
	AllListItems.Add(MakeShareable(new FOptionalPinOverrideState(
		LOCTEXT("DisableOverrideSetValue", "Disable Override and Set Value"),
		LOCTEXT("DisableOverrideSetValue_Tooltip", "Disables the override and updates the stored value.\nWarning: This setting is no longer a supported workflow and it is advised that you refactor your Blueprint to not use it!"),
		true, false, true)));

#if ALLOW_DEPRECATED_STATES
	ListItems.Append(AllListItems);
#endif
	TSharedPtr<FOptionalPinOverrideState> InitiallySelectedItem;
	for (TSharedPtr<FOptionalPinOverrideState> State : AllListItems)
	{
		if (CurrentState == *State.Get())
		{
			CurrentSelection = State;
			if (ListItems.Find(CurrentSelection) != INDEX_NONE)
			{
				InitiallySelectedItem = CurrentSelection;
			}
			break;
		}
	}

	ChildSlot
		[
			SAssignNew(WrapperWidget, SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				CreateSelectionWidget(InPin, InPropertyEntry, InitiallySelectedItem)
			]
		];
}

TSharedRef<SWidget> SOptionalPinStateView::CreateSelectionWidget(UEdGraphPin* InPin, FOptionalPinFromProperty& InPropertyEntry, TSharedPtr<FOptionalPinOverrideState> InInitiallySelectedItem)
{
	if (ListItems.Num() > 2 || ListItems.Find(InInitiallySelectedItem) == INDEX_NONE)
	{
		return
			SNew(SComboBox<TSharedPtr<FOptionalPinOverrideState>>)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ForegroundColor(FLinearColor::White)
				.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.OptionsSource(&ListItems)
				.InitiallySelectedItem(InInitiallySelectedItem)
				.OnGenerateWidget(this, &SOptionalPinStateView::OnGenerateWidget)
				.OnSelectionChanged(this, &SOptionalPinStateView::OnOverrideStateSelected, InPin, &InPropertyEntry)
				[
					SNew(SBorder)
					.Padding(0.0f)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.ColorAndOpacity(this, &SOptionalPinStateView::GetPinOverrideIconColor, &InPropertyEntry)
					[
						SNew(SImage)
						.Image(this, &SOptionalPinStateView::GetPinOverrideIcon, &InPropertyEntry)
						.ToolTipText(this, &SOptionalPinStateView::GetCurrentTooltip)
					]
				];
	}
	return
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.OnClicked(this, &SOptionalPinStateView::OnOverrideStateToggled, InPin, &InPropertyEntry)
		.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.ColorAndOpacity(this, &SOptionalPinStateView::GetPinOverrideIconColor, &InPropertyEntry)
			[
				SNew(SImage)
				.Image(this, &SOptionalPinStateView::GetPinOverrideIcon, &InPropertyEntry)
				.ToolTipText(this, &SOptionalPinStateView::GetCurrentTooltip)
			]
		];
}

TSharedRef<SWidget> SOptionalPinStateView::OnGenerateWidget(TSharedPtr<FOptionalPinOverrideState> InItem)
{
	const FSlateBrush* Brush = nullptr;
	FLinearColor Color = FColor(64, 64, 64).ReinterpretAsLinear();
	GetIconForOptionalPinOverrideState(InItem->bIsOverrideEnabled, InItem->bIsValuePinVisible, InItem->bIsOverridePinVisible, Brush);
	GetIconColorForOptionalPinOverrideState(InItem->bIsOverrideEnabled, InItem->bIsValuePinVisible, InItem->bIsOverridePinVisible, Color);
	return
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.ColorAndOpacity(Color)
			[
				SNew(SImage)
				.Image(Brush)
				.ToolTipText(InItem->TooltipText)
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(InItem->DisplayText)
			.ToolTipText(InItem->TooltipText)
		];
}

void SOptionalPinStateView::GetIconForOptionalPinOverrideState(bool bInIsOverrideEnabled, bool bIsSetValuePinVisible, bool bInIsOverridePinVisible, const FSlateBrush*& OutBrush) const
{
	const FSlateBrush* Brush = nullptr;
	if (!bInIsOverridePinVisible || (bInIsOverridePinVisible && !bInIsOverrideEnabled && bIsSetValuePinVisible))
	{
		Brush = FAppStyle::GetBrush("Icons.Warning");
	}
	else if (!bInIsOverrideEnabled && !bIsSetValuePinVisible)
	{
		Brush = FAppStyle::GetBrush("Kismet.VariableList.HideForInstance");
	}
	else
	{
		Brush = FAppStyle::GetBrush("Kismet.VariableList.ExposeForInstance");
	}
	OutBrush = Brush;
}

void SOptionalPinStateView::GetIconColorForOptionalPinOverrideState(bool bInIsOverrideEnabled, bool bIsSetValuePinVisible, bool bInIsOverridePinVisible, FLinearColor& OutColor) const
{
	FLinearColor Color = FColor(64, 64, 64).ReinterpretAsLinear();
	if (!bInIsOverridePinVisible || (bInIsOverridePinVisible && !bInIsOverrideEnabled && bIsSetValuePinVisible))
	{
		Color = FLinearColor::White;
	}
	else if (!bInIsOverrideEnabled && !bIsSetValuePinVisible)
	{
	}
	else
	{
		if (bInIsOverrideEnabled && !bIsSetValuePinVisible)
		{
			Color = FColor(215, 219, 119).ReinterpretAsLinear();
		}
		else
		{
			Color = FColor(130, 219, 119).ReinterpretAsLinear();
		}
	}
	OutColor = Color;
}

const FSlateBrush* SOptionalPinStateView::GetPinOverrideIcon(FOptionalPinFromProperty* InPropertyEntry) const
{
	const FSlateBrush* Brush = nullptr;
	GetIconForOptionalPinOverrideState(InPropertyEntry->bIsOverrideEnabled, InPropertyEntry->bIsSetValuePinVisible, InPropertyEntry->bIsOverridePinVisible, Brush);
	return Brush;
}

FLinearColor SOptionalPinStateView::GetPinOverrideIconColor(FOptionalPinFromProperty* InPropertyEntry) const
{
	FLinearColor Color = FColor(64, 64, 64).ReinterpretAsLinear();
	GetIconColorForOptionalPinOverrideState(InPropertyEntry->bIsOverrideEnabled, InPropertyEntry->bIsSetValuePinVisible, InPropertyEntry->bIsOverridePinVisible, Color);
	return Color;
}

void SOptionalPinStateView::UpdateOptionalPin(UEdGraphPin* InGraphPin, FOptionalPinFromProperty* InPropertyEntry)
{
	InGraphPin->bNotConnectable = !CurrentSelection->bIsValuePinVisible;
	InPropertyEntry->bIsOverrideEnabled = CurrentSelection->bIsOverrideEnabled;
	InPropertyEntry->bIsSetValuePinVisible = CurrentSelection->bIsValuePinVisible;
	InPropertyEntry->bIsOverridePinVisible = CurrentSelection->bIsOverridePinVisible;
	InGraphPin->bDefaultValueIsIgnored = !InPropertyEntry->bIsSetValuePinVisible;
	FBlueprintEditorUtils::MarkBlueprintAsModified(TargetBlueprint);
}

void SOptionalPinStateView::OnOverrideStateSelected(TSharedPtr<FOptionalPinOverrideState> InItem, ESelectInfo::Type InSelectInfo, UEdGraphPin* InGraphPin, FOptionalPinFromProperty* InPropertyEntry)
{
	if (InItem.IsValid())
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("PinOverrideStateChanged", "Pin Override State Changed"));
		InGraphPin->GetOwningNode()->Modify();
		InGraphPin->Modify();

		CurrentSelection = InItem;
		UpdateOptionalPin(InGraphPin, InPropertyEntry);

		if (ListItems.Num() == 2 && ListItems.Find(CurrentSelection) != INDEX_NONE)
		{
			WrapperWidget.Pin()->SetContent(CreateSelectionWidget(InGraphPin, *InPropertyEntry, CurrentSelection));
		}
	}
}

FReply SOptionalPinStateView::OnOverrideStateToggled(UEdGraphPin* InGraphPin, FOptionalPinFromProperty* InPropertyEntry)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("PinOverrideStateChanged", "Pin Override State Changed"));
	InGraphPin->GetOwningNode()->Modify();
	InGraphPin->Modify();

	if (CurrentSelection == ListItems[0])
	{
		CurrentSelection = ListItems[1];
	}
	else
	{
		CurrentSelection = ListItems[0];
	}

	UpdateOptionalPin(InGraphPin, InPropertyEntry);
	return FReply::Handled();
}

FText SOptionalPinStateView::GetCurrentTooltip() const
{
	if (CurrentSelection.IsValid())
	{
		return CurrentSelection->TooltipText;
	}
	return FText::GetEmpty();
}

//////////////////////////////////////////////////////////////////////////
// SGraphNodeMakeStruct

void SGraphNodeMakeStruct::Construct(const FArguments& InArgs, UK2Node_MakeStruct* InNode)
{
	GraphNode = InNode;

	SetCursor( EMouseCursor::CardinalCross );
	UpdateGraphNode();
}

TSharedPtr<SGraphPin> SGraphNodeMakeStruct::CreatePinWidget(UEdGraphPin* Pin) const
{
	UK2Node_MakeStruct* SetFieldsNode = CastChecked<UK2Node_MakeStruct>(GetNodeObj());
	TSharedPtr<SGraphPin> ResultPin = FNodeFactory::CreatePinWidget(Pin);

	if (!Pin->PinType.bIsReference && Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec && Pin->Direction != EGPD_Output)
	{
		// Attempt to find a matching optional pin entry that is bound to an edit condition as a potential override.
		FOptionalPinFromProperty* BoundPropertyEntryPtr = SetFieldsNode->ShowPinForProperties.FindByPredicate([Pin](const FOptionalPinFromProperty& PropertyEntry)
		{
			return PropertyEntry.bHasOverridePin && PropertyEntry.PropertyName == Pin->PinName;
		});

		if (BoundPropertyEntryPtr)
		{
			FOptionalPinFromProperty& PropertyEntry = *BoundPropertyEntryPtr;

			// Find the member property that's bound to the input pin.
			const FProperty* BoundProperty = FindFProperty<FProperty>(SetFieldsNode->StructType, Pin->PinName);
			check(BoundProperty);

			// If this member is also bound to an override condition, insert the widget that allows the user to toggle input on/off for the override value.
			if (const FBoolProperty* OverrideProperty = SetFieldsNode->GetOverrideConditionForProperty(BoundProperty))
			{
				TWeakPtr<SHorizontalBox> HorizontalPin = ResultPin->GetFullPinHorizontalRowWidget();
				if (HorizontalPin.IsValid())
				{
					// Setup the pin's editable state to be dependent on whether the override is enabled
					TAttribute<bool> IsEditableAttribute;
					TAttribute<bool>::FGetter IsEditableGetter;
					IsEditableGetter.BindSP(this, &SGraphNodeMakeStruct::IsPinEnabled, &PropertyEntry);
					IsEditableAttribute.Bind(IsEditableGetter);
					ResultPin->SetIsEditable(IsEditableAttribute);

					HorizontalPin.Pin()->InsertSlot(1)
						.Padding(0.0f, 0.0f, 2.0f, 0.0f)
						[
							SNew(SOptionalPinStateView, Pin, PropertyEntry)
						];
				}
			}
		}
	}
	return ResultPin;
}

void SGraphNodeMakeStruct::CreatePinWidgets()
{
	// Create Pin widgets for each of the pins.
	for (int32 PinIndex = 0; PinIndex < GraphNode->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* CurPin = GraphNode->Pins[PinIndex];

		if (ShouldPinBeHidden(CurPin))
		{
			AddPin(CreatePinWidget(CurPin).ToSharedRef());
		}
	}
}

bool SGraphNodeMakeStruct::IsPinEnabled(FOptionalPinFromProperty* InPropertyEntry) const
{
	return InPropertyEntry->bIsSetValuePinVisible;
}

#undef LOCTEXT_NAMESPACE
