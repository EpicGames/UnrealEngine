// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SGraphPin;
class UEdGraphPin;
class UK2Node_MakeStruct;
struct FOptionalPinFromProperty;

struct FOptionalPinOverrideState
{
	/** Display text to display for this item in the selection widget */
	FText DisplayText;

	/** Tooltip to display for this item on both the node and in the selection widget */
	FText TooltipText;

	/** TRUE if this option will enable the override */
	bool bIsOverrideEnabled;

	/** TRUE if this option will make the value visible/editable*/
	bool bIsValuePinVisible;

	/** TRUE if the override value can be edited */
	bool bIsOverridePinVisible;

	FOptionalPinOverrideState(FText InDisplayText, FText InTooltipText, bool bInIsOverridePinVisible, bool bInIsOverrideEnabled, bool bInIsValuePinVisible)
		: DisplayText(InDisplayText)
		, TooltipText(InTooltipText)
		, bIsOverrideEnabled(bInIsOverrideEnabled)
		, bIsValuePinVisible(bInIsValuePinVisible)
		, bIsOverridePinVisible(bInIsOverridePinVisible)
	{}

	bool operator== (const FOptionalPinOverrideState& InOther) const
	{
		return bIsOverrideEnabled == InOther.bIsOverrideEnabled && bIsValuePinVisible == InOther.bIsValuePinVisible && bIsOverridePinVisible == InOther.bIsOverridePinVisible;
	}
};

class SGraphNodeMakeStruct : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeMakeStruct){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UK2Node_MakeStruct* InNode);

	/** Callback for the pin for whether the pin should display as enabled */
	bool IsPinEnabled(FOptionalPinFromProperty* InPropertyEntry) const;

	// SGraphNode interface
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	virtual void CreatePinWidgets() override;
	// End of SGraphNode interface
};
