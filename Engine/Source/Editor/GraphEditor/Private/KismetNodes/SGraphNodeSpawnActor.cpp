// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeSpawnActor.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_SpawnActor.h"
#include "KismetPins/SGraphPinObject.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SGraphNodeSpawnActor"

//////////////////////////////////////////////////////////////////////////
// SGraphPinSwitchNodeDefaultCaseExec

/** 
 * GraphPin can select only actor classes generated by blueprint.
 * Instead of asset picker, a class viewer is used, and blueprint is obtained form class.
 */
class SGraphPinActorBasedBlueprintClass : public SGraphPinObject
{
	void OnClassPicked(UClass* InChosenClass)
	{
		AssetPickerAnchor->SetIsOpen(false);

		if(InChosenClass && GraphPinObj)
		{
			check(InChosenClass->ClassGeneratedBy);
			check(InChosenClass->IsChildOf(AActor::StaticClass()));
			if(const UEdGraphSchema* Schema = GraphPinObj->GetSchema())
			{
				const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeClassPinValue", "Change Class Pin Value"));
				GraphPinObj->Modify();

				Schema->TrySetDefaultObject(*GraphPinObj, InChosenClass->ClassGeneratedBy);
			}
		}
	}

	class FActorBasedBlueprintClassFilter : public IClassViewerFilter
	{
	public:

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
		{
			if(NULL != InClass)
			{
				const bool bGeneratedByBlueprint = NULL != Cast<UBlueprint>(InClass->ClassGeneratedBy);
				const bool bActorBased = InClass->IsChildOf(AActor::StaticClass());
				return bGeneratedByBlueprint && bActorBased;
			}
			return false;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			const bool bActorBased = InUnloadedClassData->IsChildOf(AActor::StaticClass());
			return bActorBased;
		}
	};

protected:

	virtual FText GetDefaultComboText() const override { return LOCTEXT( "DefaultComboText", "Select Blueprint" ); }

	virtual TSharedRef<SWidget> GenerateAssetPicker() override
	{
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.DisplayMode = EClassViewerDisplayMode::DefaultView;
		//Options.bIsActorsOnly = true;
		//Options.bIsBlueprintBaseOnly = true;
		Options.bShowUnloadedBlueprints = true;
		Options.bShowNoneOption = true;
		Options.bShowObjectRootClass = false;
		TSharedPtr< FActorBasedBlueprintClassFilter > Filter = MakeShareable(new FActorBasedBlueprintClassFilter);
		Options.ClassFilters.Add(Filter.ToSharedRef());

		return
			SNew(SBox)
			.WidthOverride(280.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(500.0f)
				[ 
					SNew(SBorder)
					.Padding(4.0f)
					.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
					[
						ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SGraphPinActorBasedBlueprintClass::OnClassPicked))
					]
				]			
			];
	}
};

//////////////////////////////////////////////////////////////////////////
// SGraphNodeSpawnActor

void SGraphNodeSpawnActor::CreatePinWidgets()
{
	UK2Node_SpawnActor* SpawnActorNode = CastChecked<UK2Node_SpawnActor>(GraphNode);
	UEdGraphPin* BlueprintPin = SpawnActorNode->GetBlueprintPin();

	// Create Pin widgets for each of the pins, except for the Blueprint pin
	for (auto PinIt = GraphNode->Pins.CreateConstIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* CurrentPin = *PinIt;
		if (CurrentPin != BlueprintPin)
		{
			CreateStandardPinWidget(CurrentPin);
		}
	}

	// Handle the Blueprint pin
	if ((BlueprintPin != NULL) && (!BlueprintPin->bHidden || (BlueprintPin->LinkedTo.Num() > 0)))
	{
		TSharedPtr<SGraphPinActorBasedBlueprintClass> NewPin = SNew(SGraphPinActorBasedBlueprintClass, BlueprintPin);
		check(NewPin.IsValid());

		this->AddPin(NewPin.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE
