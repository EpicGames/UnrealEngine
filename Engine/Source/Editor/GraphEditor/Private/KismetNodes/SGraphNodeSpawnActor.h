// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetNodes/SGraphNodeK2Default.h"

class SGraphNodeSpawnActor : public SGraphNodeK2Default
{
public:

	// SGraphNode interface
	virtual void CreatePinWidgets() override;
	// End of SGraphNode interface
};
