// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeSpawnActorFromClass.h"

#include "AssetRegistry/AssetData.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Brush.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "K2Node_SpawnActorFromClass.h"
#include "KismetPins/SGraphPinClass.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "NodeFactory.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

class SGraphPin;
class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "SGraphPinActorBasedClass"

//////////////////////////////////////////////////////////////////////////
// SGraphPinActorBasedClass

/** 
 * GraphPin can select only actor classes.
 * Instead of asset picker, a class viewer is used.
 */
class SGraphPinActorBasedClass : public SGraphPinClass
{
	class FActorBasedClassFilter : public IClassViewerFilter
	{
	public:

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
		{
			if(NULL != InClass)
			{
				const bool bActorBased = InClass->IsChildOf(AActor::StaticClass());
				const bool bNotBrushBased = !InClass->IsChildOf(ABrush::StaticClass());
				const bool bBlueprintType = UEdGraphSchema_K2::IsAllowableBlueprintVariableType(InClass);
				const bool bNotAbstract = !InClass->HasAnyClassFlags(CLASS_Abstract);
				return bActorBased && bNotBrushBased && bBlueprintType && bNotAbstract;
			}
			return false;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			const bool bActorBased = InUnloadedClassData->IsChildOf(AActor::StaticClass());
			const bool bNotBrushBased = !InUnloadedClassData->IsChildOf(ABrush::StaticClass());
			const bool bNotAbstract = !InUnloadedClassData->HasAnyClassFlags(CLASS_Abstract);
			return bActorBased && bNotBrushBased && bNotAbstract;
		}
	};

protected:

	virtual TSharedRef<SWidget> GenerateAssetPicker() override
	{
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.bIsActorsOnly = true;
		Options.DisplayMode = EClassViewerDisplayMode::DefaultView;
		Options.bShowUnloadedBlueprints = true;
		Options.bShowNoneOption = true;
		Options.bShowObjectRootClass = true;
		TSharedPtr< FActorBasedClassFilter > Filter = MakeShareable(new FActorBasedClassFilter);
		Options.ClassFilters.Add(Filter.ToSharedRef());
		// Populate the referencing asset, if possible
		if (UEdGraphPin* LocalGraphPin = GetPinObj())
		{
			if (UEdGraphNode* LocalNode = LocalGraphPin->GetOwningNode())
			{
				if (UEdGraph* LocalGraph = LocalNode->GetGraph())
				{
					UObject* GraphOuter = LocalGraph->GetOuter();
					if (GraphOuter)
					{
						Options.AdditionalReferencingAssets.Add(FAssetData(GraphOuter));
					}
				}
			}
		}

		return 
			SNew(SBox)
			.WidthOverride(280.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(500.0f)
				[
					SNew(SBorder)
					.Padding(4.0f)
					.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
					[
						ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SGraphPinActorBasedClass::OnPickedNewClass))
					]
				]			
			];
	}
};

//////////////////////////////////////////////////////////////////////////
// SGraphNodeSpawnActorFromClass

void SGraphNodeSpawnActorFromClass::CreatePinWidgets()
{
	UK2Node_SpawnActorFromClass* SpawnActorNode = CastChecked<UK2Node_SpawnActorFromClass>(GraphNode);
	UEdGraphPin* ClassPin = SpawnActorNode->GetClassPin();

	for (auto PinIt = GraphNode->Pins.CreateConstIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* CurrentPin = *PinIt;
		if ((!CurrentPin->bHidden) && (CurrentPin != ClassPin))
		{
			TSharedPtr<SGraphPin> NewPin = FNodeFactory::CreatePinWidget(CurrentPin);
			check(NewPin.IsValid());
			this->AddPin(NewPin.ToSharedRef());
		}
		else if ((ClassPin == CurrentPin) && (!ClassPin->bHidden || (ClassPin->LinkedTo.Num() > 0)))
		{
			TSharedPtr<SGraphPinActorBasedClass> NewPin = SNew(SGraphPinActorBasedClass, ClassPin);
			check(NewPin.IsValid());
			this->AddPin(NewPin.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
