// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetNodes/SGraphNodeSwitchStatement.h"

#include "Containers/Array.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_Switch.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetPins/SGraphPinExec.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "NodeFactory.h"
#include "SGraphNode.h"
#include "SGraphPin.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

//////////////////////////////////////////////////////////////////////////
// SGraphPinSwitchNodeDefaultCaseExec

class SGraphPinSwitchNodeDefaultCaseExec : public SGraphPinExec
{
public:
	SLATE_BEGIN_ARGS(SGraphPinSwitchNodeDefaultCaseExec)	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin)
	{
		SGraphPin::Construct(SGraphPin::FArguments().PinLabelStyle(FName("Graph.Node.DefaultPinName")), InPin);

		CachePinIcons();
	}
};

//////////////////////////////////////////////////////////////////////////
// SGraphNodeSwitchStatement

void SGraphNodeSwitchStatement::Construct(const FArguments& InArgs, UK2Node_Switch* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor( EMouseCursor::CardinalCross );

	this->UpdateGraphNode();
}

void SGraphNodeSwitchStatement::CreatePinWidgets()
{
	UK2Node_Switch* SwitchNode = CastChecked<UK2Node_Switch>(GraphNode);
	UEdGraphPin* DefaultPin = SwitchNode->GetDefaultPin();

	// Create Pin widgets for each of the pins, except for the default pin
	for (auto PinIt = GraphNode->Pins.CreateConstIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* CurrentPin = *PinIt;
		if ((!CurrentPin->bHidden) && (CurrentPin != DefaultPin))
		{
			TSharedPtr<SGraphPin> NewPin = FNodeFactory::CreatePinWidget(CurrentPin);
			check(NewPin.IsValid());

			this->AddPin(NewPin.ToSharedRef());
		}
	}

	// Handle the default pin
	if (DefaultPin != NULL)
	{
		// Create some padding
		RightNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(1.0f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Graph.Pin.DefaultPinSeparator"))
			];

		// Create the pin itself
		TSharedPtr<SGraphPin> NewPin = SNew(SGraphPinSwitchNodeDefaultCaseExec, DefaultPin);

		this->AddPin(NewPin.ToSharedRef());
	}
}

void SGraphNodeSwitchStatement::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
		NSLOCTEXT("SwitchStatementNode", "SwitchStatementNodeAddPinButton", "Add pin"),
		NSLOCTEXT("SwitchStatementNode", "SwitchStatementNodeAddPinButton_Tooltip", "Add new pin"));

	FMargin AddPinPadding = Settings->GetOutputPinPadding();
	AddPinPadding.Top += 6.0f;

	OutputBox->AddSlot()
	.AutoHeight()
	.VAlign(VAlign_Center)
	.Padding(AddPinPadding)
	[
		AddPinButton
	];
}

EVisibility SGraphNodeSwitchStatement::IsAddPinButtonVisible() const
{
	if (UK2Node_Switch* SwitchNode = CastChecked<UK2Node_Switch>(GraphNode)
		; SwitchNode && SwitchNode->SupportsAddPinButton())
	{
		return SGraphNode::IsAddPinButtonVisible();
	}
	return EVisibility::Collapsed;
}

FReply SGraphNodeSwitchStatement::OnAddPin()
{
	UK2Node_Switch* SwitchNode = CastChecked<UK2Node_Switch>(GraphNode);

	const FScopedTransaction Transaction( NSLOCTEXT("Kismet", "AddExecutionPin", "Add Execution Pin") );
	SwitchNode->Modify();

	SwitchNode->AddPinToSwitchNode();
	FBlueprintEditorUtils::MarkBlueprintAsModified(SwitchNode->GetBlueprint());

	UpdateGraphNode();
	GraphNode->GetGraph()->NotifyNodeChanged(GraphNode);

	return FReply::Handled();
}
