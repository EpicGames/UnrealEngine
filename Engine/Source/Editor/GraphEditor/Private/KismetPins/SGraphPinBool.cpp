// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinBool.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"

class SWidget;

void SGraphPinBool::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinBool::GetDefaultValueWidget()
{
	return SNew(SCheckBox)
		.IsChecked(this, &SGraphPinBool::IsDefaultValueChecked)
		.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
		.OnCheckStateChanged(this, &SGraphPinBool::OnDefaultValueCheckBoxChanged)
		.Visibility( this, &SGraphPin::GetDefaultValueVisibility );
}

ECheckBoxState SGraphPinBool::IsDefaultValueChecked() const
{
	FString CurrentValue = GraphPinObj->GetDefaultAsString();
	return CurrentValue.ToBool() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SGraphPinBool::OnDefaultValueCheckBoxChanged(ECheckBoxState InIsChecked)
{
	if(GraphPinObj->IsPendingKill())
	{
		return;
	}

	const FString BoolString = (InIsChecked == ECheckBoxState::Checked) ? TEXT("true") : TEXT("false");
	if(GraphPinObj->GetDefaultAsString() != BoolString)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeBoolPinValue", "Change Bool Pin Value" ) );
		GraphPinObj->Modify();

		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, BoolString);
	}
}
