// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinClass.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "SGraphPin.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SGraphPinClass"

/////////////////////////////////////////////////////
// SGraphPinClass

void SGraphPinClass::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	bAllowAbstractClasses = true;
}

FReply SGraphPinClass::OnClickUse()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	if(GraphPinObj && GraphPinObj->GetSchema())
	{
		const UClass* PinRequiredParentClass = Cast<const UClass>(GraphPinObj->PinType.PinSubCategoryObject.Get());
		ensure(PinRequiredParentClass);

		const UClass* SelectedClass = GEditor->GetFirstSelectedClass(PinRequiredParentClass);
		if(SelectedClass)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeClassPinValue", "Change Class Pin Value"));
			GraphPinObj->Modify();

			GraphPinObj->GetSchema()->TrySetDefaultObject(*GraphPinObj, const_cast<UClass*>(SelectedClass));
		}
	}

	return FReply::Handled();
}

class FGraphPinFilter : public IClassViewerFilter
{
public:
	/** Package containing the graph pin */
	const UPackage* GraphPinOutermostPackage;

	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	const UClass* RequiredInterface = nullptr;

	bool bAllowAbstractClasses = true;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		// If it appears on the allowed child-of classes list (or there is nothing on that list)
		bool Result = (InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed);
		if (Result)
		{
			check(InClass != nullptr);
			const UPackage* ClassPackage = InClass->GetOutermost();
			check(ClassPackage != nullptr);
		
			// Don't allow classes from a loaded map (e.g. LSBPs) unless we're already working inside that package context. Otherwise, choosing the class would lead to a GLEO at save time.
			Result &= !ClassPackage->ContainsMap() || ClassPackage == GraphPinOutermostPackage;
			Result &= !InClass->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown);
			Result &= bAllowAbstractClasses || !InClass->HasAnyClassFlags(CLASS_Abstract);
			// either there is not a required interface, or our target class DOES implement that interface
			Result &= (RequiredInterface == nullptr || InClass->ImplementsInterface(RequiredInterface));
		}

		return Result;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return (InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed) 
			&& (!InUnloadedClassData->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown))
			&& (bAllowAbstractClasses || !InUnloadedClassData->HasAnyClassFlags(CLASS_Abstract))
			// either there is not a required interface, or our target class DOES implement that interface
			&& (RequiredInterface == nullptr || InUnloadedClassData->ImplementsInterface(RequiredInterface));
	}
};

TSharedRef<SWidget> SGraphPinClass::GenerateAssetPicker()
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.bShowNoneOption = true;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;

	// Get the min. spec for the classes allowed
	const UClass* PinRequiredParentClass = Cast<const UClass>(GraphPinObj->PinType.PinSubCategoryObject.Get());
	ensure(PinRequiredParentClass);
	if (PinRequiredParentClass == NULL)
	{
		PinRequiredParentClass = UObject::StaticClass();
	}

	//Looks like this defaults to ClassName? Either way, allow UPARAM to specify this. 
	const FString ShowDisplayNamesString = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, FBlueprintMetadata::MD_ShowDisplayNames);
	if (!ShowDisplayNamesString.IsEmpty() && ShowDisplayNamesString.ToBool() == true)
	{
		Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	}

	TSharedPtr<FGraphPinFilter> Filter = MakeShareable(new FGraphPinFilter);
	Filter->bAllowAbstractClasses = bAllowAbstractClasses;

	FString AllowedClassesString = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, FBlueprintMetadata::MD_AllowedClasses);
	if (!AllowedClassesString.IsEmpty())
	{
		Filter->AllowedChildrenOfClasses.Append(PropertyCustomizationHelpers::GetClassesFromMetadataString(AllowedClassesString));
	}
	
	// Check with the node to see if there is any "AllowAbstract" metadata for the pin
	FString AllowAbstractString = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, FBlueprintMetadata::MD_AllowAbstractClasses);
	
	// Override bAllowAbstractClasses is the AllowAbstract metadata was set
	if (!AllowAbstractString.IsEmpty())
	{
		Filter->bAllowAbstractClasses = AllowAbstractString.ToBool();
	}
	
	Options.ClassFilters.Add(Filter.ToSharedRef());

	if (Filter->AllowedChildrenOfClasses.Num() == 0)
	{
		Filter->AllowedChildrenOfClasses.Add(PinRequiredParentClass);
	}
	
	Filter->GraphPinOutermostPackage = GraphPinObj->GetOuter()->GetOutermost();

	if (UEdGraphNode* ParentNode = GraphPinObj->GetOwningNode())
	{
		FString PossibleInterface = ParentNode->GetPinMetaData(GraphPinObj->PinName, TEXT("MustImplement"));
		if (!PossibleInterface.IsEmpty())
		{
			Filter->RequiredInterface = UClass::TryFindTypeSlow<UClass>(PossibleInterface);
		}
	}

	return
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.MaxHeight(500.0f)
			[ 
				SNew(SBorder)
				.Padding(4.0f)
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				[
					ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SGraphPinClass::OnPickedNewClass))
				]
			]			
		];
}

FOnClicked SGraphPinClass::GetOnUseButtonDelegate()
{
	return FOnClicked::CreateSP( this, &SGraphPinClass::OnClickUse );
}

void SGraphPinClass::OnPickedNewClass(UClass* ChosenClass)
{
	if(GraphPinObj->IsPendingKill())
	{
		return;
	}

	FString NewPath;
	if (ChosenClass)
	{
		NewPath = ChosenClass->GetPathName();
	}

	if(GraphPinObj->GetDefaultAsString() != NewPath)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeClassPinValue", "Change Class Pin Value" ) );
		GraphPinObj->Modify();

		AssetPickerAnchor->SetIsOpen(false);
		GraphPinObj->GetSchema()->TrySetDefaultObject(*GraphPinObj, ChosenClass);
	}
}

FText SGraphPinClass::GetDefaultComboText() const
{ 
	return LOCTEXT( "DefaultComboText", "Select Class" );
}

const FAssetData& SGraphPinClass::GetAssetData(bool bRuntimePath) const
{
	if (bRuntimePath)
	{
		// For runtime use the default path
		return SGraphPinObject::GetAssetData(bRuntimePath);
	}

	FString CachedRuntimePath = CachedEditorAssetData.GetObjectPathString() + TEXT("_C");

	if (GraphPinObj->DefaultObject)
	{
		if (!GraphPinObj->DefaultObject->GetPathName().Equals(CachedRuntimePath, ESearchCase::CaseSensitive))
		{
			// This will cause it to use the UBlueprint
			CachedEditorAssetData = FAssetData(GraphPinObj->DefaultObject, false);
		}
	}
	else if (!GraphPinObj->DefaultValue.IsEmpty())
	{
		if (!GraphPinObj->DefaultValue.Equals(CachedRuntimePath, ESearchCase::CaseSensitive))
		{
			FString EditorPath = GraphPinObj->DefaultValue;
			EditorPath.RemoveFromEnd(TEXT("_C"));
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			CachedEditorAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(EditorPath));

			if (!CachedEditorAssetData.IsValid())
			{
				FString PackageName = FPackageName::ObjectPathToPackageName(EditorPath);
				FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
				FString ObjectName = FPackageName::ObjectPathToObjectName(EditorPath);

				// Fake one
				CachedEditorAssetData = FAssetData(FName(*PackageName), FName(*PackagePath), FName(*ObjectName), UObject::StaticClass()->GetClassPathName());
			}
		}
	}
	else
	{
		if (CachedEditorAssetData.IsValid())
		{
			CachedEditorAssetData = FAssetData();
		}
	}

	return CachedEditorAssetData;
}

#undef LOCTEXT_NAMESPACE
