// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetPins/SGraphPinCollisionProfile.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/CollisionProfile.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "SNameComboBox.h"
#include "ScopedTransaction.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SWidget;

void SGraphPinCollisionProfile::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}


TSharedRef<SWidget>	SGraphPinCollisionProfile::GetDefaultValueWidget()
{
	UCollisionProfile::GetProfileNames(NameList);

	TSharedPtr<FName> InitialSelectedName = GetSelectedName();
	if (InitialSelectedName.IsValid())
	{
		SetPropertyWithName(*InitialSelectedName.Get());
	}

	return SAssignNew(NameComboBox, SNameComboBox)
		.ContentPadding(FMargin(6.0f, 2.0f))
		.OptionsSource(&NameList)
		.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
		.InitiallySelectedItem(InitialSelectedName)
		.OnSelectionChanged(this, &SGraphPinCollisionProfile::OnSelectionChanged)
		.OnComboBoxOpening(this, &SGraphPinCollisionProfile::OnComboBoxOpening)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
}


void SGraphPinCollisionProfile::OnSelectionChanged(TSharedPtr<FName> NameItem, ESelectInfo::Type SelectInfo)
{
	if (NameItem.IsValid())
	{
		SetPropertyWithName(*NameItem);
	}
}


void SGraphPinCollisionProfile::OnComboBoxOpening()
{
	TSharedPtr<FName> SelectedName = GetSelectedName();
	if (SelectedName.IsValid())
	{
		check(NameComboBox.IsValid());
		NameComboBox->SetSelectedItem(SelectedName);
	}
}


TSharedPtr<FName> SGraphPinCollisionProfile::GetSelectedName() const
{
	int32 NameCount = NameList.Num();
	if (NameCount <= 0)
	{
		return NULL;
	}

	FName Name;
	GetPropertyAsName(Name);

	for (int32 NameIndex = 0; NameIndex < NameCount; ++NameIndex)
	{
		if (Name == *NameList[NameIndex].Get())
		{
			return NameList[NameIndex];
		}
	}

	return NameList[0];
}


void SGraphPinCollisionProfile::SetPropertyWithName(const FName& Name)
{
	check(GraphPinObj);
	check(GraphPinObj->PinType.PinSubCategoryObject == FCollisionProfileName::StaticStruct());
	
	FString PinString = TEXT("(Name=\"");
	PinString += *Name.ToString();
	PinString += TEXT("\")");

	if(GraphPinObj->GetDefaultAsString() != PinString)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeCollisionProfilePinValue", "Change Collision Profile Pin Value" ) );
		GraphPinObj->Modify();

		if (PinString != GraphPinObj->GetDefaultAsString())
		{
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, PinString);
		}
	}
}


void SGraphPinCollisionProfile::GetPropertyAsName(FName& OutName) const
{
	check(GraphPinObj);
	check(GraphPinObj->PinType.PinSubCategoryObject == FCollisionProfileName::StaticStruct());

	FString PinString = GraphPinObj->GetDefaultAsString();

	if (PinString.StartsWith(TEXT("("), ESearchCase::CaseSensitive) && PinString.EndsWith(TEXT(")"), ESearchCase::CaseSensitive))
	{
		PinString.LeftChopInline(1, EAllowShrinking::No);
		PinString.RightChopInline(1, EAllowShrinking::No);
		PinString.Split(TEXT("="), nullptr, &PinString, ESearchCase::CaseSensitive);

		if (PinString.StartsWith(TEXT("\""), ESearchCase::CaseSensitive) && PinString.EndsWith(TEXT("\""), ESearchCase::CaseSensitive))
		{
			PinString.LeftChopInline(1, EAllowShrinking::No);
			PinString.RightChopInline(1, EAllowShrinking::No);
		}
	}

	if (!PinString.IsEmpty())
	{
		OutName = *PinString;

		UCollisionProfile* CollisionProfile = UCollisionProfile::Get();
		check(CollisionProfile);

		const FName* RedirectName = CollisionProfile->LookForProfileRedirect(OutName);
		if (RedirectName)
		{
			OutName = *RedirectName;
		}
	}
}
