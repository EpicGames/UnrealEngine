// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SNameComboBox;
class SWidget;
class UEdGraphPin;

/**
 * Customizes a CollisionProfileName graph pin to use a dropdown
 */
class SGraphPinCollisionProfile : public SGraphPin
{
public:

	SLATE_BEGIN_ARGS(SGraphPinCollisionProfile) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:

	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;

	void OnSelectionChanged(TSharedPtr<FName> NameItem, ESelectInfo::Type SelectInfo);
	void OnComboBoxOpening();

	TSharedPtr<FName> GetSelectedName() const;

	void SetPropertyWithName(const FName& Name);
	void GetPropertyAsName(FName& OutName) const;

protected:

	TArray<TSharedPtr<FName>> NameList;
	TSharedPtr<SNameComboBox> NameComboBox;
};
