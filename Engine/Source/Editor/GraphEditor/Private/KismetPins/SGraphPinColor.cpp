// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinColor.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/Engine.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "SGraphNode.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

class SWidget;
struct FGeometry;


void SGraphPinColor::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinColor::GetDefaultValueWidget()
{
	return SAssignNew(DefaultValueWidget, SBorder)
		.BorderImage( FAppStyle::GetBrush("FilledBorder") )
		.Visibility( this, &SGraphPin::GetDefaultValueVisibility )
		.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
		.Padding(1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SColorBlock )
				.Color( this, &SGraphPinColor::GetColor )
				.ShowBackgroundForAlpha(true)
				.OnMouseButtonDown( this, &SGraphPinColor::OnColorBoxClicked )
			]
		];
}

FReply SGraphPinColor::OnColorBoxClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FColorPickerArgs PickerArgs;
		PickerArgs.bIsModal = true;
		PickerArgs.ParentWidget = DefaultValueWidget;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.InitialColor = GetColor();
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SGraphPinColor::OnColorCommitted);
		PickerArgs.bUseAlpha = true;

		OpenColorPicker(PickerArgs);

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FLinearColor SGraphPinColor::GetColor() const
{
	FString ColorString = GraphPinObj->GetDefaultAsString();
	FLinearColor PinColor;

	// Ensure value is sensible
	if (!PinColor.InitFromString(ColorString)) 
	{
		PinColor = FLinearColor::Black;
	}
	return PinColor;
}

void SGraphPinColor::OnColorCommitted(FLinearColor InColor)
{
	if(GraphPinObj->IsPendingKill())
	{
		return;
	}

	// Update pin object
	FString ColorString = InColor.ToString();

	if(GraphPinObj->GetDefaultAsString() != ColorString)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeColorPinValue", "Change Color Pin Value" ) );
		GraphPinObj->Modify();
		
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ColorString);

		if (OwnerNodePtr.IsValid())
		{
			OwnerNodePtr.Pin()->UpdateGraphNode();
		}
	}
}
