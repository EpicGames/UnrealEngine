// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphPinDataTableRowName.h"

#include "Containers/Array.h"
#include "Engine/DataTable.h"
#include "HAL/PlatformCrt.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class UEdGraphPin;

void SGraphPinDataTableRowName::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj, class UDataTable* InDataTable)
{
	DataTable = InDataTable;
	RefreshNameList();
	SGraphPinNameList::Construct(SGraphPinNameList::FArguments(), InGraphPinObj, NameList);
}

SGraphPinDataTableRowName::SGraphPinDataTableRowName()
{
}

SGraphPinDataTableRowName::~SGraphPinDataTableRowName()
{
}

void SGraphPinDataTableRowName::PreChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info)
{
}

void SGraphPinDataTableRowName::PostChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info)
{
	if (Changed && (Changed == DataTable.Get()) && (FDataTableEditorUtils::EDataTableChangeInfo::RowList == Info))
	{
		RefreshNameList();
	}
}

void SGraphPinDataTableRowName::RefreshNameList()
{
	NameList.Empty();
	if (DataTable.IsValid())
	{
		auto Names = DataTable->GetRowNames();
		for (auto Name : Names)
		{
			TSharedPtr<FName> RowNameItem = MakeShareable(new FName(Name));
			NameList.Add(RowNameItem);
		}
	}
}
