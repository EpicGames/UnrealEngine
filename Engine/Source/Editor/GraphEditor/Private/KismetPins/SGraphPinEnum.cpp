// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinEnum.h"

#include "Containers/BitArray.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SGraphPinComboBox.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;

//Construct combo box using combo button and combo list
void SPinComboBox::Construct( const FArguments& InArgs )
{
	ComboItemList = InArgs._ComboItemList.Get();
	OnSelectionChanged = InArgs._OnSelectionChanged;
	VisibleText = InArgs._VisibleText;
	OnGetDisplayName = InArgs._OnGetDisplayName;
	OnGetTooltip = InArgs._OnGetTooltip;

	this->ChildSlot
	[
		SAssignNew( ComboButton, SComboButton )
		.ContentPadding(3.0f)
		.MenuPlacement(MenuPlacement_BelowAnchor)
		.ButtonContent()
		[
			// Wrap in configurable box to restrain height/width of menu
			SNew(SBox)
			.MinDesiredWidth(150.0f)
			[
				SNew( STextBlock ).ToolTipText(NSLOCTEXT("PinComboBox", "ToolTip", "Select enum values from the list"))
				.Text( this, &SPinComboBox::OnGetVisibleTextInternal )
				.Font( FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ) )
			]
		]
		.MenuContent()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.Padding(0.0f) // No padding here so that the inner box controls matching width with padding
			[
				SNew(SBox)
				.MaxDesiredHeight(450.0f)
				.MinDesiredWidth(this, &SPinComboBox::GetMenuDesiredWidth)
				.Padding(3.0f, 6.0f)
				[
					SAssignNew( ComboList, SComboList )
					.ListItemsSource( &ComboItemList )
					.OnGenerateRow( this, &SPinComboBox::OnGenerateComboWidget )
					.OnSelectionChanged( this, &SPinComboBox::OnSelectionChangedInternal )
				]
			]
		]
	];
}

FOptionalSize SPinComboBox::GetMenuDesiredWidth() const
{
	if (ComboButton)
	{
		return ComboButton->GetDesiredSize().X;
	}
	return FOptionalSize(150.0f);
}

void SPinComboBox::RemoveItemByIndex(int32 InIndexToRemove)
{
	for(int32 Index=0;Index<ComboItemList.Num();Index++)
	{
		if(*ComboItemList[Index] == InIndexToRemove)
		{
			ComboItemList.RemoveAt(Index);
			break;
		}
	}
}

//Function to set the newly selected item
void SPinComboBox::OnSelectionChangedInternal( TSharedPtr<int32> NewSelection, ESelectInfo::Type SelectInfo )
{
	if (CurrentSelection.Pin() != NewSelection)
	{
		CurrentSelection = NewSelection;
		// Close the popup as soon as the selection changes
		ComboButton->SetIsOpen( false );

		OnSelectionChanged.ExecuteIfBound( NewSelection, SelectInfo );
	}
}

// Function to create each row of the combo widget
TSharedRef<ITableRow> SPinComboBox::OnGenerateComboWidget( TSharedPtr<int32> InComboIndex, const TSharedRef<STableViewBase>& OwnerTable )
{
	int32 RowIndex = *InComboIndex;

	return
		SNew(STableRow< TSharedPtr<int32> >, OwnerTable)
		[
			SNew(SBox)
			.MinDesiredWidth(150.0f)
			.Padding(8.0f, 3.0f)
			[
				SNew(STextBlock)
				.Text( this, &SPinComboBox::GetRowString, RowIndex )
				.ToolTipText( this, &SPinComboBox::GetRowTooltip, RowIndex )
				.Font( FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ) )
			]
		];
}

void SGraphPinEnum::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	ValidEnumValues = InArgs._ValidEnumValues;
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinEnum::GetDefaultValueWidget()
{
	//Get list of enum indexes
	TArray< TSharedPtr<int32> > ComboItems;
	GenerateComboBoxIndexes( ComboItems );

	//Create widget
	return SAssignNew(ComboBox, SPinComboBox)
		.ComboItemList( ComboItems )
		.VisibleText( this, &SGraphPinEnum::OnGetText )
		.OnSelectionChanged( this, &SGraphPinEnum::ComboBoxSelectionChanged )
		.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
		.Visibility( this, &SGraphPin::GetDefaultValueVisibility )
		.OnGetDisplayName(this, &SGraphPinEnum::OnGetFriendlyName)
		.OnGetTooltip(this, &SGraphPinEnum::OnGetTooltip);
}

FText SGraphPinEnum::OnGetFriendlyName(int32 EnumIndex)
{
	if(GraphPinObj->IsPendingKill())
	{
		return FText();
	}

	UEnum* EnumPtr = Cast<UEnum>(GraphPinObj->PinType.PinSubCategoryObject.Get());

	check(EnumPtr);
	check(EnumIndex < EnumPtr->NumEnums());

	FText EnumValueName = EnumPtr->GetDisplayNameTextByIndex(EnumIndex);
	return EnumValueName;
}

FText SGraphPinEnum::OnGetTooltip(int32 EnumIndex)
{
	UEnum* EnumPtr = Cast<UEnum>(GraphPinObj->PinType.PinSubCategoryObject.Get());

	check(EnumPtr);
	check(EnumIndex < EnumPtr->NumEnums());

	FText EnumValueTooltip = EnumPtr->GetToolTipTextByIndex(EnumIndex);
	return EnumValueTooltip;
}

void SGraphPinEnum::ComboBoxSelectionChanged( TSharedPtr<int32> NewSelection, ESelectInfo::Type /*SelectInfo*/ )
{
	UEnum* EnumPtr = Cast<UEnum>(GraphPinObj->PinType.PinSubCategoryObject.Get());
	check(EnumPtr);

	FString EnumSelectionString;
	if (NewSelection.IsValid())
	{
		check(*NewSelection < EnumPtr->NumEnums() - 1);
		EnumSelectionString  = EnumPtr->GetNameStringByIndex(*NewSelection);
	}
	else
	{
		EnumSelectionString = FName(NAME_None).ToString();
	}
	

	if(GraphPinObj->GetDefaultAsString() != EnumSelectionString)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeEnumPinValue", "Change Enum Pin Value" ) );
		GraphPinObj->Modify();

		//Set new selection
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, EnumSelectionString);
	}
}

FString SGraphPinEnum::OnGetText() const 
{
	FString SelectedString = GraphPinObj->GetDefaultAsString();

	UEnum* EnumPtr = Cast<UEnum>(GraphPinObj->PinType.PinSubCategoryObject.Get());
	if (EnumPtr && EnumPtr->NumEnums())
	{
		const int32 MaxIndex = EnumPtr->NumEnums() - 1;
		for (int32 EnumIdx = 0; EnumIdx < MaxIndex; ++EnumIdx)
		{
			// Ignore hidden enum values
			if( !EnumPtr->HasMetaData(TEXT("Hidden"), EnumIdx ) )
			{
				if (SelectedString == EnumPtr->GetNameStringByIndex(EnumIdx))
				{
					FString EnumDisplayName = EnumPtr->GetDisplayNameTextByIndex(EnumIdx).ToString();
					if (EnumDisplayName.Len() == 0)
					{
						return SelectedString;
					}
					else
					{
						return EnumDisplayName;
					}
				}
			}
		}

		if (SelectedString == EnumPtr->GetNameStringByIndex(MaxIndex))
		{
			return TEXT("(INVALID)");
		}

	}
	return SelectedString;
}

void SGraphPinEnum::GenerateComboBoxIndexes( TArray< TSharedPtr<int32> >& OutComboBoxIndexes )
{
	UEnum* EnumPtr = Cast<UEnum>( GraphPinObj->PinType.PinSubCategoryObject.Get() );
	if (EnumPtr)
	{
		//NumEnums() - 1, because the last item in an enum is the _MAX item
		for (int32 EnumIndex = 0; EnumIndex < EnumPtr->NumEnums() - 1; ++EnumIndex)
		{
			// Ignore hidden enum values
			if( !EnumPtr->HasMetaData(TEXT("Hidden"), EnumIndex ) && (ValidEnumValues.IsEmpty() || ValidEnumValues.Contains(EnumPtr->GetNameByIndex(EnumIndex))))
			{
				TSharedPtr<int32> EnumIdxPtr(new int32(EnumIndex));
				OutComboBoxIndexes.Add(EnumIdxPtr);
			}
		}
	}
}
