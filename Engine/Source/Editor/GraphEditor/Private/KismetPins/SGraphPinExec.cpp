// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinExec.h"

#include "HAL/PlatformMath.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SSpacer.h"

class SWidget;
class UEdGraphPin;
struct FSlateBrush;

void SGraphPinExec::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InPin);

	// Call utility function so inheritors can also call it since arguments can't be passed through
	CachePinIcons();
}

void SGraphPinExec::CachePinIcons()
{
	CachedImg_Pin_ConnectedHovered = FAppStyle::GetBrush(TEXT("Graph.ExecPin.ConnectedHovered"));
	CachedImg_Pin_Connected = FAppStyle::GetBrush(TEXT("Graph.ExecPin.Connected"));
	CachedImg_Pin_DisconnectedHovered = FAppStyle::GetBrush(TEXT("Graph.ExecPin.DisconnectedHovered"));
	CachedImg_Pin_Disconnected = FAppStyle::GetBrush(TEXT("Graph.ExecPin.Disconnected"));
}

TSharedRef<SWidget>	SGraphPinExec::GetDefaultValueWidget()
{
	return SNew(SSpacer); // not used for exec pin		
}

const FSlateBrush* SGraphPinExec::GetPinIcon() const
{
	const FSlateBrush* Brush = NULL;

	if (IsConnected())
	{
		Brush = IsHovered() ? CachedImg_Pin_ConnectedHovered : CachedImg_Pin_Connected;
	}
	else
	{
		Brush = IsHovered() ? CachedImg_Pin_DisconnectedHovered : CachedImg_Pin_Disconnected;
	}

	return Brush;
}
