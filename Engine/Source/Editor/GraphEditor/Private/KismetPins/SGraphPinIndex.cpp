// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinIndex.h"

#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "Misc/Attribute.h"
#include "SPinTypeSelector.h"
#include "UObject/UObjectGlobals.h"

class SWidget;

void SGraphPinIndex::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinIndex::GetDefaultValueWidget()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	return SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(Schema, &UEdGraphSchema_K2::GetVariableTypeTree))
		.TargetPinType(this, &SGraphPinIndex::OnGetPinType)
		.OnPinTypeChanged(this, &SGraphPinIndex::OnTypeChanged)
		.Schema(Schema)
		.TypeTreeFilter(ETypeTreeFilter::IndexTypesOnly)
		.IsEnabled(true)
		.bAllowArrays(false);
}

FEdGraphPinType SGraphPinIndex::OnGetPinType() const
{
	return GraphPinObj->PinType;
}

void SGraphPinIndex::OnTypeChanged(const FEdGraphPinType& PinType)
{
	if (GraphPinObj)
	{
		GraphPinObj->Modify();
		GraphPinObj->PinType = PinType;
		// Let the node know that one of its' pins had their pin type changed
		if (UEdGraphNode* OwningNode = GraphPinObj->GetOwningNode())
		{
			OwningNode->PinTypeChanged(GraphPinObj);
			OwningNode->ReconstructNode();
		}
	}
}
