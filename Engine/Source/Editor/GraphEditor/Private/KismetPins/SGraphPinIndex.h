// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphPin.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;

class SGraphPinIndex : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinIndex) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	
	FEdGraphPinType OnGetPinType() const;
	void OnTypeChanged(const FEdGraphPinType& PinType);

protected:
	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface
};
