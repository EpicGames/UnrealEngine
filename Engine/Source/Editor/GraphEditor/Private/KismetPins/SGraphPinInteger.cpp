// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetPins/SGraphPinInteger.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "SGraphPin.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

void SGraphPinInteger::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPinNum<int32>::Construct(SGraphPinNum<int32>::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinInteger::GetDefaultValueWidget()
{
	check(GraphPinObj);

	const FName PinSubCategory = GraphPinObj->PinType.PinSubCategory;
	const UEnum* BitmaskEnum = Cast<UEnum>(GraphPinObj->PinType.PinSubCategoryObject.Get());

	if (PinSubCategory == UEdGraphSchema_K2::PSC_Bitmask)
	{
		struct BitmaskFlagInfo
		{
			int32 Value;
			FText DisplayName;
			FText ToolTipText;
		};

		const int32 BitmaskBitCount = sizeof(int32) << 3;

		TArray<BitmaskFlagInfo> BitmaskFlags;
		BitmaskFlags.Reserve(BitmaskBitCount);

		if(BitmaskEnum)
		{
			const bool bUseEnumValuesAsMaskValues = BitmaskEnum->GetBoolMetaData(FBlueprintMetadata::MD_UseEnumValuesAsMaskValuesInEditor);
			auto AddNewBitmaskFlagLambda = [BitmaskEnum, &BitmaskFlags](int32 InEnumIndex, int32 InFlagValue)
			{
				BitmaskFlagInfo* BitmaskFlag = new(BitmaskFlags) BitmaskFlagInfo();

				BitmaskFlag->Value = InFlagValue;
				BitmaskFlag->DisplayName = BitmaskEnum->GetDisplayNameTextByIndex(InEnumIndex);
				BitmaskFlag->ToolTipText = BitmaskEnum->GetToolTipTextByIndex(InEnumIndex);
				if (BitmaskFlag->ToolTipText.IsEmpty())
				{
					BitmaskFlag->ToolTipText = FText::Format(NSLOCTEXT("GraphEditor", "BitmaskDefaultFlagToolTipText", "Toggle {0} on/off"), BitmaskFlag->DisplayName);
				}
			};

			// Note: This loop doesn't include (BitflagsEnum->NumEnums() - 1) in order to skip the implicit "MAX" value that gets added to the enum type at compile time.
			for(int32 BitmaskEnumIndex = 0; BitmaskEnumIndex < BitmaskEnum->NumEnums() - 1; ++BitmaskEnumIndex)
			{
				const int64 EnumValue = BitmaskEnum->GetValueByIndex(BitmaskEnumIndex);
				const bool bIsHidden = BitmaskEnum->HasMetaData(TEXT("Hidden"), BitmaskEnumIndex);
				if (EnumValue >= 0 && !bIsHidden)
				{
					if (bUseEnumValuesAsMaskValues)
					{
						if (EnumValue < MAX_int32 && FMath::IsPowerOfTwo(EnumValue))
						{
							AddNewBitmaskFlagLambda(BitmaskEnumIndex, static_cast<int32>(EnumValue));
						}
					}
					else if (EnumValue < BitmaskBitCount)
					{
						AddNewBitmaskFlagLambda(BitmaskEnumIndex, 1 << static_cast<int32>(EnumValue));
					}
				}
			}
		}
		else
		{
			for(int BitmaskFlagIndex = 0; BitmaskFlagIndex < BitmaskBitCount; ++BitmaskFlagIndex)
			{
				BitmaskFlagInfo* BitmaskFlag = new(BitmaskFlags) BitmaskFlagInfo();

				BitmaskFlag->Value = static_cast<int32>(1 << BitmaskFlagIndex);
				BitmaskFlag->DisplayName = FText::Format(NSLOCTEXT("GraphEditor", "BitmaskDefaultFlagDisplayName", "Flag {0}"), FText::AsNumber(BitmaskFlagIndex + 1));
				BitmaskFlag->ToolTipText = FText::Format(NSLOCTEXT("GraphEditor", "BitmaskDefaultFlagToolTipText", "Toggle {0} on/off"), BitmaskFlag->DisplayName);
			}
		}

		const auto& GetComboButtonText = [this, BitmaskFlags]() -> FText
		{
			int32 BitmaskValue = FCString::Atoi(*GraphPinObj->GetDefaultAsString());
			if(BitmaskValue != 0)
			{
				if(BitmaskValue & (BitmaskValue - 1))
				{
					return NSLOCTEXT("GraphEditor", "BitmaskButtonContentMultipleBitsSet", "(Multiple)");
				}
				else
				{
					for(int i = 0; i < BitmaskFlags.Num(); ++i)
					{
						if(BitmaskValue & BitmaskFlags[i].Value)
						{
							return BitmaskFlags[i].DisplayName;
						}
					}
				}
			}

			return NSLOCTEXT("GraphEditor", "BitmaskButtonContentNoFlagsSet", "(No Flags)");
		};

		return SNew(SComboButton)
			.ContentPadding(3.0f)
			.MenuPlacement(MenuPlacement_BelowAnchor)
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
			.ButtonContent()
			[
				// Wrap in configurable box to restrain height/width of menu
				SNew(SBox)
				.MinDesiredWidth(84.0f)
				[
					SNew(STextBlock)
					.Text_Lambda(GetComboButtonText)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
			]
			.OnGetMenuContent_Lambda([this, BitmaskFlags]()
			{
				FMenuBuilder MenuBuilder(false, nullptr);

				for(int i = 0; i < BitmaskFlags.Num(); ++i)
				{
					MenuBuilder.AddMenuEntry(
						BitmaskFlags[i].DisplayName,
						BitmaskFlags[i].ToolTipText,
						FSlateIcon(),
						FUIAction
						(
							FExecuteAction::CreateLambda([this, BitmaskFlags, i]()
							{
								const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangePinValue", "Change Pin Value"));
								const int32 CurValue = FCString::Atoi(*GraphPinObj->GetDefaultAsString());
								GraphPinObj->Modify();
								GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, FString::FromInt(CurValue ^ BitmaskFlags[i].Value));
							}),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([this, BitmaskFlags, i]() -> bool
							{
								const int32 CurValue = FCString::Atoi(*GraphPinObj->GetDefaultAsString());
								return (CurValue & BitmaskFlags[i].Value) != 0;
							})
						),
						NAME_None,
						EUserInterfaceActionType::Check);
				}

				return MenuBuilder.MakeWidget();
			});
	}
	
	return SGraphPinNum<int32>::GetDefaultValueWidget();
}
