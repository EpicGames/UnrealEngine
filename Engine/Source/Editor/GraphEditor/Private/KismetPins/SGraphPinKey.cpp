// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinKey.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "SKeySelector.h"
#include "ScopedTransaction.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class SWidget;

void SGraphPinKey::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	if (InGraphPinObj->bOrphanedPin)
	{
		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
		return;
	}

	SelectedKey = FKey(*InGraphPinObj->GetDefaultAsString());

	InGraphPinObj->AutogeneratedDefaultValue = "None";
	if (SelectedKey.GetFName() == NAME_None)
	{
		InGraphPinObj->GetSchema()->ResetPinToAutogeneratedDefaultValue(InGraphPinObj, false);
		SelectedKey = FKey(*InGraphPinObj->GetDefaultAsString());
	}

	if (InGraphPinObj->Direction == EEdGraphPinDirection::EGPD_Input)
	{
		// None is a valid key
		if (SelectedKey.GetFName() == NAME_None)
		{
			SelectedKey = EKeys::Invalid;
		}
		else if (!SelectedKey.IsValid())
		{
			// Ensure first valid key is always set by default
			TArray<FKey> KeyList;
			EKeys::GetAllKeys(KeyList);
			SelectedKey = KeyList[0];
			InGraphPinObj->GetSchema()->TrySetDefaultValue(*InGraphPinObj, SelectedKey.ToString());
		}
	}

	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinKey::GetDefaultValueWidget()
{
	return SNew(SKeySelector)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		.CurrentKey(this, &SGraphPinKey::GetCurrentKey)
		.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
		.OnKeyChanged(this, &SGraphPinKey::OnKeyChanged);
}

TOptional<FKey> SGraphPinKey::GetCurrentKey() const
{
	return SelectedKey;
}

void SGraphPinKey::OnKeyChanged(TSharedPtr<FKey> InSelectedKey)
{
	if(GraphPinObj->IsPendingKill())
	{
		return;
	}

	if(SelectedKey != *InSelectedKey.Get())
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeKeyPinValue", "Change Key Pin Value" ) );
		GraphPinObj->Modify();

		SelectedKey = *InSelectedKey.Get();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, SelectedKey.ToString());
	}
}
