// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"
#include "Misc/Optional.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;
class UEdGraphPin;

class SGraphPinKey : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinKey) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:

	/**
	 *	Function to create class specific widget.
	 *
	 *	@return Reference to the newly created widget object
	 */
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;

private:

	/** Gets the current Key being edited. */
	TOptional<FKey> GetCurrentKey() const;

	/** Updates the pin default when a new key is selected. */
	void OnKeyChanged(TSharedPtr<FKey> SelectedKey);

	FKey SelectedKey;
};
