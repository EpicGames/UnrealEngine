// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphPinNameList.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "SNameComboBox.h"
#include "ScopedTransaction.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class SWidget;

void SGraphPinNameList::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj, const TArray<TSharedPtr<FName>>& InNameList)
{
	NameList = InNameList;
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinNameList::GetDefaultValueWidget()
{
	TSharedPtr<FName> CurrentlySelectedName;

	if (GraphPinObj)
	{
		// Preserve previous selection, or set to first in list
		FName PreviousSelection = FName(*GraphPinObj->GetDefaultAsString());
		for (TSharedPtr<FName> ListNamePtr : NameList)
		{
			if (PreviousSelection == *ListNamePtr.Get())
			{
				CurrentlySelectedName = ListNamePtr;
				break;
			}
		}
	}
	
	// Create widget
	return SAssignNew(ComboBox, SNameComboBox)
		.ContentPadding(FMargin(6.0f, 2.0f))
		.OptionsSource(&NameList)
		.InitiallySelectedItem(CurrentlySelectedName)
		.OnSelectionChanged(this, &SGraphPinNameList::ComboBoxSelectionChanged)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		;
}

void SGraphPinNameList::ComboBoxSelectionChanged(TSharedPtr<FName> NameItem, ESelectInfo::Type SelectInfo )
{
	FName Name = NameItem.IsValid() ? *NameItem : NAME_None;
	if (auto Schema = (GraphPinObj ? GraphPinObj->GetSchema() : NULL))
	{
		FString NameAsString = Name.ToString();
		if(GraphPinObj->GetDefaultAsString() != NameAsString)
		{
			const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeNameListPinValue", "Change Name List Pin Value" ) );
			GraphPinObj->Modify();

			Schema->TrySetDefaultValue(*GraphPinObj, NameAsString);
		}
	}
}

