// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetPins/SGraphPinObject.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Set.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IContentBrowserSingleton.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SGraphPinObject"

namespace GraphPinObjectDefs
{
	// Active Combo pin alpha
	static const float ActiveComboAlpha = 1.f;
	// InActive Combo pin alpha
	static const float InActiveComboAlpha = 0.6f;
	// Active foreground pin alpha
	static const float ActivePinForegroundAlpha = 1.f;
	// InActive foreground pin alpha
	static const float InactivePinForegroundAlpha = 0.15f;
	// Active background pin alpha
	static const float ActivePinBackgroundAlpha = 0.8f;
	// InActive background pin alpha
	static const float InactivePinBackgroundAlpha = 0.4f;
};

void SGraphPinObject::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinObject::GetDefaultValueWidget()
{
	if (GraphPinObj == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	const UEdGraphSchema* Schema = GraphPinObj->GetSchema();

	if (Schema == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	if(ShouldDisplayAsSelfPin())
	{
		return SNew(SEditableTextBox)
			.Style( FAppStyle::Get(), "Graph.EditableTextBox" )
			.Text( this, &SGraphPinObject::GetValue )
			.SelectAllTextWhenFocused(false)
			.Visibility( this, &SGraphPinObject::GetDefaultValueVisibility )
			.IsReadOnly( true )
			.ForegroundColor( FSlateColor::UseForeground() );
	}
	// Don't show literal buttons for component type objects
	if (Schema->ShouldShowAssetPickerForPin(GraphPinObj))
	{
		return
			SNew(SHorizontalBox)
			.Visibility( this, &SGraphPin::GetDefaultValueVisibility )
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f,0.0f)
			.MaxWidth(100.0f)
			[
				SAssignNew(AssetPickerAnchor, SComboButton)
				.ButtonStyle( FAppStyle::Get(), "PropertyEditor.AssetComboStyle" )
				.ForegroundColor( this, &SGraphPinObject::OnGetComboForeground)
				.ContentPadding( FMargin(2.0f,2.0f,2.0f,1.0f) )
				.ButtonColorAndOpacity( this, &SGraphPinObject::OnGetWidgetBackground )
				.MenuPlacement(MenuPlacement_BelowAnchor)
				.IsEnabled(this, &SGraphPin::IsEditingEnabled)
				.ButtonContent()
				[
					SNew(STextBlock)
					.ColorAndOpacity( this, &SGraphPinObject::OnGetComboForeground )
					.TextStyle( FAppStyle::Get(), "PropertyEditor.AssetClass" )
					.Font( FAppStyle::GetFontStyle( "PropertyWindow.NormalFont" ) )
					.Text( this, &SGraphPinObject::OnGetComboTextValue )
					.ToolTipText( this, &SGraphPinObject::GetObjectToolTip )
				]
				.OnGetMenuContent(this, &SGraphPinObject::GenerateAssetPicker)
			]
			// Use button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f,0.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(UseButton, SButton)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ButtonColorAndOpacity( this, &SGraphPinObject::OnGetWidgetBackground )
				.OnClicked( GetOnUseButtonDelegate() )
				.ContentPadding(1.f)
				.ToolTipText(NSLOCTEXT("GraphEditor", "ObjectGraphPin_Use_Tooltip", "Use asset browser selection"))
				.IsEnabled(this, &SGraphPin::IsEditingEnabled)
				[
					SNew(SImage)
					.ColorAndOpacity( this, &SGraphPinObject::OnGetWidgetForeground )
					.Image( FAppStyle::GetBrush(TEXT("Icons.CircleArrowLeft")) )
				]
			]
			// Browse button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f,0.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(BrowseButton, SButton)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ButtonColorAndOpacity( this, &SGraphPinObject::OnGetWidgetBackground )
				.OnClicked( GetOnBrowseButtonDelegate() )
				.ContentPadding(0.0f)
				.ToolTipText(NSLOCTEXT("GraphEditor", "ObjectGraphPin_Browse_Tooltip", "Browse"))
				[
					SNew(SImage)
					.ColorAndOpacity( this, &SGraphPinObject::OnGetWidgetForeground )
					.Image( FAppStyle::GetBrush(TEXT("Icons.Search")) )
				]
			];
	}

	return SNullWidget::NullWidget;
}

FOnClicked SGraphPinObject::GetOnUseButtonDelegate()
{
	return FOnClicked::CreateSP( this, &SGraphPinObject::OnClickUse );
}

FOnClicked SGraphPinObject::GetOnBrowseButtonDelegate()
{
	return FOnClicked::CreateSP( this, &SGraphPinObject::OnClickBrowse );
}

FText SGraphPinObject::GetObjectToolTip() const
{
	return GetValue();
}

FString SGraphPinObject::GetObjectToolTipAsString() const
{
	return GetValue().ToString();
}

FReply SGraphPinObject::OnClickUse()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	UClass* ObjectClass = Cast<UClass>(GraphPinObj->PinType.PinSubCategoryObject.Get());
	if (ObjectClass != NULL)
	{
		UObject* SelectedObject = GEditor->GetSelectedObjects()->GetTop(ObjectClass);
		if(SelectedObject != NULL)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeObjectPinValue", "Change Object Pin Value"));
			GraphPinObj->Modify();

			GraphPinObj->GetSchema()->TrySetDefaultObject(*GraphPinObj, SelectedObject);
		}
	}

	return FReply::Handled();
}

FReply SGraphPinObject::OnClickBrowse()
{
	const FAssetData& AssetData = GetAssetData(false);
	if (AssetData.IsValid())
	{
		TArray<FAssetData> Objects;
		Objects.Add(AssetData);

		GEditor->SyncBrowserToObjects(Objects);
	}
	return FReply::Handled();
}

TSharedRef<SWidget> SGraphPinObject::GenerateAssetPicker()
{
	// This class and its children are the classes that we can show objects for
	UClass* AllowedClass = Cast<UClass>(GraphPinObj->PinType.PinSubCategoryObject.Get());

	if (AllowedClass == NULL)
	{
		AllowedClass = UObject::StaticClass();
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(AllowedClass->GetClassPathName());
	AssetPickerConfig.bAllowNullSelection = true;
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SGraphPinObject::OnAssetSelectedFromPicker);
	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SGraphPinObject::OnAssetEnterPressedInPicker);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.InitialAssetSelection = CachedAssetData.GetAsset();

	UObject* OwningAsset = GraphPinObj->GetOwningNode();
	while (OwningAsset && !OwningAsset->IsAsset())
	{
		OwningAsset = OwningAsset->GetOuter();
	}	
	if (OwningAsset)
	{
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(OwningAsset));
	}

	// Check with the node to see if there is any "AllowClasses" or "DisallowedClasses" metadata for the pin
	FString AllowedClassesFilterString = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, FName(TEXT("AllowedClasses")));
	if( !AllowedClassesFilterString.IsEmpty() )
	{
		// Clear out the allowed class names and have the pin's metadata override.
		AssetPickerConfig.Filter.ClassPaths.Empty();

		// Parse and add the classes from the metadata
		TArray<FString> AllowedClassesFilterNames;
		AllowedClassesFilterString.ParseIntoArrayWS(AllowedClassesFilterNames, TEXT(","), true);
		for(const FString& AllowedClassesFilterName : AllowedClassesFilterNames)
		{
			ensureAlwaysMsgf(!FPackageName::IsShortPackageName(AllowedClassesFilterName), TEXT("Short class names are not supported as AllowedClasses on pin \"%s\": class \"%s\""), *GraphPinObj->PinName.ToString(), *AllowedClassesFilterName);
			AssetPickerConfig.Filter.ClassPaths.Add(FTopLevelAssetPath(AllowedClassesFilterName));
		}
	}

	FString DisallowedClassesFilterString = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, FName(TEXT("DisallowedClasses")));
	if(!DisallowedClassesFilterString.IsEmpty())
	{
		TArray<FString> DisallowedClassesFilterNames;
		DisallowedClassesFilterString.ParseIntoArrayWS(DisallowedClassesFilterNames, TEXT(","), true);
		for(const FString& DisallowedClassesFilterName : DisallowedClassesFilterNames)
		{
			ensureAlwaysMsgf(!FPackageName::IsShortPackageName(DisallowedClassesFilterName), TEXT("Short class names are not supported as DisallowedClasses on pin \"%s\": class \"%s\""), *GraphPinObj->PinName.ToString(), *DisallowedClassesFilterName);
			AssetPickerConfig.Filter.RecursiveClassPathsExclusionSet.Add(FTopLevelAssetPath(DisallowedClassesFilterName));
		}
	}

	return
		SNew(SBox)
		.HeightOverride(300.0f)
		.WidthOverride(300.0f)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("Menu.Background") )
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];
}

void SGraphPinObject::OnAssetSelectedFromPicker(const struct FAssetData& AssetData)
{
	if(GraphPinObj->IsPendingKill())
	{
		return;
	}

	const FAssetData& CurrentAssetData = GetAssetData(true);
	if(CurrentAssetData != AssetData)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeObjectPinValue", "Change Object Pin Value" ) );
		GraphPinObj->Modify();

		// Close the asset picker
		AssetPickerAnchor->SetIsOpen(false);

		// Set the object found from the asset picker
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, AssetData.GetObjectPathString());
	}
}

void SGraphPinObject::OnAssetEnterPressedInPicker(const TArray<FAssetData>& InSelectedAssets)
{
	if(InSelectedAssets.Num() > 0)
	{
		OnAssetSelectedFromPicker(InSelectedAssets[0]);
	}
}

bool SGraphPinObject::ShouldDisplayAsSelfPin() const
{
	if (AllowSelfPinWidget() && GraphPinObj)
	{
		if (GraphPinObj->GetSchema()->IsSelfPin(*GraphPinObj))
		{
			return true;
		}

		// Also check function/node metadata
		const FString DefaultToSelfString = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, FBlueprintMetadata::MD_DefaultToSelf);
		if (DefaultToSelfString == GraphPinObj->PinName.ToString())
		{
			return true;
		}
	}

	return false;
}


FText SGraphPinObject::GetValue() const
{
	const FAssetData& CurrentAssetData = GetAssetData(true);
	FText Value;
	if (CurrentAssetData.IsValid())
	{
		Value = FText::FromString(CurrentAssetData.GetFullName());
	}
	else
	{
		if (GraphPinObj->GetSchema()->IsSelfPin(*GraphPinObj))
		{
			Value =  FText::FromName(GraphPinObj->PinName);
		}
		else if (ShouldDisplayAsSelfPin())
		{
			Value = FText::FromName(UEdGraphSchema_K2::PN_Self);
		}
		else
		{
			Value = FText::GetEmpty();
		}
	}
	return Value;
}

FText SGraphPinObject::GetObjectName() const
{
	FText Value = FText::GetEmpty();
	
	if (GraphPinObj != NULL)
	{
		const FAssetData& CurrentAssetData = GetAssetData(true);
		if (CurrentAssetData.IsValid())
		{
			Value = FText::FromString(CurrentAssetData.AssetName.ToString());
			int32 StringLen = Value.ToString().Len();

			//If string is too long, then truncate (eg. "abcdefgijklmnopq" is converted as "abcd...nopq")
			const int32 MaxAllowedLength = 16;
			if (StringLen > MaxAllowedLength)
			{
				//Take first 4 characters
				FString TruncatedStr(Value.ToString().Left(4));
				TruncatedStr += FString( TEXT("..."));
				
				//Take last 4 characters
				TruncatedStr += Value.ToString().Right(4);
				Value = FText::FromString(TruncatedStr);
			}
		}
	}
	return Value;
}

FText SGraphPinObject::GetDefaultComboText() const
{
	return LOCTEXT( "DefaultComboText", "Select Asset" );
}

FText SGraphPinObject::OnGetComboTextValue() const
{
	FText Value = GetDefaultComboText();
	
	if (GraphPinObj != nullptr)
	{
		const FAssetData& CurrentAssetData = GetAssetData(true);

		UObject* DefaultObject = GraphPinObj->DefaultObject;
		if (UField* Field = Cast<UField>(DefaultObject))
		{
			Value = Field->GetDisplayNameText();
		}
		else if (CurrentAssetData.IsValid())
		{
			Value = FText::FromString(CurrentAssetData.AssetName.ToString());
		}
	}
	return Value;
}

FSlateColor SGraphPinObject::OnGetComboForeground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? GraphPinObjectDefs::ActiveComboAlpha : GraphPinObjectDefs::InActiveComboAlpha;
	return FSlateColor( FLinearColor( 1.f, 1.f, 1.f, Alpha ) );
}

FSlateColor SGraphPinObject::OnGetWidgetForeground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? GraphPinObjectDefs::ActivePinForegroundAlpha : GraphPinObjectDefs::InactivePinForegroundAlpha;
	return FSlateColor( FLinearColor( 1.f, 1.f, 1.f, Alpha ) );
}

FSlateColor SGraphPinObject::OnGetWidgetBackground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? GraphPinObjectDefs::ActivePinBackgroundAlpha : GraphPinObjectDefs::InactivePinBackgroundAlpha;
	return FSlateColor( FLinearColor( 1.f, 1.f, 1.f, Alpha ) );
}

const FAssetData& SGraphPinObject::GetAssetData(bool bRuntimePath) const
{
	// For normal assets, the editor and runtime path are the same
	if (GraphPinObj->DefaultObject)
	{
		if (!GraphPinObj->DefaultObject->GetPathName().Equals(CachedAssetData.GetObjectPathString(), ESearchCase::CaseSensitive))
		{
			// This always uses the exact object pointed at
			CachedAssetData = FAssetData(GraphPinObj->DefaultObject, true);
		}
	}
	else if (!GraphPinObj->DefaultValue.IsEmpty())
	{
		FSoftObjectPath ObjectPath = FSoftObjectPath(GraphPinObj->DefaultValue);
		if (ObjectPath != CachedAssetData.GetSoftObjectPath())
		{
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			CachedAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(ObjectPath);

			if (!CachedAssetData.IsValid())
			{
				FString PackageName = FPackageName::ObjectPathToPackageName(GraphPinObj->DefaultValue);
				FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
				FString ObjectName = FPackageName::ObjectPathToObjectName(GraphPinObj->DefaultValue);

				// Fake one
				CachedAssetData = FAssetData(FName(*PackageName), FName(*PackagePath), FName(*ObjectName), UObject::StaticClass()->GetClassPathName());
			}
		}
	}
	else
	{
		if (CachedAssetData.IsValid())
		{
			CachedAssetData = FAssetData();
		}
	}

	return CachedAssetData;
}

#undef LOCTEXT_NAMESPACE
