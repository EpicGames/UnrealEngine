// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinString.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "GenericPlatform/GenericApplication.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "PropertyEditorUtils.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

class SWidget;

void SGraphPinString::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinString::GetDefaultValueWidget()
{
	const TSharedPtr<SWidget> DropdownWidget = TryBuildComboBoxWidget();
	if (DropdownWidget.Get())
	{
		return DropdownWidget.ToSharedRef();
	}

	// Only allow actual string pins to be multi-line
	// Other text based pins (such as names and numbers) should be single-line only
	const bool bIsMultiLine = GraphPinObj->PinType.PinCategory == UEdGraphSchema_K2::PC_String;

	if (bIsMultiLine)
	{
		return SNew(SBox)
			.MinDesiredWidth(18.0f)
			.MaxDesiredHeight(200.0f)
			[
				SNew(SMultiLineEditableTextBox)
				.Style(FAppStyle::Get(), "Graph.EditableTextBox")
				.Text(this, &SGraphPinString::GetTypeInValue)
				.SelectAllTextWhenFocused(true)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.IsReadOnly(this, &SGraphPinString::GetDefaultValueIsReadOnly)
				.OnTextCommitted(this, &SGraphPinString::SetTypeInValue)
				.ForegroundColor(FSlateColor::UseForeground())
				.WrapTextAt(400.0f)
				.ModiferKeyForNewLine(EModifierKey::Shift)
			];
	}
	else
	{
		return SNew(SBox)
			.MinDesiredWidth(18.0f)
			.MaxDesiredWidth(400.0f)
			[
				SNew(SEditableTextBox)
					.Style(FAppStyle::Get(), "Graph.EditableTextBox")
					.Text(this, &SGraphPinString::GetTypeInValue)
					.SelectAllTextWhenFocused(true)
					.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
					.IsReadOnly(this, &SGraphPinString::GetDefaultValueIsReadOnly)
					.OnTextCommitted(this, &SGraphPinString::SetTypeInValue)
					.ForegroundColor(FSlateColor::UseForeground())
			];
	}
}

FText SGraphPinString::GetTypeInValue() const
{
	return FText::FromString( GraphPinObj->GetDefaultAsString() );
}

void SGraphPinString::SetTypeInValue(const FText& NewTypeInValue, ETextCommit::Type /*CommitInfo*/)
{
	if(GraphPinObj->IsPendingKill())
	{
		return;
	}

	if(!GraphPinObj->GetDefaultAsString().Equals(NewTypeInValue.ToString()))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeStringPinValue", "Change String Pin Value" ) );
		GraphPinObj->Modify();

		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, NewTypeInValue.ToString());
	}
}

bool SGraphPinString::GetDefaultValueIsReadOnly() const
{
	return GraphPinObj->bDefaultValueIsReadOnly;
}

TSharedRef<SWidget> SGraphPinString::GenerateComboBoxEntry(const TSharedPtr<FString> Value)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*Value))
		.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
}

void SGraphPinString::HandleComboBoxSelectionChanged(const TSharedPtr<FString> Value,
	const ESelectInfo::Type InSelectInfo)
{
	if (GraphPinObj->IsPendingKill())
	{
		return;
	}

	if (!GraphPinObj->GetDefaultAsString().Equals(*Value))
	{
		const FScopedTransaction Transaction(
			NSLOCTEXT("GraphEditor", "ChangeStringPinValue", "Change String Pin Value")
		);

		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, *Value);
	}
}

TSharedPtr<SWidget> SGraphPinString::TryBuildComboBoxWidget()
{
	FString GetOptionsFunctionName = GraphPinObj->GetOwningNode()->GetPinMetaData(
		GraphPinObj->PinName,
		FBlueprintMetadata::MD_GetOptions
	);

	if (GetOptionsFunctionName.IsEmpty())
	{
		return nullptr;
	}

	ComboBoxOptions.Empty();

	TArray<UObject*> PropertyContainers;

	const UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(GraphPinObj->GetOwningNode());
	if (CallFunctionNode)
	{
		const UClass* FunctionParentClass = CallFunctionNode->FunctionReference.GetMemberParentClass();
		if (!FunctionParentClass)
		{
			// If we don't have a parent class, then we have to be in self context
			if (!CallFunctionNode->FunctionReference.IsSelfContext())
			{
				return nullptr;
			}

			// If we're in self context, let's get the Blueprint we're residing in
			const UBlueprint* Blueprint = GraphPinObj->GetOwningNode()->GetTypedOuter<UBlueprint>();
			if (!Blueprint || !Blueprint->GeneratedClass)
			{
				return nullptr;
			}

			// Work with the blueprint class the node is in
			FunctionParentClass = Blueprint->GeneratedClass;
		}

		PropertyContainers.Add(FunctionParentClass->GetDefaultObject());
	}

	PropertyEditorUtils::GetPropertyOptions(PropertyContainers, GetOptionsFunctionName, ComboBoxOptions);

	if(ComboBoxOptions.Num() == 0)
	{
		return nullptr;
	}

	const FString DefaultString = GraphPinObj->GetDefaultAsString();
	const TSharedPtr<FString>* DefaultValue = ComboBoxOptions.FindByPredicate([&DefaultString](
		const TSharedPtr<FString>& Value)
	{
		return Value.IsValid() ? *Value == DefaultString : false;
	});

	return SNew(SBox)
		.MinDesiredWidth(18.0f)
		.MaxDesiredWidth(400.0f)
		[
			SNew(SSearchableComboBox)
				.OptionsSource(&ComboBoxOptions)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
				.OnGenerateWidget(this, &SGraphPinString::GenerateComboBoxEntry)
				.OnSelectionChanged(this, &SGraphPinString::HandleComboBoxSelectionChanged)
				.ContentPadding(3.0f)
				.InitiallySelectedItem(DefaultValue ? *DefaultValue : nullptr)
				.Content()
				[
					SNew(STextBlock)
						.Text(this, &SGraphPinString::GetTypeInValue)
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				]
		];
}
