// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinStruct.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SGraphPin.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SlotBase.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "Styling/AppStyle.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "SGraphPinStruct"

/////////////////////////////////////////////////////
// SGraphPinStruct

void SGraphPinStruct::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

FReply SGraphPinStruct::OnClickUse()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	UObject* SelectedObject = GEditor->GetSelectedObjects()->GetTop(UScriptStruct::StaticClass());
	if (SelectedObject)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeStructPinValue", "Change Struct Pin Value"));
		GraphPinObj->Modify();

		GraphPinObj->GetSchema()->TrySetDefaultObject(*GraphPinObj, SelectedObject);
	}

	return FReply::Handled();
}

class FGraphPinStructFilter : public IStructViewerFilter
{
public:
	/** The meta struct for the property that classes must be a child-of. */
	const UScriptStruct* MetaStruct = nullptr;

	// TODO: Have a flag controlling whether we allow UserDefinedStructs, even when a MetaClass is set (as they cannot support inheritance, but may still be allowed (eg, data tables))?

	virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
	{
		if (InStruct->IsA<UUserDefinedStruct>())
		{
			// User Defined Structs don't support inheritance, so only include them if we have don't a MetaStruct set
			return MetaStruct == nullptr;
		}

		// Don't show any hidden structs
		static const FName NAME_HiddenMetaTag = "Hidden";
		if (InStruct->HasMetaData(NAME_HiddenMetaTag))
		{
			return false;
		}

		// Query the native struct to see if it has the correct parent type (if any)
		return !MetaStruct || InStruct->IsChildOf(MetaStruct);
	}

	virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
	{
		// User Defined Structs don't support inheritance, so only include them if we have don't a MetaStruct set
		return MetaStruct == nullptr;
	}
};

TSharedRef<SWidget> SGraphPinStruct::GenerateAssetPicker()
{
	FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

	// Fill in options
	FStructViewerInitializationOptions Options;
	Options.Mode = EStructViewerMode::StructPicker;
	Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
	Options.bShowNoneOption = true;

	const UScriptStruct* MetaStruct = nullptr;
	static const FName NAME_MetaStructMetaTag = "MetaStruct";
	const FString MetaStructName = GraphPinObj->GetOwningNode()->GetPinMetaData(GraphPinObj->PinName, NAME_MetaStructMetaTag);
	if (!MetaStructName.IsEmpty())
	{
		MetaStruct = UClass::TryFindTypeSlow<UScriptStruct>(MetaStructName, EFindFirstObjectOptions::EnsureIfAmbiguous);
		if (!MetaStruct)
		{
			MetaStruct = LoadObject<UScriptStruct>(nullptr, *MetaStructName);
		}
	}

	TSharedRef<FGraphPinStructFilter> StructFilter = MakeShared<FGraphPinStructFilter>();
	Options.StructFilter = StructFilter;
	StructFilter->MetaStruct = MetaStruct;

	return
		SNew(SBox)
		.WidthOverride(280.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.MaxHeight(500.0f)
			[ 
				SNew(SBorder)
				.Padding(4.0f)
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				[
					StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateSP(this, &SGraphPinStruct::OnPickedNewStruct))
				]
			]			
		];
}

FOnClicked SGraphPinStruct::GetOnUseButtonDelegate()
{
	return FOnClicked::CreateSP(this, &SGraphPinStruct::OnClickUse);
}

void SGraphPinStruct::OnPickedNewStruct(const UScriptStruct* ChosenStruct)
{
	if(GraphPinObj->IsPendingKill())
	{
		return;
	}

	FString NewPath;
	if (ChosenStruct)
	{
		NewPath = ChosenStruct->GetPathName();
	}

	if (GraphPinObj->GetDefaultAsString() != NewPath)
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeStructPinValue", "Change Struct Pin Value" ) );
		GraphPinObj->Modify();

		AssetPickerAnchor->SetIsOpen(false);
		GraphPinObj->GetSchema()->TrySetDefaultObject(*GraphPinObj, const_cast<UScriptStruct*>(ChosenStruct));
	}
}

FText SGraphPinStruct::GetDefaultComboText() const
{ 
	return LOCTEXT("DefaultComboText", "Select Struct");
}

#undef LOCTEXT_NAMESPACE
