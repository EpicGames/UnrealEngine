// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "KismetPins/SGraphPinObject.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;
class UEdGraphPin;
class UScriptStruct;

/////////////////////////////////////////////////////
// SGraphPinStruct

class SGraphPinStruct : public SGraphPinObject
{
public:
	SLATE_BEGIN_ARGS(SGraphPinStruct) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	// Called when a new struct was picked via the asset picker
	void OnPickedNewStruct(const UScriptStruct* ChosenStruct);

	//~ Begin SGraphPinObject Interface
	virtual FReply OnClickUse() override;
	virtual bool AllowSelfPinWidget() const override { return false; }
	virtual TSharedRef<SWidget> GenerateAssetPicker() override;
	virtual FText GetDefaultComboText() const override;
	virtual FOnClicked GetOnUseButtonDelegate() override;
	//~ End SGraphPinObject Interface
};
