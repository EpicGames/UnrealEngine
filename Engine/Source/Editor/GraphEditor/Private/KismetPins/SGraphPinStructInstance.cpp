// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetPins/SGraphPinStructInstance.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "IStructureDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/OutputDeviceNull.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/StructOnScope.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SGraphPinStructInstance)

class SWidget;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "SGraphPinStructInstance"

void SGraphPinStructInstance::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	if (InGraphPinObj->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && InGraphPinObj->PinType.PinSubCategoryObject.Get())
	{
		// See if a wrapper struct was passed in to use for automatic UI creation
		UScriptStruct* DataStruct = Cast<UScriptStruct>(InGraphPinObj->PinType.PinSubCategoryObject.Get());
		const UScriptStruct* PinEditableStruct = InArgs._StructEditWrapper;

		if (DataStruct && PinEditableStruct)
		{
			EditWrapperInstance = MakeShared<FStructOnScope>(PinEditableStruct);
		}
	}

	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

void SGraphPinStructInstance::ParseDefaultValueData()
{
	FPinStructEditWrapper* EditInstance = GetEditWrapper();

	if (EditInstance && GraphPinObj)
	{
		// Read Pin Data
		FString DefaultValueString = GraphPinObj->GetDefaultAsString();

		UScriptStruct* StructDefinition = const_cast<UScriptStruct*>(Cast<UScriptStruct>(EditInstance->GetDataScriptStruct()));

		FOutputDeviceNull NullOutput;
		StructDefinition->ImportText(*DefaultValueString, EditInstance->GetDataMemory(), nullptr, PPF_SerializedAsImportText, &NullOutput, GraphPinObj->PinName.ToString());
	}
}

void SGraphPinStructInstance::SaveDefaultValueData()
{
	FPinStructEditWrapper* EditInstance = GetEditWrapper();

	ensureMsgf(EditInstance, TEXT("SaveDefaultValueData must be overriden by any widget that does not use an edit wrapper struct!"));

	if (EditInstance && GraphPinObj)
	{
		// Set Pin Data
		FString ExportText;

		UScriptStruct* StructDefinition = const_cast<UScriptStruct*>(Cast<UScriptStruct>(EditInstance->GetDataScriptStruct()));
		StructDefinition->ExportText(ExportText, EditInstance->GetDataMemory(), EditInstance->GetDataMemory(), nullptr, PPF_SerializedAsImportText, nullptr);

		if (ExportText != GraphPinObj->GetDefaultAsString())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangePinValue", "Change Pin Value"));
			GraphPinObj->Modify();
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ExportText);
			RefreshCachedData();
		}
	}
}

void SGraphPinStructInstance::RefreshCachedData()
{
	FPinStructEditWrapper* EditInstance = GetEditWrapper();

	if (EditInstance)
	{
		CachedDescription = EditInstance->GetPreviewDescription();
	}
}


TSharedRef<SWidget>	SGraphPinStructInstance::GetDefaultValueWidget()
{
	if (GraphPinObj == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	ParseDefaultValueData();

	//Create widget
	return SNew(SVerticalBox)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew( ComboButton, SComboButton )
			.OnGetMenuContent(this, &SGraphPinStructInstance::GetEditContent)
			.ContentPadding( FMargin( 2.0f, 2.0f ) )
			.ButtonContent()
			[
				SNew( STextBlock )
				.Text( LOCTEXT("EditText", "Edit") )
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			GetDescriptionContent()
		];
}

TSharedRef<SWidget> SGraphPinStructInstance::GetEditContent()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	ViewArgs.bHideSelectionTip = true;
	ViewArgs.bLockable = false;
	ViewArgs.bUpdatesFromSelection = false;
	ViewArgs.bShowOptions = false;
	ViewArgs.bShowModifiedPropertiesOption = false;

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(ViewArgs, StructureViewArgs, EditWrapperInstance, LOCTEXT("DefaultValue", "Value"));
	StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &SGraphPinStructInstance::PropertyValueChanged);

	return SNew(SBox)
		.MaxDesiredHeight(350.0f)
		.MinDesiredWidth(350.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
			[
				StructureDetailsView->GetWidget().ToSharedRef()
			]
		];
}

TSharedRef<SWidget> SGraphPinStructInstance::GetDescriptionContent()
{
	RefreshCachedData();

	return SNew(STextBlock)
		.Text(this, &SGraphPinStructInstance::GetCachedDescriptionText)
		.AutoWrapText(true);
}

FText SGraphPinStructInstance::GetCachedDescriptionText() const
{
	return CachedDescription;
}

FPinStructEditWrapper* SGraphPinStructInstance::GetEditWrapper() const
{
	if (EditWrapperInstance.IsValid() && EditWrapperInstance->IsValid())
	{
		return (FPinStructEditWrapper*)EditWrapperInstance->GetStructMemory();
	}
	return nullptr;
}

void SGraphPinStructInstance::PropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	SaveDefaultValueData();
}

#undef LOCTEXT_NAMESPACE
