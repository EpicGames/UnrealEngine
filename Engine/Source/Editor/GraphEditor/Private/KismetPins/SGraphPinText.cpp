// Copyright Epic Games, Inc. All Rights Reserved.


#include "KismetPins/SGraphPinText.h"
#include "ScopedTransaction.h"
#include "STextPropertyEditableTextBox.h"

namespace
{
	/** Allows STextPropertyEditableTextBox to edit a graph pin */
	class FEditableTextGraphPin : public IEditableTextProperty
	{
	public:
		FEditableTextGraphPin(UEdGraphPin* InGraphPinObj)
			: GraphPinObjRef(InGraphPinObj)
		{
		}

		virtual bool IsMultiLineText() const override
		{
			return true;
		}

		virtual bool IsPassword() const override
		{
			return false;
		}

		virtual bool IsReadOnly() const override
		{
			if (const UEdGraphPin* GraphPinObj = GraphPinObjRef.Get())
			{
				return GraphPinObj->bDefaultValueIsReadOnly;
			}

			return false;
		}

		virtual bool IsDefaultValue() const override
		{
			if (const UEdGraphPin* GraphPinObj = GraphPinObjRef.Get())
			{
				FString TextAsString;
				FTextStringHelper::WriteToBuffer(TextAsString, GraphPinObj->DefaultTextValue);
				return TextAsString.Equals(GraphPinObj->AutogeneratedDefaultValue, ESearchCase::CaseSensitive);
			}

			return false;
		}

		virtual FText GetToolTipText() const override
		{
			return FText::GetEmpty();
		}

		virtual int32 GetNumTexts() const override
		{
			return 1;
		}

		virtual FText GetText(const int32 InIndex) const override
		{
			check(InIndex == 0);
			if (const UEdGraphPin* GraphPinObj = GraphPinObjRef.Get())
			{
				return GraphPinObj->DefaultTextValue;
			}

			return FText::GetEmpty();
		}

		virtual void SetText(const int32 InIndex, const FText& InText) override
		{
			check(InIndex == 0);
			if (UEdGraphPin* GraphPinObj = GraphPinObjRef.Get())
			{
				const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeTxtPinValue", "Change Text Pin Value"));
				GraphPinObj->Modify();
				GraphPinObj->GetSchema()->TrySetDefaultText(*GraphPinObj, InText);
			}
		}

		virtual bool IsValidText(const FText& InText, FText& OutErrorMsg) const override
		{
			return true;
		}

#if USE_STABLE_LOCALIZATION_KEYS
		virtual void GetStableTextId(const int32 InIndex, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey) const override
		{
			check(InIndex == 0);
			if (const UEdGraphPin* GraphPinObj = GraphPinObjRef.Get())
			{
				StaticStableTextId(GraphPinObj->GetOwningNodeUnchecked(), InEditAction, InTextSource, InProposedNamespace, InProposedKey, OutStableNamespace, OutStableKey);
			}
		}
#endif // USE_STABLE_LOCALIZATION_KEYS

	private:
		FEdGraphPinReference GraphPinObjRef;
	};
}

void SGraphPinText::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinText::GetDefaultValueWidget()
{
	return SNew(STextPropertyEditableTextBox, MakeShareable(new FEditableTextGraphPin(GraphPinObj)))
		.Style(FAppStyle::Get(), "Graph.EditableTextBox")
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		.ForegroundColor(FSlateColor::UseForeground())
		.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
		.WrapTextAt(400.0f)
		.MinDesiredWidth(18.0f)
		.MaxDesiredHeight(200.0f);
}
