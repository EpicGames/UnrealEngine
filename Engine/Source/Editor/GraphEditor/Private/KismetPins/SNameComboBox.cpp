// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNameComboBox.h"

#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;


void SNameComboBox::Construct( const FArguments& InArgs )
{
	SelectionChanged = InArgs._OnSelectionChanged;
	GetTextLabelForItem = InArgs._OnGetNameLabelForItem;
	Font = InArgs._Font;

	// Then make widget
	this->ChildSlot
	[
		SAssignNew(NameCombo, SComboBox< TSharedPtr<FName> > )
		.ComboBoxStyle(InArgs._ComboBoxStyle)
		.OptionsSource(InArgs._OptionsSource)
		.OnGenerateWidget(this, &SNameComboBox::MakeItemWidget)
		.OnSelectionChanged(this, &SNameComboBox::OnSelectionChanged)
		.OnComboBoxOpening(InArgs._OnComboBoxOpening)
		.InitiallySelectedItem(InArgs._InitiallySelectedItem)
		.ContentPadding(InArgs._ContentPadding)
		[
			SNew(STextBlock)
				.ColorAndOpacity(InArgs._ColorAndOpacity)
				.Text(this, &SNameComboBox::GetSelectedNameLabel)
				.Font(InArgs._Font)
		]
	];
	SelectedItem = NameCombo->GetSelectedItem();
}

FText SNameComboBox::GetItemNameLabel(TSharedPtr<FName> NameItem) const
{
	if (!NameItem.IsValid())
	{
		return FText::GetEmpty();
	}

	return (GetTextLabelForItem.IsBound())
		? FText::FromString(GetTextLabelForItem.Execute(NameItem))
		: FText::FromName(*NameItem);
}

FText SNameComboBox::GetSelectedNameLabel() const
{
	TSharedPtr<FName> StringItem = NameCombo->GetSelectedItem();
	return GetItemNameLabel(StringItem);
}

TSharedRef<SWidget> SNameComboBox::MakeItemWidget( TSharedPtr<FName> NameItem ) 
{
	check( NameItem.IsValid() );

	return SNew(STextBlock)
		.Text(this, &SNameComboBox::GetItemNameLabel, NameItem)
		.Font(Font);
}

void SNameComboBox::OnSelectionChanged (TSharedPtr<FName> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		SelectedItem = Selection;
	}
	SelectionChanged.ExecuteIfBound(Selection, SelectInfo);
}

void SNameComboBox::SetSelectedItem(TSharedPtr<FName> NewSelection)
{
	NameCombo->SetSelectedItem(NewSelection);
}

void SNameComboBox::RefreshOptions()
{
	NameCombo->RefreshOptions();
}

void SNameComboBox::ClearSelection( )
{
	NameCombo->ClearSelection();
}
