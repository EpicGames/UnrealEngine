// Copyright Epic Games, Inc. All Rights Reserved.


#include "NodeFactory.h"

#include "BlueprintConnectionDrawingPolicy.h"
#include "ConnectionDrawingPolicy.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNode_Documentation.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/CollisionProfile.h"
#include "InputCoreTypes.h"
#include "K2Node.h"
#include "K2Node_AddPinInterface.h"
#include "K2Node_CallMaterialParameterCollectionFunction.h"
#include "K2Node_Composite.h"
#include "K2Node_Copy.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_Event.h"
#include "K2Node_FormatText.h"
#include "K2Node_Knot.h"
#include "K2Node_MakeStruct.h"
#include "K2Node_PromotableOperator.h"
#include "K2Node_SpawnActor.h"
#include "K2Node_SpawnActorFromClass.h"
#include "K2Node_Switch.h"
#include "K2Node_Timeline.h"
#include "KismetNodes/SGraphNodeCallParameterCollectionFunction.h"
#include "KismetNodes/SGraphNodeFormatText.h"
#include "KismetNodes/SGraphNodeK2Composite.h"
#include "KismetNodes/SGraphNodeK2Copy.h"
#include "KismetNodes/SGraphNodeK2CreateDelegate.h"
#include "KismetNodes/SGraphNodeK2Default.h"
#include "KismetNodes/SGraphNodeK2Event.h"
#include "KismetNodes/SGraphNodeK2Sequence.h"
#include "KismetNodes/SGraphNodeK2Timeline.h"
#include "KismetNodes/SGraphNodeK2Var.h"
#include "KismetNodes/SGraphNodeMakeStruct.h"
#include "KismetNodes/SGraphNodeSpawnActor.h"
#include "KismetNodes/SGraphNodeSpawnActorFromClass.h"
#include "KismetNodes/SGraphNodeSwitchStatement.h"
#include "KismetPins/SGraphPinBool.h"
#include "KismetPins/SGraphPinClass.h"
#include "KismetPins/SGraphPinCollisionProfile.h"
#include "KismetPins/SGraphPinColor.h"
#include "KismetPins/SGraphPinEnum.h"
#include "KismetPins/SGraphPinExec.h"
#include "KismetPins/SGraphPinIndex.h"
#include "KismetPins/SGraphPinInteger.h"
#include "KismetPins/SGraphPinKey.h"
#include "KismetPins/SGraphPinNum.h"
#include "KismetPins/SGraphPinObject.h"
#include "KismetPins/SGraphPinString.h"
#include "KismetPins/SGraphPinStruct.h"
#include "KismetPins/SGraphPinText.h"
#include "KismetPins/SGraphPinVector.h"
#include "KismetPins/SGraphPinVector2D.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "SGraphNodeComment.h"
#include "SGraphNodeDefault.h"
#include "SGraphNodeDocumentation.h"
#include "SGraphNodeKnot.h"
#include "SGraphNodePromotableOperator.h"
#include "SGraphPin.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

struct FLinearColor;

TSharedPtr<SGraphNode> FNodeFactory::CreateNodeWidget(UEdGraphNode* InNode)
{
	check(InNode != NULL);

	// First give a shot to the node itself
	{
		TSharedPtr<SGraphNode> NodeCreatedResult = InNode->CreateVisualWidget();
		if (NodeCreatedResult.IsValid())
		{
			return NodeCreatedResult;
		}
	}

	// First give a shot to the registered node factories
	for (auto FactoryIt = FEdGraphUtilities::VisualNodeFactories.CreateIterator(); FactoryIt; ++FactoryIt)
	{
		TSharedPtr<FGraphPanelNodeFactory> FactoryPtr = *FactoryIt;
		if (FactoryPtr.IsValid())
		{
			TSharedPtr<SGraphNode> ResultVisualNode = FactoryPtr->CreateNode(InNode);
			if (ResultVisualNode.IsValid())
			{
				return ResultVisualNode;
			}
		}
	}

	if (UK2Node* K2Node = Cast<UK2Node>(InNode))
	{
		if (UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(InNode))
		{
			return SNew(SGraphNodeK2Composite, CompositeNode);
		}
		else if (K2Node->DrawNodeAsVariable())
		{
			return SNew(SGraphNodeK2Var, K2Node);
		}
		else if (UK2Node_Switch* SwitchNode = Cast<UK2Node_Switch>(InNode))
		{
			return SNew(SGraphNodeSwitchStatement, SwitchNode);
		}
		else if(UK2Node_PromotableOperator* PromotableOperator = Cast<UK2Node_PromotableOperator>(InNode))
		{
			return SNew(SGraphNodePromotableOperator, PromotableOperator);
		}
		else if (InNode->GetClass()->ImplementsInterface(UK2Node_AddPinInterface::StaticClass()))
		{
			return SNew(SGraphNodeK2Sequence, CastChecked<UK2Node>(InNode));
		}
		else if (UK2Node_Timeline* TimelineNode = Cast<UK2Node_Timeline>(InNode))
		{
			return SNew(SGraphNodeK2Timeline, TimelineNode);
		}
		else if(UK2Node_SpawnActor* SpawnActorNode = Cast<UK2Node_SpawnActor>(InNode))
		{
			return SNew(SGraphNodeSpawnActor, SpawnActorNode);
		}
		else if(UK2Node_SpawnActorFromClass* SpawnActorNodeFromClass = Cast<UK2Node_SpawnActorFromClass>(InNode))
		{
			return SNew(SGraphNodeSpawnActorFromClass, SpawnActorNodeFromClass);
		}
		else if(UK2Node_CreateDelegate* CreateDelegateNode = Cast<UK2Node_CreateDelegate>(InNode))
		{
			return SNew(SGraphNodeK2CreateDelegate, CreateDelegateNode);
		}
		else if (UK2Node_CallMaterialParameterCollectionFunction* CallFunctionNode = Cast<UK2Node_CallMaterialParameterCollectionFunction>(InNode))
		{
			return SNew(SGraphNodeCallParameterCollectionFunction, CallFunctionNode);
		}
		else if (UK2Node_Event* EventNode = Cast<UK2Node_Event>(InNode))
		{
			return SNew(SGraphNodeK2Event, EventNode);
		}
		else if (UK2Node_FormatText* FormatTextNode = Cast<UK2Node_FormatText>(InNode))
		{
			return SNew(SGraphNodeFormatText, FormatTextNode);
		}
		else if (UK2Node_Knot* Knot = Cast<UK2Node_Knot>(InNode))
		{
			return SNew(SGraphNodeKnot, Knot);
		}
		else if (UK2Node_MakeStruct* MakeStruct = Cast<UK2Node_MakeStruct>(InNode))
		{
			return SNew(SGraphNodeMakeStruct, MakeStruct);
		}
		else if (UK2Node_Copy* CopyNode = Cast<UK2Node_Copy>(InNode))
		{
			return SNew(SGraphNodeK2Copy, CopyNode);
		}
		else
		{
			return SNew(SGraphNodeK2Default, K2Node);
		}
	}
	else if(UEdGraphNode_Documentation* DocNode = Cast<UEdGraphNode_Documentation>(InNode))
	{
		return SNew(SGraphNodeDocumentation, DocNode);
	}
	else if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(InNode))
	{
		return SNew(SGraphNodeComment, CommentNode);
	}
	else
	{
		return SNew(SGraphNodeDefault)
			.GraphNodeObj(InNode);
	}
}

TSharedPtr<SGraphPin> FNodeFactory::CreatePinWidget(UEdGraphPin* InPin)
{
	check(InPin != NULL);

	// First give a shot to the registered pin factories
	for (auto FactoryIt = FEdGraphUtilities::VisualPinFactories.CreateIterator(); FactoryIt; ++FactoryIt)
	{
		TSharedPtr<FGraphPanelPinFactory> FactoryPtr = *FactoryIt;
		if (FactoryPtr.IsValid())
		{
			TSharedPtr<SGraphPin> ResultVisualPin = FactoryPtr->CreatePin(InPin);
			if (ResultVisualPin.IsValid())
			{
				return ResultVisualPin;
			}
		}
	}

	if (const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(InPin->GetSchema()))
	{
		TSharedPtr<SGraphPin> K2PinWidget = CreateK2PinWidget(InPin);
		if(K2PinWidget.IsValid())
		{
			return K2PinWidget;
		}
	}
	
	// If we didn't pick a custom pin widget, use an uncustomized basic pin
	return SNew(SGraphPin, InPin);
}


TSharedPtr<SGraphPin> FNodeFactory::CreateK2PinWidget(UEdGraphPin* InPin)
{
	if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		return SNew(SGraphPinBool, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Text)
	{
		return SNew(SGraphPinText, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
	{
		return SNew(SGraphPinExec, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		const UClass* ObjectMetaClass = Cast<UClass>(InPin->PinType.PinSubCategoryObject.Get());
		if (ObjectMetaClass && ObjectMetaClass->IsChildOf<UScriptStruct>())
		{
			return SNew(SGraphPinStruct, InPin);
		}
		else
		{
			return SNew(SGraphPinObject, InPin);
		}
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
	{
		return SNew(SGraphPinObject, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
	{
		return SNew(SGraphPinObject, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class)
	{
		return SNew(SGraphPinClass, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
	{
		return SNew(SGraphPinClass, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Int)
	{
		return SNew(SGraphPinInteger, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Int64)
	{
		return SNew(SGraphPinNum<int64>, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real)
	{
		return SNew(SGraphPinNum<double>, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_String || InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		return SNew(SGraphPinString, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		// If you update this logic you'll probably need to update UEdGraphSchema_K2::ShouldHidePinDefaultValue!
		UScriptStruct* ColorStruct = TBaseStructure<FLinearColor>::Get();
		UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
		UScriptStruct* Vector3fStruct = TVariantStructure<FVector3f>::Get();
		UScriptStruct* Vector2DStruct = TBaseStructure<FVector2D>::Get();
		UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();

		if (InPin->PinType.PinSubCategoryObject == ColorStruct)
		{
			return SNew(SGraphPinColor, InPin);
		}
		else if ((InPin->PinType.PinSubCategoryObject == VectorStruct) || (InPin->PinType.PinSubCategoryObject == Vector3fStruct) || (InPin->PinType.PinSubCategoryObject == RotatorStruct))
		{
			return SNew(SGraphPinVector<double>, InPin);
		}
		else if (InPin->PinType.PinSubCategoryObject == Vector2DStruct)
		{
			return SNew(SGraphPinVector2D<double>, InPin);
		}
		else if (InPin->PinType.PinSubCategoryObject == FKey::StaticStruct())
		{
			return SNew(SGraphPinKey, InPin);
		}
		else if (InPin->PinType.PinSubCategoryObject == FCollisionProfileName::StaticStruct())
		{
			return SNew(SGraphPinCollisionProfile, InPin);
		}
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		// Check for valid enum object reference
		if ((InPin->PinType.PinSubCategoryObject != NULL) && (InPin->PinType.PinSubCategoryObject->IsA(UEnum::StaticClass())))
		{
			return SNew(SGraphPinEnum, InPin);
		}
		else
		{
			return SNew(SGraphPinInteger, InPin);
		}
	}
	else if ((InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard) && (InPin->PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Index))
	{
		return SNew(SGraphPinIndex, InPin);
	}
	else if(InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate)
	{
		return SNew(SGraphPinString, InPin);
	}
	else if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_FieldPath)
	{
	return SNew(SGraphPinString, InPin);
	}

	return nullptr;
}

FConnectionDrawingPolicy* FNodeFactory::CreateConnectionPolicy(const UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
{
	FConnectionDrawingPolicy* ConnectionDrawingPolicy;

	// First give the schema a chance to provide the connection drawing policy
	ConnectionDrawingPolicy = Schema->CreateConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);

	// First give a shot to the registered connection factories
	if (!ConnectionDrawingPolicy)
	{
		for (TSharedPtr<FGraphPanelPinConnectionFactory> FactoryPtr : FEdGraphUtilities::VisualPinConnectionFactories)
		{
			if (FactoryPtr.IsValid())
			{
				ConnectionDrawingPolicy = FactoryPtr->CreateConnectionPolicy(Schema, InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);

				if (ConnectionDrawingPolicy)
				{
					break;
				}
			}
		}
	}

	// If neither the schema nor the factory provides a policy, try the hardcoded ones
	//@TODO: Fold all of this code into registered factories for the various schemas!
	if (!ConnectionDrawingPolicy)
	{
		if (Schema->IsA(UEdGraphSchema_K2::StaticClass()))
		{
			ConnectionDrawingPolicy = new FKismetConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);
		}
		else
		{
			ConnectionDrawingPolicy = new FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements);
		}
	}

	// If we never picked a custom policy, use the uncustomized standard policy
	return ConnectionDrawingPolicy;
}

TSharedPtr<SGraphNode> FGraphNodeFactory::CreateNodeWidget(UEdGraphNode* InNode)
{
	return FNodeFactory::CreateNodeWidget(InNode);
}

TSharedPtr<SGraphPin> FGraphNodeFactory::CreatePinWidget(UEdGraphPin* InPin)
{
	return FNodeFactory::CreatePinWidget(InPin);
}

FConnectionDrawingPolicy* FGraphNodeFactory::CreateConnectionPolicy(const UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
{
	return FNodeFactory::CreateConnectionPolicy(Schema, InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}
