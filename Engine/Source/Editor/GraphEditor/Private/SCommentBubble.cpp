// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCommentBubble.h"

#include "EdGraph/EdGraphNode.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/ICursor.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"

class SWidget;

namespace SCommentBubbleDefs
{
	/** Bubble fade up/down delay */
	static const float FadeDelay = -3.5f;

	/** Bubble Toggle Icon Fade Speed */
	static const float FadeDownSpeed = 5.f;

	/** Height of the arrow connecting the bubble to the node */
	static const float BubbleArrowHeight = 8.f;

	/** Offset from the left edge to comment bubbles arrow center */
	static const float ArrowCentreOffset = 12.f;

	/** Offset from the left edge to comment bubbles toggle button center */
	static const float ToggleButtonCentreOffset = 3.f;

	/** Luminance CoEficients */
	static const FLinearColor LuminanceCoEff( 0.2126f, 0.7152f, 0.0722f, 0.f );

	/** Clear text box background color */
	static const FLinearColor TextClearBackground( 0.f, 0.f, 0.f, 0.f );

};


void SCommentBubble::Construct( const FArguments& InArgs )
{
	check( InArgs._Text.IsBound() );
	check( InArgs._GraphNode );

	GraphNode				= InArgs._GraphNode;
	CommentAttribute		= InArgs._Text;
	OnTextCommittedDelegate	= InArgs._OnTextCommitted;
	OnToggledDelegate		= InArgs._OnToggled;
	ColorAndOpacity			= InArgs._ColorAndOpacity;
	bAllowPinning			= InArgs._AllowPinning;
	bEnableTitleBarBubble	= InArgs._EnableTitleBarBubble;
	bEnableBubbleCtrls		= InArgs._EnableBubbleCtrls;
	bInvertLODCulling		= InArgs._InvertLODCulling;
	GraphLOD				= InArgs._GraphLOD;
	IsGraphNodeHovered		= InArgs._IsGraphNodeHovered;
	HintText				= InArgs._HintText.IsSet() ? InArgs._HintText : NSLOCTEXT( "CommentBubble", "EditCommentHint", "Click to edit" );
	OpacityValue			= SCommentBubbleDefs::FadeDelay;
	// Create default delegate/attribute handlers if required
	ToggleButtonCheck		= InArgs._ToggleButtonCheck.IsBound() ?	InArgs._ToggleButtonCheck : 
																	TAttribute<ECheckBoxState>( this, &SCommentBubble::GetToggleButtonCheck );
	// Ensue this value is set to something sensible
	SetForegroundColor(FStyleColors::Foreground);
	BubbleLuminance = 0.0f;

	// Cache the comment
	CachedComment = CommentAttribute.Get();
	CachedCommentText = FText::FromString( CachedComment );

	// Create Widget
	UpdateBubble();
}

FCursorReply SCommentBubble::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	const FVector2f Size( GetDesiredSize().X, GetDesiredSize().Y - SCommentBubbleDefs::BubbleArrowHeight );
	const FSlateRect TestRect( MyGeometry.AbsolutePosition, MyGeometry.AbsolutePosition + Size );

	if( TestRect.ContainsPoint( CursorEvent.GetScreenSpacePosition() ))
	{
		return FCursorReply::Cursor( EMouseCursor::TextEditBeam );
	}
	return FCursorReply::Cursor( EMouseCursor::Default );
}

void SCommentBubble::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SCompoundWidget::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );

	// Check Editable and Hovered so we can prevent bubble toggling in read only graphs.
	const bool bNodeEditable = !IsReadOnly();
	const bool bEnableTitleHintBubble = bEnableTitleBarBubble && bNodeEditable;
	const bool bTitleBarBubbleVisible = bEnableTitleHintBubble && IsGraphNodeHovered.IsBound();

	if( bTitleBarBubbleVisible || IsBubbleVisible() )
	{
		const FLinearColor BubbleColor = GetBubbleColor().GetSpecifiedColor() * SCommentBubbleDefs::LuminanceCoEff;
		BubbleLuminance = BubbleColor.R + BubbleColor.G + BubbleColor.B;
		SetForegroundColor(BubbleLuminance < 0.5f ? FStyleColors::Foreground : FStyleColors::Background);
	}

	TickVisibility(InCurrentTime, InDeltaTime);

	if( CachedComment != CommentAttribute.Get() )
	{
		CachedComment = CommentAttribute.Get();
		CachedCommentText = FText::FromString( CachedComment );

		// Toggle the comment on/off, provided it the parent isn't a comment node
		if( !bInvertLODCulling )
		{
			// If the comment visibility isn't correct, toggle it.
			if (CachedComment.IsEmpty() == GraphNode->bCommentBubbleVisible)
			{
				OnCommentBubbleToggle(CachedComment.IsEmpty() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
			}
			else
			{
				// we just need to refresh the widget's visibility, not the graph node
				UpdateBubble();
			}
		}
	}
}

void SCommentBubble::TickVisibility(const double InCurrentTime, const float InDeltaTime)
{
	if( !GraphNode->bCommentBubbleVisible )
	{
		const bool bNodeEditable = !IsReadOnly();
		const bool bEnableTitleHintBubble = bEnableTitleBarBubble && bNodeEditable;
		const bool bTitleBarBubbleVisible = bEnableTitleHintBubble && IsGraphNodeHovered.IsBound();

		if( bTitleBarBubbleVisible )
		{
			const bool bIsCommentHovered = IsHovered() || IsGraphNodeHovered.Execute();

			if( bIsCommentHovered )
			{
				if( OpacityValue < 1.f )
				{
					const float FadeUpAmt = InDeltaTime * SCommentBubbleDefs::FadeDownSpeed;
					OpacityValue = FMath::Min( OpacityValue + FadeUpAmt, 1.f );
				}
			}
			else
			{
				if( OpacityValue > SCommentBubbleDefs::FadeDelay )
				{
					const float FadeDownAmt = InDeltaTime * SCommentBubbleDefs::FadeDownSpeed;
					OpacityValue = FMath::Max( OpacityValue - FadeDownAmt, SCommentBubbleDefs::FadeDelay );
				}
			}
		}
	}
}
void SCommentBubble::UpdateBubble()
{
	if( GraphNode->bCommentBubbleVisible )
	{
		const FSlateBrush* CommentCalloutArrowBrush = FAppStyle::GetBrush(TEXT("Graph.Node.CommentArrow"));
		const FMargin BubblePadding = FAppStyle::GetMargin( TEXT("Graph.Node.Comment.BubbleWidgetMargin"));
		const FMargin PinIconPadding = FAppStyle::GetMargin( TEXT("Graph.Node.Comment.PinIconPadding"));
		const FMargin BubbleOffset = FAppStyle::GetMargin( TEXT("Graph.Node.Comment.BubbleOffset"));
		// Conditionally create bubble controls
		TSharedPtr<SWidget> BubbleControls = SNullWidget::NullWidget;

		if( bEnableBubbleCtrls )
		{
			if( bAllowPinning )
			{
				SAssignNew( BubbleControls, SVerticalBox )
				.Visibility( this, &SCommentBubble::GetBubbleVisibility )
				+SVerticalBox::Slot()
				.Padding( 1.f )
				.AutoHeight()
				.VAlign( VAlign_Top )
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( PinIconPadding )
					[
						SNew( SCheckBox )
						.Style( &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>( "CommentBubblePin" ))
						.IsChecked( this, &SCommentBubble::GetPinnedButtonCheck )
						.OnCheckStateChanged( this, &SCommentBubble::OnPinStateToggle )
						.ToolTipText( this, &SCommentBubble::GetScaleButtonTooltip )
						.Cursor( EMouseCursor::Default )
						.ForegroundColor( this, &SCommentBubble::GetForegroundColor )
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( 1.f )
				.VAlign( VAlign_Top )
				[
					SNew( SCheckBox )
					.Style( &FAppStyle::Get().GetWidgetStyle< FCheckBoxStyle >( "CommentBubbleButton" ))
					.IsChecked( ToggleButtonCheck )
					.OnCheckStateChanged( this, &SCommentBubble::OnCommentBubbleToggle )
					.ToolTipText( NSLOCTEXT( "CommentBubble", "ToggleCommentTooltip", "Toggle Comment Bubble" ))
					.Cursor( EMouseCursor::Default )
					.ForegroundColor(FStyleColors::Foreground)
				];
			}
			else
			{
				SAssignNew( BubbleControls, SVerticalBox )
				.Visibility( this, &SCommentBubble::GetBubbleVisibility )
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( 1.f )
				.VAlign( VAlign_Top )
				[
					SNew( SCheckBox )
					.Style( &FAppStyle::Get().GetWidgetStyle< FCheckBoxStyle >( "CommentBubbleButton" ))
					.IsChecked( ToggleButtonCheck )
					.OnCheckStateChanged( this, &SCommentBubble::OnCommentBubbleToggle )
					.ToolTipText( NSLOCTEXT( "CommentBubble", "ToggleCommentTooltip", "Toggle Comment Bubble" ))
					.Cursor( EMouseCursor::Default )
					.ForegroundColor( FStyleColors::Foreground )
				];
			}
		}
		// Create the comment bubble widget
		ChildSlot
		[
			SNew(SVerticalBox)
			.Visibility( this, &SCommentBubble::GetBubbleVisibility )
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						SNew(SImage)
						.Image( FAppStyle::GetBrush( TEXT("Graph.Node.CommentBubble")) )
						.ColorAndOpacity( this, &SCommentBubble::GetBubbleColor )
					]
					+SOverlay::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.Padding( BubblePadding )
						.AutoWidth()
						[
							SAssignNew(TextBlock, SMultiLineEditableTextBox)
							.Text(MakeAttributeLambda([this] { return CachedCommentText; }))
							.Style(FAppStyle::Get(), "Graph.CommentBubble.EditableText")
							.HintText( NSLOCTEXT( "CommentBubble", "EditCommentHint", "Click to edit" ))
							.IsReadOnly(this, &SCommentBubble::IsReadOnly)
							.SelectAllTextWhenFocused( true )
							.RevertTextOnEscape( true )
							.ClearKeyboardFocusOnCommit( true )
							.ModiferKeyForNewLine(EModifierKey::Shift)
							.OnTextCommitted(this, &SCommentBubble::OnCommentTextCommitted)
							.ForegroundColor(this, &SCommentBubble::GetTextForegroundColor)
							.ReadOnlyForegroundColor(this, &SCommentBubble::GetReadOnlyTextForegroundColor)
							.BackgroundColor(this, &SCommentBubble::GetTextBackgroundColor)
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign( HAlign_Right )
						.Padding( 0.f )
						[
							BubbleControls.ToSharedRef()
						]
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding( BubbleOffset )
				.MaxWidth( CommentCalloutArrowBrush->ImageSize.X )
				[
					SNew(SImage)
					.Image( CommentCalloutArrowBrush )
					.ColorAndOpacity( this, &SCommentBubble::GetBubbleColor )
				]
			]
		];
	}
	else 
	{
		TSharedPtr<SWidget> TitleBarBubble = SNullWidget::NullWidget;

		if( bEnableTitleBarBubble )
		{
			const FMargin BubbleOffset = FAppStyle::GetMargin( TEXT("Graph.Node.Comment.BubbleOffset"));
			// Create Title bar bubble toggle widget
			SAssignNew( TitleBarBubble, SHorizontalBox )
			.Visibility( this, &SCommentBubble::GetToggleButtonVisibility )
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign( HAlign_Center )
			.VAlign( VAlign_Top )
			.Padding( BubbleOffset )
			[
				SNew( SCheckBox )
				.Style( &FAppStyle::Get().GetWidgetStyle< FCheckBoxStyle >( "CommentTitleButton" ))
				.IsChecked( ToggleButtonCheck )
				.OnCheckStateChanged( this, &SCommentBubble::OnCommentBubbleToggle )
				.ToolTipText( NSLOCTEXT( "CommentBubble", "ToggleCommentTooltip", "Toggle Comment Bubble" ))
				.Cursor( EMouseCursor::Default )
				.ForegroundColor( this, &SCommentBubble::GetToggleButtonColor )
			];
		}
		ChildSlot
		[
			TitleBarBubble.ToSharedRef()
		];
	}
}

bool SCommentBubble::TextBlockHasKeyboardFocus() const
{
	if (TextBlock.IsValid())
	{
		return TextBlock->HasKeyboardFocus();
	}
	return false;
}

FVector2D SCommentBubble::GetOffset() const
{
	return FVector2D(GetOffset2f());
}

FVector2f SCommentBubble::GetOffset2f() const
{
	return FVector2f( 0.f, -GetDesiredSize().Y );
}

float SCommentBubble::GetArrowCenterOffset() const
{
	float CentreOffset = GraphNode->bCommentBubbleVisible ? SCommentBubbleDefs::ArrowCentreOffset : SCommentBubbleDefs::ToggleButtonCentreOffset;
	const bool bVisibleAndUnpinned = !GraphNode->bCommentBubblePinned && GraphNode->bCommentBubbleVisible;
	if (bVisibleAndUnpinned && GraphNode->DEPRECATED_NodeWidget.IsValid())
	{
		TSharedPtr<SGraphPanel> GraphPanel = GraphNode->DEPRECATED_NodeWidget.Pin()->GetOwnerPanel();
		CentreOffset *= GraphPanel.IsValid() ? GraphPanel->GetZoomAmount() : 1.f;
	}
	return CentreOffset;
}

FVector2D SCommentBubble::GetSize() const
{
	return FVector2D(GetSize2f());
}

FVector2f SCommentBubble::GetSize2f() const
{
	return GetDesiredSize();
}

bool SCommentBubble::IsBubbleVisible() const
{
	EGraphRenderingLOD::Type CurrLOD = GraphLOD.Get();
	const bool bShowScaled = CurrLOD > EGraphRenderingLOD::LowDetail;
	const bool bShowPinned = CurrLOD <= EGraphRenderingLOD::MediumDetail;

	if( bAllowPinning && !bInvertLODCulling )
	{
		return GraphNode->bCommentBubblePinned ? true : bShowScaled;
	}
	return bInvertLODCulling ? bShowPinned : !bShowPinned;
}

bool SCommentBubble::IsScalingAllowed() const
{
	return !GraphNode->bCommentBubblePinned || !GraphNode->bCommentBubbleVisible;
}

FText SCommentBubble::GetScaleButtonTooltip() const
{
	if( GraphNode->bCommentBubblePinned )
	{
		return NSLOCTEXT( "CommentBubble", "AllowScaleButtonTooltip", "Allow this bubble to scale with zoom" );
	}
	else
	{
		return NSLOCTEXT( "CommentBubble", "PreventScaleButtonTooltip", "Prevent this bubble scaling with zoom" );
	}
}

FSlateColor SCommentBubble::GetToggleButtonColor() const
{
	const FLinearColor BubbleColor = ColorAndOpacity.Get().GetSpecifiedColor();
	return FLinearColor( 1.f, 1.f, 1.f, OpacityValue * OpacityValue );
}

FSlateColor SCommentBubble::GetBubbleColor() const
{
	FLinearColor ReturnColor = ColorAndOpacity.Get().GetSpecifiedColor();

	if(!GraphNode->IsNodeEnabled() || GraphNode->IsDisplayAsDisabledForced() || GraphNode->IsNodeUnrelated())
	{
		ReturnColor.A *= 0.6f;
	}
	return ReturnColor;
}
void SCommentBubble::OnCommentTextCommitted( const FText& NewText, ETextCommit::Type CommitInfo )
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		CachedComment = NewText.ToString();
		CachedCommentText = NewText;
		OnTextCommittedDelegate.ExecuteIfBound(CachedCommentText, CommitInfo);
	}
}

FSlateColor SCommentBubble::GetTextBackgroundColor() const
{
	return TextBlock->HasKeyboardFocus() ? FSlateColor::UseStyle() : SCommentBubbleDefs::TextClearBackground;
}

FSlateColor SCommentBubble::GetTextForegroundColor() const
{
	if (TextBlock->HasKeyboardFocus())
	{
		return BubbleLuminance < 0.5f ? FStyleColors::Background : FStyleColors::Foreground;
	}
	else
	{
		return BubbleLuminance < 0.5f ? FStyleColors::Foreground : FStyleColors::Background;
	}
}

FSlateColor SCommentBubble::GetReadOnlyTextForegroundColor() const
{
	return TextBlock->HasKeyboardFocus() ? FStyleColors::Foreground : FSlateColor::UseStyle();
}

EVisibility SCommentBubble::GetToggleButtonVisibility() const
{
	EVisibility ButtonVisibility = EVisibility::Collapsed;

	if( OpacityValue > 0.f && !GraphNode->bCommentBubbleVisible )
	{
		ButtonVisibility = EVisibility::Visible;
	}
	return ButtonVisibility;
}

EVisibility SCommentBubble::GetBubbleVisibility() const
{
	return IsBubbleVisible() ? EVisibility::Visible : EVisibility::Hidden;
}

ECheckBoxState SCommentBubble::GetToggleButtonCheck() const
{
	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	if( GraphNode )
	{
		CheckState = GraphNode->bCommentBubbleVisible ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return CheckState;
}

void SCommentBubble::OnCommentBubbleToggle( ECheckBoxState State )
{
	const bool bNewVisibilityState = (State == ECheckBoxState::Checked);
	if( !IsReadOnly() && bNewVisibilityState != GraphNode->bCommentBubbleVisible)
	{
		const FScopedTransaction Transaction( NSLOCTEXT( "CommentBubble", "BubbleVisibility", "Comment Bubble Visibility" ) );
		GraphNode->Modify();
		SetCommentBubbleVisibility(bNewVisibilityState);
		OnToggledDelegate.ExecuteIfBound(GraphNode->bCommentBubbleVisible);
	}
}

void SCommentBubble::SetCommentBubbleVisibility(bool bVisible)
{
	if (!IsReadOnly() && bVisible != GraphNode->bCommentBubbleVisible)
	{
		GraphNode->bCommentBubbleVisible = bVisible;
		OpacityValue = 0.f;
		UpdateBubble();
	}
}

ECheckBoxState SCommentBubble::GetPinnedButtonCheck() const
{
	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	if( GraphNode )
	{
		CheckState = GraphNode->bCommentBubblePinned ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return CheckState;
}

void SCommentBubble::OnPinStateToggle( ECheckBoxState State ) const
{
	if( !IsReadOnly() )
	{
		const FScopedTransaction Transaction( NSLOCTEXT( "CommentBubble", "BubblePinned", "Comment Bubble Pin" ) );
		GraphNode->Modify();
		GraphNode->bCommentBubblePinned = State == ECheckBoxState::Checked;
	}
}

bool SCommentBubble::IsReadOnly() const
{
	bool bReadOnly = true;
	if (GraphNode && GraphNode->DEPRECATED_NodeWidget.IsValid())
	{
		bReadOnly = !GraphNode->DEPRECATED_NodeWidget.Pin()->IsNodeEditable();
	}
	return bReadOnly;
}
