// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphActionMenu.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EditorCategoryUtils.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Views/ITypedTableView.h"
#include "GraphActionNode.h"
#include "GraphEditorDragDropAction.h"
#include "IDocumentation.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class SWidget;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "GraphActionMenu"

//////////////////////////////////////////////////////////////////////////

namespace UE::GraphEditor::Private
{
	// These constants control where we attempt to put the focused item
	// this is the 'displayed index' of a menu entry, e.g. 2nd or 12th from
	// the top of the list control. We're handling this ourself because
	// this control has to amortize list building because the list is
	// so large... a virtual list control that can handle >1m items 
	// is probably the ideal.
	const int32 PREFERRED_TOP_INDEX = 2;
	const int32 PREFERRED_BOTTOM_INDEX = 12;
}

template<typename ItemType>
class SCategoryHeaderTableRow : public STableRow<ItemType>
{
public:
	SLATE_BEGIN_ARGS(SCategoryHeaderTableRow)
	{}
		SLATE_DEFAULT_SLOT(typename SCategoryHeaderTableRow::FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow<ItemType>::ChildSlot
		.Padding(0.0f, 2.0f, .0f, 0.0f)
		[
			SAssignNew(ContentBorder, SBorder)
			.BorderImage(this, &SCategoryHeaderTableRow::GetBackgroundImage)
			.Padding(FMargin(3.0f, 5.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(5.0f)
				.AutoWidth()
				[
					SNew(SExpanderArrow, STableRow< ItemType >::SharedThis(this))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					InArgs._Content.Widget
				]
			]
		];

		STableRow < ItemType >::ConstructInternal(
			typename STableRow< ItemType >::FArguments()
			.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.ShowSelection(false),
			InOwnerTableView
			);
	}

	const FSlateBrush* GetBackgroundImage() const
	{
		if ( STableRow<ItemType>::IsHovered() )
		{
			return FAppStyle::Get().GetBrush("Brushes.Secondary");
		}
		else
		{
			return FAppStyle::Get().GetBrush("Brushes.Header");
		}
	}

	virtual void SetContent(TSharedRef< SWidget > InContent) override
	{
		ContentBorder->SetContent(InContent);
	}

	virtual void SetRowContent(TSharedRef< SWidget > InContent) override
	{
		ContentBorder->SetContent(InContent);
	}

	virtual const FSlateBrush* GetBorder() const
	{
		return nullptr;
	}

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			STableRow<ItemType>::ToggleExpansion();
			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}
private:
	TSharedPtr<SBorder> ContentBorder;
};

//////////////////////////////////////////////////////////////////////////

namespace GraphActionMenuHelpers
{
	bool ActionMatchesName(const FEdGraphSchemaAction* InGraphAction, const FName& ItemName)
	{
		bool bCheck = false;

		bCheck |= (InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId() &&
			((FEdGraphSchemaAction_K2Var*)InGraphAction)->GetVariableName() == ItemName);
		bCheck |= (InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2LocalVar::StaticGetTypeId() &&
			((FEdGraphSchemaAction_K2LocalVar*)InGraphAction)->GetVariableName() == ItemName);
		bCheck |= (InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId() &&
			((FEdGraphSchemaAction_K2Graph*)InGraphAction)->EdGraph &&
			((FEdGraphSchemaAction_K2Graph*)InGraphAction)->EdGraph->GetFName() == ItemName);
		bCheck |= (InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2Enum::StaticGetTypeId() &&
			((FEdGraphSchemaAction_K2Enum*)InGraphAction)->GetPathName() == ItemName);
		bCheck |= (InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2Struct::StaticGetTypeId() &&
			((FEdGraphSchemaAction_K2Struct*)InGraphAction)->GetPathName() == ItemName);
		bCheck |= (InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2Delegate::StaticGetTypeId() &&
			((FEdGraphSchemaAction_K2Delegate*)InGraphAction)->GetDelegateName() == ItemName);

		const bool bIsTargetNodeSubclass = (InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2TargetNode::StaticGetTypeId()) ||
			(InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2Event::StaticGetTypeId()) ||
			(InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2InputAction::StaticGetTypeId());
		bCheck |= (bIsTargetNodeSubclass &&
			((FEdGraphSchemaAction_K2TargetNode*)InGraphAction)->NodeTemplate->GetNodeTitle(ENodeTitleType::EditableTitle).ToString() == ItemName.ToString());

		return bCheck;
	}
}

//////////////////////////////////////////////////////////////////////////

void SDefaultGraphActionWidget::Construct(const FArguments& InArgs, const FCreateWidgetForActionData* InCreateData)
{
	ActionPtr = InCreateData->Action;
	MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		.ToolTipText(InCreateData->Action->GetTooltipDescription())
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
			.Text(InCreateData->Action->GetMenuDescription())
			.HighlightText(InArgs._HighlightText.IsSet() ? InArgs._HighlightText : InCreateData->HighlightText)
		]
	];
}

FReply SDefaultGraphActionWidget::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if( MouseButtonDownDelegate.Execute( ActionPtr ) )
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

//////////////////////////////////////////////////////////////////////////

class SGraphActionCategoryWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SGraphActionCategoryWidget ) 
	{}
		SLATE_ATTRIBUTE( FText, HighlightText )
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )
		SLATE_EVENT( FIsSelected, IsSelected )
		SLATE_ATTRIBUTE( bool, IsReadOnly )
	SLATE_END_ARGS()

	TWeakPtr<FGraphActionNode> ActionNode;
	TAttribute<bool> IsReadOnly;
public:
	TWeakPtr<SInlineEditableTextBlock> InlineWidget;

	void Construct( const FArguments& InArgs, TSharedPtr<FGraphActionNode> InActionNode )
	{
		ActionNode = InActionNode;

		FText CategoryTooltip;
		FString CategoryLink, CategoryExcerpt;
		FEditorCategoryUtils::GetCategoryTooltipInfo(*InActionNode->GetDisplayName().ToString(), CategoryTooltip, CategoryLink, CategoryExcerpt);

		TSharedRef<SToolTip> ToolTipWidget = IDocumentation::Get()->CreateToolTip(CategoryTooltip, NULL, CategoryLink, CategoryExcerpt);
		IsReadOnly = InArgs._IsReadOnly;

		this->ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineWidget, SInlineEditableTextBlock)
				.Font( FCoreStyle::GetDefaultFontStyle("Bold", 9) )
				.Text( FEditorCategoryUtils::GetCategoryDisplayString(InActionNode->GetDisplayName()) )
				.ToolTip( ToolTipWidget )
				.HighlightText( InArgs._HighlightText )
				.OnVerifyTextChanged( this, &SGraphActionCategoryWidget::OnVerifyTextChanged )
				.OnTextCommitted( InArgs._OnTextCommitted )
				.IsSelected( InArgs._IsSelected )
				.IsReadOnly( InArgs._IsReadOnly )
			]
		];
	}

	// SWidget interface
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override
	{
		TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
		if (GraphDropOp.IsValid())
		{
			GraphDropOp->DroppedOnCategory( ActionNode.Pin()->GetCategoryPath() );
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override
	{
		TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
		if (GraphDropOp.IsValid())
		{
			GraphDropOp->SetHoveredCategoryName( ActionNode.Pin()->GetDisplayName() );
		}
	}

	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override
	{
		TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
		if (GraphDropOp.IsValid())
		{
			GraphDropOp->SetHoveredCategoryName( FText::GetEmpty() );
		}
	}

	// End of SWidget interface

	/** Callback for the SInlineEditableTextBlock to verify the text before commit */
	bool OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage)
	{
		if(InText.ToString().Len() >= NAME_SIZE)
		{
			OutErrorMessage = LOCTEXT("CategoryNameTooLong_Error", "Name too long!");
			return false;
		}

		return true;
	}
};

//////////////////////////////////////////////////////////////////////////

FString SGraphActionMenu::LastUsedFilterText;

void SGraphActionMenu::Construct( const FArguments& InArgs, bool bIsReadOnly/* = true*/ )
{
	this->SelectedSuggestionScore = TNumericLimits<float>::Lowest();
	this->SelectedSuggestionSourceIndex = INDEX_NONE;
	this->SelectedAction = TSharedPtr<FGraphActionNode>();
	this->bIgnoreUIUpdate = false;
	this->bUseSectionStyling = InArgs._UseSectionStyling;
	this->bIsKeyboardNavigating = false;
	this->bAllowPreselectedItemActivation = InArgs._bAllowPreselectedItemActivation;
	this->bAutomaticallySelectSingleAction = InArgs._bAutomaticallySelectSingleAction;
	this->DefaultRowExpanderBaseIndentLevel = InArgs._DefaultRowExpanderBaseIndentLevel;
	this->DisplayIndex = UE::GraphEditor::Private::PREFERRED_TOP_INDEX;

	this->bAutoExpandActionMenu = InArgs._AutoExpandActionMenu;
	this->bShowFilterTextBox = InArgs._ShowFilterTextBox;
	this->bAlphaSortItems = InArgs._AlphaSortItems;
	this->bSortItemsRecursively = InArgs._SortItemsRecursively;
	this->OnActionSelected = InArgs._OnActionSelected;
	this->OnActionDoubleClicked = InArgs._OnActionDoubleClicked;
	this->OnActionDragged = InArgs._OnActionDragged;
	this->OnCategoryDragged = InArgs._OnCategoryDragged;
	this->OnCreateWidgetForAction = InArgs._OnCreateWidgetForAction;
	this->OnCreateCustomRowExpander = InArgs._OnCreateCustomRowExpander;
	this->OnGetActionList = InArgs._OnGetActionList;
	this->OnCollectAllActions = InArgs._OnCollectAllActions;
	this->OnCollectStaticSections = InArgs._OnCollectStaticSections;
	this->OnCategoryTextCommitted = InArgs._OnCategoryTextCommitted;
	this->OnCanRenameSelectedAction = InArgs._OnCanRenameSelectedAction;
	this->OnGetSectionTitle = InArgs._OnGetSectionTitle;
	this->OnGetSectionToolTip = InArgs._OnGetSectionToolTip;
	this->OnGetSectionWidget = InArgs._OnGetSectionWidget;
	this->FilteredRootAction = FGraphActionNode::NewRootNode();
	this->OnActionMatchesName = InArgs._OnActionMatchesName;	
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->GraphObj = InArgs._GraphObj;

	// Default graph action list (also provides an empty source list to start with)
	AllActions = MakeShared<FGraphActionListBuilderBase>();
	
	if(OnGetActionList.IsBound())
	{
		// If we are obtaining a new action list at refresh time, ensure that the indirect collection delegate is unbound
		if (!ensureMsgf(!OnCollectAllActions.IsBound(), TEXT("The OnCollectAllActions delegate is bound, but will not be invoked, because OnGetActionList has also been bound and will be used to obtain the action list for this menu. To resolve this, one of these events should be removed from its construction.")))
		{
			OnCollectAllActions.Unbind();
		}
	}

	// If a delegate for filtering text is passed in, assign it so that it will be used instead of the built-in filter box
	if(InArgs._OnGetFilterText.IsBound())
	{
		this->OnGetFilterText = InArgs._OnGetFilterText;
	}

	TreeView = SNew(STreeView< TSharedPtr<FGraphActionNode> >)
		.TreeItemsSource(&(this->FilteredRootAction->Children))
		.OnGenerateRow(this, &SGraphActionMenu::MakeWidget, bIsReadOnly)
		.OnSelectionChanged(this, &SGraphActionMenu::OnItemSelected)
		.OnMouseButtonDoubleClick(this, &SGraphActionMenu::OnItemDoubleClicked)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		.OnGetChildren(this, &SGraphActionMenu::OnGetChildrenForCategory)
		.SelectionMode(ESelectionMode::Single)
		.OnItemScrolledIntoView(this, &SGraphActionMenu::OnItemScrolledIntoView)
		.OnSetExpansionRecursive(this, &SGraphActionMenu::OnSetExpansionRecursive)
		.HighlightParentNodesForSelection(true);


	this->ChildSlot
	[
		SNew(SVerticalBox)

		// FILTER BOX
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(FilterTextBox, SSearchBox)
			// If there is an external filter delegate, do not display this filter box
			.Visibility(InArgs._OnGetFilterText.IsBound()? EVisibility::Collapsed : EVisibility::Visible)
			.OnTextChanged( this, &SGraphActionMenu::OnFilterTextChanged )
			.OnTextCommitted( this, &SGraphActionMenu::OnFilterTextCommitted )
			.DelayChangeNotificationsWhileTyping(false)
		]

		// ACTION LIST
		+SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
		.FillHeight(1.f)
		[
			SNew(SScrollBorder, TreeView.ToSharedRef())
			[
				TreeView.ToSharedRef()
			]
		]
	];

	// When the search box has focus, we want first chance handling of any key down events so we can handle the up/down and escape keys the way we want
	FilterTextBox->SetOnKeyDownHandler(FOnKeyDown::CreateSP(this, &SGraphActionMenu::OnKeyDown));

	if (!InArgs._ShowFilterTextBox)
	{
		FilterTextBox->SetVisibility(EVisibility::Collapsed);
	}

	// Get all actions.
	RefreshAllActions(false);

	if(bAutomaticallySelectSingleAction)
	{
		TArray<TSharedPtr<FGraphActionNode>> ActionNodes;
		FilteredRootAction->GetAllActionNodes(ActionNodes);

		if(ActionNodes.Num() == 1)
		{
			OnItemSelected(ActionNodes[0], ESelectInfo::Direct);
		}
	}
}

void SGraphActionMenu::RefreshAllActions(bool bPreserveExpansion, bool bHandleOnSelectionEvent/*=true*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SGraphActionMenu::RefreshAllActions);

	// Save Selection (of only the first selected thing)
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = TreeView->GetSelectedItems();
	TSharedPtr<FGraphActionNode> CurrentSelectedAction = SelectedNodes.Num() > 0 ? SelectedNodes[0] : nullptr;

	if (OnGetActionList.IsBound())
	{
		// Obtain the source action list directly.
		AllActions = OnGetActionList.Execute();
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SGraphActionMenu::CollectAllActions);

		// Collect actions into our local list context.
		AllActions->Empty();
		OnCollectAllActions.ExecuteIfBound(*AllActions);
	}

	GenerateFilteredItems(bPreserveExpansion);

	// Re-apply selection #0 if possible
	if (CurrentSelectedAction.IsValid())
	{
		// Clear the selection, we will be re-selecting the previous action
		TreeView->ClearSelection();

		if(bHandleOnSelectionEvent)
		{
			SelectItemByName(*CurrentSelectedAction->GetDisplayName().ToString(), ESelectInfo::OnMouseClick, CurrentSelectedAction->SectionID, SelectedNodes[0]->IsCategoryNode());
		}
		else
		{
			// If we do not want to handle the selection, set it directly so it will reselect the item but not handle the event.
			SelectItemByName(*CurrentSelectedAction->GetDisplayName().ToString(), ESelectInfo::Direct, CurrentSelectedAction->SectionID, SelectedNodes[0]->IsCategoryNode());
		}
	}
}

void SGraphActionMenu::GetSectionExpansion(TMap<int32, bool>& SectionExpansion) const
{

}

void SGraphActionMenu::SetSectionExpansion(const TMap<int32, bool>& InSectionExpansion)
{
	for ( auto& PossibleSection : FilteredRootAction->Children )
	{
		if ( PossibleSection->IsSectionHeadingNode() )
		{
			const bool* IsExpanded = InSectionExpansion.Find(PossibleSection->SectionID);
			if ( IsExpanded != nullptr )
			{
				TreeView->SetItemExpansion(PossibleSection, *IsExpanded);
			}
		}
	}
}

TSharedRef<SEditableTextBox> SGraphActionMenu::GetFilterTextBox()
{
	return FilterTextBox.ToSharedRef();
}

void SGraphActionMenu::GetSelectedActions(TArray< TSharedPtr<FEdGraphSchemaAction> >& OutSelectedActions) const
{
	OutSelectedActions.Empty();

	TArray<TSharedPtr<FGraphActionNode>> SelectedNodes = TreeView->GetSelectedItems();
	for (TSharedPtr<FGraphActionNode>& SelectedNode : SelectedNodes)
	{
		if (SelectedNode.IsValid() && SelectedNode->IsActionNode())
		{
			OutSelectedActions.Add(SelectedNode->Action);
		}
	}
}

void SGraphActionMenu::OnRequestRenameOnActionNode()
{
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = TreeView->GetSelectedItems();
	if(SelectedNodes.Num() > 0)
	{
		if (!SelectedNodes[0]->BroadcastRenameRequest())
		{
			TreeView->RequestScrollIntoView(SelectedNodes[0]);
		}
	}
}

bool SGraphActionMenu::CanRequestRenameOnActionNode() const
{
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = TreeView->GetSelectedItems();
	if(SelectedNodes.Num() == 1 && OnCanRenameSelectedAction.IsBound())
	{
		return OnCanRenameSelectedAction.Execute(SelectedNodes[0]);
	}

	return false;
}

FString SGraphActionMenu::GetSelectedCategoryName() const
{
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = TreeView->GetSelectedItems();
	return (SelectedNodes.Num() > 0) ? SelectedNodes[0]->GetDisplayName().ToString() : FString();
}

void SGraphActionMenu::GetSelectedCategorySubActions(TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const
{
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = TreeView->GetSelectedItems();
	for ( int32 SelectionIndex = 0; SelectionIndex < SelectedNodes.Num(); SelectionIndex++ )
	{
		if ( SelectedNodes[SelectionIndex].IsValid() )
		{
			GetCategorySubActions(SelectedNodes[SelectionIndex], OutActions);
		}
	}
}

void SGraphActionMenu::GetCategorySubActions(TWeakPtr<FGraphActionNode> InAction, TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const
{
	if(InAction.IsValid())
	{
		TSharedPtr<FGraphActionNode> CategoryNode = InAction.Pin();
		TArray<TSharedPtr<FGraphActionNode>> Children;
		CategoryNode->GetLeafNodes(Children);

		for (int32 i = 0; i < Children.Num(); ++i)
		{
			TSharedPtr<FGraphActionNode> CurrentChild = Children[i];

			if (CurrentChild.IsValid() && CurrentChild->IsActionNode())
			{
				OutActions.Add(CurrentChild->Action);
			}
		}
	}
}

bool SGraphActionMenu::SelectItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo, int32 SectionId/* = INDEX_NONE */, bool bIsCategory/* = false*/)
{
	if (ItemName != NAME_None)
	{
		TSharedPtr<FGraphActionNode> SelectionNode;

		TArray<TSharedPtr<FGraphActionNode>> GraphNodes;
		FilteredRootAction->GetAllNodes(GraphNodes);
		for (int32 i = 0; i < GraphNodes.Num() && !SelectionNode.IsValid(); ++i)
		{
			TSharedPtr<FGraphActionNode> CurrentGraphNode = GraphNodes[i];
			FEdGraphSchemaAction* GraphAction = CurrentGraphNode->GetPrimaryAction().Get();

			// If the user is attempting to select a category, make sure it's a category
			if( CurrentGraphNode->IsCategoryNode() == bIsCategory )
			{
				if(SectionId == INDEX_NONE || CurrentGraphNode->SectionID == SectionId)
				{
					if (GraphAction)
					{
						if ((OnActionMatchesName.IsBound() && OnActionMatchesName.Execute(GraphAction, ItemName)) || GraphActionMenuHelpers::ActionMatchesName(GraphAction, ItemName))
						{
							SelectionNode = GraphNodes[i];

							break;
						}
					}
					
					if (CurrentGraphNode->GetDisplayName().ToString() == FName::NameToDisplayString(ItemName.ToString(), false))
					{
						SelectionNode = CurrentGraphNode;

						break;
					}
				}
			}

			// One of the children may match
			for(int32 ChildIdx = 0; ChildIdx < CurrentGraphNode->Children.Num() && !SelectionNode.IsValid(); ++ChildIdx)
			{
				TSharedPtr<FGraphActionNode> CurrentChildNode = CurrentGraphNode->Children[ChildIdx];

				if(FEdGraphSchemaAction* ChildGraphAction = CurrentChildNode->Action.Get())
				{
					// If the user is attempting to select a category, make sure it's a category
					if( CurrentChildNode->IsCategoryNode() == bIsCategory )
					{
						if(SectionId == INDEX_NONE || CurrentChildNode->SectionID == SectionId)
						{
							if(ChildGraphAction)
							{
								if ((OnActionMatchesName.IsBound() && OnActionMatchesName.Execute(ChildGraphAction, ItemName)) || GraphActionMenuHelpers::ActionMatchesName(ChildGraphAction, ItemName))
								{
									SelectionNode = GraphNodes[i]->Children[ChildIdx];

									break;
								}
							}
							else if (CurrentChildNode->GetDisplayName().ToString() == FName::NameToDisplayString(ItemName.ToString(), false))
							{
								SelectionNode = CurrentChildNode;

								break;
							}
						}
					}
				}
			}
		}

		if(SelectionNode.IsValid())
		{
			// Expand the parent nodes
			for (TSharedPtr<FGraphActionNode> ParentAction = SelectionNode->GetParentNode().Pin(); ParentAction.IsValid(); ParentAction = ParentAction->GetParentNode().Pin())
			{
				TreeView->SetItemExpansion(ParentAction, true);
			}

			// Select the node
			TreeView->SetSelection(SelectionNode,SelectInfo);
			TreeView->RequestScrollIntoView(SelectionNode);
			return true;
		}
	}
	else
	{
		TreeView->ClearSelection();
		return true;
	}
	return false;
}

void SGraphActionMenu::ExpandCategory(const FText& CategoryName)
{
	if (!CategoryName.IsEmpty())
	{
		TArray<TSharedPtr<FGraphActionNode>> GraphNodes;
		FilteredRootAction->GetAllNodes(GraphNodes);
		for (int32 i = 0; i < GraphNodes.Num(); ++i)
		{
			if (GraphNodes[i]->GetDisplayName().EqualTo(CategoryName))
			{
				GraphNodes[i]->ExpandAllChildren(TreeView);
			}
		}
	}
}

static bool CompareGraphActionNode(TSharedPtr<FGraphActionNode> A, TSharedPtr<FGraphActionNode> B)
{
	check(A.IsValid());
	check(B.IsValid());

	// First check grouping is the same
	if (A->GetDisplayName().ToString() != B->GetDisplayName().ToString())
	{
		return false;
	}

	if (A->SectionID != B->SectionID)
	{
		return false;
	}

	if (A->HasValidAction() && B->HasValidAction())
	{
		return A->GetPrimaryAction()->GetMenuDescription().CompareTo(B->GetPrimaryAction()->GetMenuDescription()) == 0;
	}
	else if(!A->HasValidAction() && !B->HasValidAction())
	{
		return true;
	}
	else
	{
		return false;
	}
}

template<typename ItemType, typename ComparisonType> 
void RestoreExpansionState(TSharedPtr< STreeView<ItemType> > InTree, const TArray<ItemType>& ItemSource, const TSet<ItemType>& OldExpansionState, ComparisonType ComparisonFunction)
{
	check(InTree.IsValid());

	// Iterate over new tree items
	for(int32 ItemIdx=0; ItemIdx<ItemSource.Num(); ItemIdx++)
	{
		ItemType NewItem = ItemSource[ItemIdx];

		// Look through old expansion state
		for (typename TSet<ItemType>::TConstIterator OldExpansionIter(OldExpansionState); OldExpansionIter; ++OldExpansionIter)
		{
			const ItemType OldItem = *OldExpansionIter;
			// See if this matches this new item
			if(ComparisonFunction(OldItem, NewItem))
			{
				// It does, so expand it
				InTree->SetItemExpansion(NewItem, true);
			}
		}
	}
}

void SGraphActionMenu::UpdateForNewActions(int32 IdxStart)
{
	check(bAlphaSortItems && bSortItemsRecursively);

	ScoreAndAddActions(IdxStart);
	MarkActiveSuggestion();

	if (ShouldExpandNodes())
	{
		// Expand all
		FilteredRootAction->ExpandAllChildren(TreeView);
	}

	TreeView->RequestTreeRefresh();
}

void SGraphActionMenu::GenerateFilteredItems(bool bPreserveExpansion)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SGraphActionMenu::GenerateFilteredItems);

	SelectedSuggestionScore = TNumericLimits<float>::Lowest();
	SelectedSuggestionSourceIndex = INDEX_NONE;
	SelectedAction = TSharedPtr<FGraphActionNode>();

	// First, save off current expansion state
	TSet< TSharedPtr<FGraphActionNode> > OldExpansionState;
	if(bPreserveExpansion)
	{
		TreeView->GetExpandedItems(OldExpansionState);
	}

	// Clear the filtered root action
	FilteredRootAction->ClearChildren();

	// Collect the list of always visible sections if any, and force the creation of those sections.
	if ( OnCollectStaticSections.IsBound() )
	{
		TArray<int32> StaticSectionIDs;
		OnCollectStaticSections.Execute(StaticSectionIDs);

		for ( int32 i = 0; i < StaticSectionIDs.Num(); i++ )
		{
			FilteredRootAction->AddSection(0, StaticSectionIDs[i]);
		}
	}
	
	ScoreAndAddActions();

	FilteredRootAction->SortChildren(bAlphaSortItems, bSortItemsRecursively);

	TreeView->RequestTreeRefresh();

	MarkActiveSuggestion();

	if (ShouldExpandNodes())
	{
		// Expand all
		FilteredRootAction->ExpandAllChildren(TreeView);
	}
	else
	{
		// Get _all_ new nodes (flattened tree basically)
		TArray< TSharedPtr<FGraphActionNode> > AllNodes;
		FilteredRootAction->GetAllNodes(AllNodes);

		// Expand to match the old state
		RestoreExpansionState< TSharedPtr<FGraphActionNode> >(TreeView, AllNodes, OldExpansionState, CompareGraphActionNode);
	}
}

// Returns true if the tree should be autoexpanded
bool SGraphActionMenu::ShouldExpandNodes() const
{
	// Expand all the categories that have filter results, or when there are only a few to show
	const bool bFilterActive = !GetFilterText().IsEmpty();
	const bool bOnlyAFewTotal = AllActions->GetNumActions() < 10;

	return bFilterActive || bOnlyAFewTotal || bAutoExpandActionMenu;
}

bool SGraphActionMenu::CanRenameNode(TWeakPtr<FGraphActionNode> InNode) const
{
	if (OnCanRenameSelectedAction.IsBound())
	{
		return OnCanRenameSelectedAction.Execute(InNode);
	}
	return false;
}

void SGraphActionMenu::OnFilterTextChanged( const FText& InFilterText )
{
	GenerateFilteredItems(false);
}

void SGraphActionMenu::OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		TryToSpawnActiveSuggestion();
	}
}

bool SGraphActionMenu::TryToSpawnActiveSuggestion()
{
	TArray< TSharedPtr<FGraphActionNode> > SelectionList = TreeView->GetSelectedItems();

	if (SelectionList.Num() == 1)
	{
		// This isnt really a keypress - its Direct, but its always called from a keypress function. (Maybe pass the selectinfo in ?)
		OnItemSelected( SelectionList[0], ESelectInfo::OnKeyPress );
		return true;
	}
	else if (GetTotalLeafNodes() == 1)
	{
		OnItemSelected( GetFirstAction(), ESelectInfo::OnKeyPress);
		return true;
	}

	return false;
}

void SGraphActionMenu::OnGetChildrenForCategory( TSharedPtr<FGraphActionNode> InItem, TArray< TSharedPtr<FGraphActionNode> >& OutChildren )
{
	if (InItem->Children.Num())
	{
		OutChildren = InItem->Children;
	}
}

void SGraphActionMenu::OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, TWeakPtr< FGraphActionNode > InAction )
{
	if(OnCategoryTextCommitted.IsBound())
	{
		OnCategoryTextCommitted.Execute(NewText, InTextCommit, InAction);
	}
}

void SGraphActionMenu::OnItemScrolledIntoView( TSharedPtr<FGraphActionNode> InActionNode, const TSharedPtr<ITableRow>& InWidget )
{
	if (InActionNode->IsRenameRequestPending())
	{
		InActionNode->BroadcastRenameRequest();
	}
}

TSharedRef<ITableRow> SGraphActionMenu::MakeWidget( TSharedPtr<FGraphActionNode> InItem, const TSharedRef<STableViewBase>& OwnerTable, bool bIsReadOnly )
{
	TSharedPtr<IToolTip> SectionToolTip;

	if ( InItem->IsSectionHeadingNode() )
	{
		if ( OnGetSectionToolTip.IsBound() )
		{
			SectionToolTip = OnGetSectionToolTip.Execute(InItem->SectionID);
		}
	}

	// In the case of FGraphActionNodes that have multiple actions, all of the actions will
	// have the same text as they will have been created at the same point - only the actual
	// action itself will differ, which is why parts of this function only refer to InItem->Actions[0]
	// rather than iterating over the array

	// Create the widget but do not add any content, the widget is needed to pass the IsSelectedExclusively function down to the potential SInlineEditableTextBlock widget
	TSharedPtr< STableRow< TSharedPtr<FGraphActionNode> > > TableRow;
	
	if ( InItem->IsSectionHeadingNode() )
	{
		TableRow = SNew(SCategoryHeaderTableRow< TSharedPtr<FGraphActionNode> >, OwnerTable)
			.ToolTip(SectionToolTip);
	}
	else
	{
		TableRow = SNew(STableRow< TSharedPtr<FGraphActionNode> >, OwnerTable)
			.OnDragDetected(this, &SGraphActionMenu::OnItemDragDetected)
			.ShowSelection(!InItem->IsSeparator())
			.bAllowPreselectedItemActivation(bAllowPreselectedItemActivation);
	}

	TSharedPtr<SHorizontalBox> RowContainer;
	TableRow->SetRowContent
	( 
		SAssignNew(RowContainer, SHorizontalBox)
	);

	TSharedPtr<SWidget> RowContent;
	FMargin RowPadding = FMargin(0, 2);

	if( InItem->IsActionNode() )
	{
		check(InItem->HasValidAction());

		FCreateWidgetForActionData CreateData(&InItem->OnRenameRequest());
		CreateData.Action = InItem->GetPrimaryAction();
		CreateData.HighlightText = TAttribute<FText>(this, &SGraphActionMenu::GetFilterText);
		CreateData.MouseButtonDownDelegate = FCreateWidgetMouseButtonDown::CreateSP( this, &SGraphActionMenu::OnMouseButtonDownEvent );

		if(OnCreateWidgetForAction.IsBound())
		{
			CreateData.IsRowSelectedDelegate = FIsSelected::CreateSP( TableRow.Get(), &STableRow< TSharedPtr<FGraphActionNode> >::IsSelected );
			CreateData.bIsReadOnly = bIsReadOnly;
			CreateData.bHandleMouseButtonDown = false;		//Default to NOT using the delegate. OnCreateWidgetForAction can set to true if we need it
			RowContent = OnCreateWidgetForAction.Execute( &CreateData );
		}
		else
		{
			RowContent = SNew(SDefaultGraphActionWidget, &CreateData);
		}
	}
	else if( InItem->IsCategoryNode() )
	{
		TWeakPtr< FGraphActionNode > WeakItem = InItem;

		// Hook up the delegate for verifying the category action is read only or not
		SGraphActionCategoryWidget::FArguments ReadOnlyArgument;
		if(bIsReadOnly)
		{
			ReadOnlyArgument.IsReadOnly(bIsReadOnly);
		}
		else
		{
			ReadOnlyArgument.IsReadOnly_Lambda([WeakThis = this->AsWeak(), WeakItem]
			{
				const TSharedPtr AsSharedWidget = WeakThis.Pin();
				if (const SGraphActionMenu* Menu = static_cast<SGraphActionMenu*>(AsSharedWidget.Get()))
				{
					return !Menu->CanRenameNode(WeakItem);
				}
				return true;
			});
		}

		TSharedRef<SGraphActionCategoryWidget> CategoryWidget =
			SNew(SGraphActionCategoryWidget, InItem)
			.HighlightText(this, &SGraphActionMenu::GetFilterText)
			.OnTextCommitted(this, &SGraphActionMenu::OnNameTextCommitted, TWeakPtr< FGraphActionNode >(InItem))
			.IsSelected(TableRow.Get(), &STableRow< TSharedPtr<FGraphActionNode> >::IsSelectedExclusively)
			.IsReadOnly(ReadOnlyArgument._IsReadOnly);

		if(!bIsReadOnly)
		{
			InItem->OnRenameRequest().BindSP( CategoryWidget->InlineWidget.Pin().Get(), &SInlineEditableTextBlock::EnterEditingMode );
		}

		RowContent = CategoryWidget;
	}
	else if( InItem->IsSeparator() )
	{
		RowPadding = FMargin(0);

		FText SectionTitle;
		if( OnGetSectionTitle.IsBound() )
		{
			SectionTitle = OnGetSectionTitle.Execute(InItem->SectionID);
		}

		if( SectionTitle.IsEmpty() )
		{
			RowContent = SNew( SVerticalBox )
			.Visibility(EVisibility::HitTestInvisible)

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			// Add some empty space before the line, and a tiny bit after it
			.Padding( 0.0f, 1.f, 0.0f, 1.f )
			[
				SNew(SSeparator)
				.SeparatorImage(FAppStyle::Get().GetBrush("Menu.Separator"))
				.Thickness(1.0f)
			];
		}
		else
		{
			RowContent = SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SRichTextBlock)
				.Text(SectionTitle)
				.TransformPolicy(ETextTransformPolicy::ToUpper)
				.DecoratorStyleSet(&FAppStyle::Get())
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(FMargin(0,0,2,0))
			[
				OnGetSectionWidget.IsBound() ? OnGetSectionWidget.Execute(TableRow.ToSharedRef(), InItem->SectionID) : SNullWidget::NullWidget
			];
		}
	}

	TSharedPtr<SExpanderArrow> ExpanderWidget;
	if (OnCreateCustomRowExpander.IsBound())
	{
		FCustomExpanderData CreateData;
		CreateData.TableRow        = TableRow;
		CreateData.WidgetContainer = RowContainer;

		if (InItem->IsActionNode())
		{
			check(InItem->HasValidAction());
			CreateData.RowAction = InItem->GetPrimaryAction();
		}

		ExpanderWidget = OnCreateCustomRowExpander.Execute(CreateData);
	}
	else 
	{
		ExpanderWidget =
			SNew(SExpanderArrow, TableRow)
			.BaseIndentLevel(DefaultRowExpanderBaseIndentLevel);
	}

	RowContainer->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Right)
	[
		ExpanderWidget.ToSharedRef()
	];

	RowContainer->AddSlot()
	.FillWidth(1.0)
	.Padding(RowPadding)
	[
		RowContent.ToSharedRef()
	];

	return TableRow.ToSharedRef();
}

FText SGraphActionMenu::GetFilterText() const
{
	// If there is an external source for the filter, use that text instead
	if(OnGetFilterText.IsBound())
	{
		return OnGetFilterText.Execute();
	}
	
	return FilterTextBox->GetText();
}

void SGraphActionMenu::OnItemSelected( TSharedPtr< FGraphActionNode > InSelectedItem, ESelectInfo::Type SelectInfo )
{
	if (!bIgnoreUIUpdate)
	{
		HandleSelection(InSelectedItem, SelectInfo);
	}
}

void SGraphActionMenu::OnItemDoubleClicked( TSharedPtr< FGraphActionNode > InClickedItem )
{
	if ( InClickedItem.IsValid() && !bIgnoreUIUpdate )
	{
		if ( InClickedItem->IsActionNode() )
		{
			OnActionDoubleClicked.ExecuteIfBound({InClickedItem->Action});
		}
		else if (InClickedItem->Children.Num())
		{
			TreeView->SetItemExpansion(InClickedItem, !TreeView->IsItemExpanded(InClickedItem));
		}
	}
}

FReply SGraphActionMenu::OnItemDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// Start a function-call drag event for any entry that can be called by kismet
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = TreeView->GetSelectedItems();
		if(SelectedNodes.Num() > 0)
		{
			TSharedPtr<FGraphActionNode> Node = SelectedNodes[0];
			// Dragging a ctaegory
			if(Node.IsValid() && Node->IsCategoryNode())
			{
				if(OnCategoryDragged.IsBound())
				{
					return OnCategoryDragged.Execute(Node->GetCategoryPath(), MouseEvent);
				}
			}
			// Dragging an action
			else
			{
				if(OnActionDragged.IsBound())
				{
					TArray< TSharedPtr<FEdGraphSchemaAction> > Actions;
					GetSelectedActions(Actions);
					return OnActionDragged.Execute(Actions, MouseEvent);
				}
			}
		}
	}

	return FReply::Unhandled();
}

bool SGraphActionMenu::OnMouseButtonDownEvent( TWeakPtr<FEdGraphSchemaAction> InAction )
{
	bool bResult = false;
	if( (!bIgnoreUIUpdate) && InAction.IsValid() && OnActionSelected.IsBound())
	{
		OnActionSelected.Execute({InAction.Pin()}, ESelectInfo::OnMouseClick);
		bResult = true;
	}
	return bResult;
}

FReply SGraphActionMenu::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent )
{
	int32 SelectionDelta = 0;
	bIsKeyboardNavigating = false;

	// Escape dismisses the menu without placing a node
	if (KeyEvent.GetKey() == EKeys::Escape)
	{
		FSlateApplication::Get().DismissAllMenus();
		return FReply::Handled();
	}
	else if ((KeyEvent.GetKey() == EKeys::Enter) && !bIgnoreUIUpdate)
	{
		return TryToSpawnActiveSuggestion() ? FReply::Handled() : FReply::Unhandled();
	}
	else if (!FilterTextBox->GetText().IsEmpty())
	{
		// Needs to be done here in order not to eat up the text navigation key events when list isn't populated
		if (GetTotalLeafNodes() == 0)
		{
			return FReply::Unhandled();
		}

		if (KeyEvent.GetKey() == EKeys::Up)
		{
			SelectPreviousAction();
		}
		else if (KeyEvent.GetKey() == EKeys::Down)
		{
			SelectNextAction();
		}
		else if (KeyEvent.GetKey() == EKeys::PageUp)
		{
			const int32 NumItemsInAPage = 15; // arbitrary jump because we can't get at the visible item count from here
			SelectPreviousAction(NumItemsInAPage);
		}
		else if (KeyEvent.GetKey() == EKeys::PageDown)
		{
			const int32 NumItemsInAPage = 15; // arbitrary jump because we can't get at the visible item count from here
			SelectNextAction(NumItemsInAPage);
		}
		else if (KeyEvent.GetKey() == EKeys::Home && KeyEvent.IsControlDown())
		{
			SelectFirstAction();
		}
		else if (KeyEvent.GetKey() == EKeys::End && KeyEvent.IsControlDown())
		{
			SelectLastAction();
		}
		else
		{
			return FReply::Unhandled();
		}

		MarkActiveSuggestion();
		return FReply::Handled();
	}
	else
	{
		// When all else fails, it means we haven't filtered the list and we want to handle it as if we were just scrolling through a normal tree view
		return TreeView->OnKeyDown(FindChildGeometry(MyGeometry, TreeView.ToSharedRef()), KeyEvent);
	}
}

void SGraphActionMenu::MarkActiveSuggestion()
{
	TGuardValue<bool> PreventSelectionFromTriggeringCommit(bIgnoreUIUpdate, true);

	if (SelectedAction.IsValid())
	{
		TreeView->SetSelection(SelectedAction);
		int32 Idx = FilteredRootAction->GetLinearizedIndex(SelectedAction);
		TreeView->SetScrollOffset(FMath::Max(((float)Idx) - (float)DisplayIndex, 0.f));
	}
	else
	{
		TreeView->ClearSelection();
	}
}

void SGraphActionMenu::AddReferencedObjects( FReferenceCollector& Collector )
{
	for (int32 CurTypeIndex = 0; CurTypeIndex < AllActions->GetNumActions(); ++CurTypeIndex)
	{
		TSharedPtr<FEdGraphSchemaAction> Action = AllActions->GetSchemaAction(CurTypeIndex);
		Action->AddReferencedObjects(Collector);
	}
}

FString SGraphActionMenu::GetReferencerName() const
{
	return TEXT("SGraphActionMenu");
}

bool SGraphActionMenu::HandleSelection( TSharedPtr< FGraphActionNode > &InSelectedItem, ESelectInfo::Type InSelectionType )
{
	bool bResult = false;
	if( OnActionSelected.IsBound() )
	{
		if ( InSelectedItem.IsValid() && InSelectedItem->IsActionNode() )
		{
			OnActionSelected.Execute({InSelectedItem->Action}, InSelectionType);
			bResult = true;
		}
		else
		{
			OnActionSelected.Execute(TArray< TSharedPtr<FEdGraphSchemaAction> >(), InSelectionType);
			bResult = true;
		}
	}
	return bResult;
}

void SGraphActionMenu::OnSetExpansionRecursive(TSharedPtr<FGraphActionNode> InTreeNode, bool bInIsItemExpanded)
{
	if (InTreeNode.IsValid() && InTreeNode->Children.Num())
	{
		TreeView->SetItemExpansion(InTreeNode, bInIsItemExpanded);

		for (TSharedPtr<FGraphActionNode> Child : InTreeNode->Children)
		{
			OnSetExpansionRecursive(Child, bInIsItemExpanded);
		}
	}
}

void SGraphActionMenu::ScoreAndAddActions(int32 StartingIndex)
{
	// Trim and sanitized the filter text (so that it more likely matches the action descriptions)
	FString TrimmedFilterString = FText::TrimPrecedingAndTrailing(GetFilterText()).ToString();

	// Remember the last filter string to that external clients can access it
	LastUsedFilterText = TrimmedFilterString;

	// Tokenize the search box text into a set of terms; all of them must be present to pass the filter
	TArray<FString> FilterTerms;
	TrimmedFilterString.ParseIntoArray(FilterTerms, TEXT(" "), true);
	for (FString& String : FilterTerms)
	{
		String = String.ToLower();
	}

	// Generate a list of sanitized versions of the strings
	TArray<FString> SanitizedFilterTerms;
	for (int32 iFilters = 0; iFilters < FilterTerms.Num(); iFilters++)
	{
		FString EachString = FName::NameToDisplayString(FilterTerms[iFilters], false);
		EachString = EachString.Replace(TEXT(" "), TEXT(""));
		SanitizedFilterTerms.Add(EachString);
	}
	ensure(SanitizedFilterTerms.Num() == FilterTerms.Num());// Both of these should match !

	const bool bRequiresFiltering = FilterTerms.Num() > 0 && !bIsKeyboardNavigating;
	float BestMatchCount = SelectedSuggestionScore;
	int32 BestMatchIndex = SelectedSuggestionSourceIndex;

	// Get the schema of the graph that we are in so that we can correctly get the action weight
	const UEdGraphSchema* ActionSchema = GraphObj ? GraphObj->GetSchema() : GetDefault<UEdGraphSchema>();
	check(ActionSchema);

	bool bIsPartialBuild = StartingIndex != INDEX_NONE;
	const int32 NumActions = AllActions->GetNumActions();
	for (int32 CurTypeIndex = bIsPartialBuild ? StartingIndex : 0; CurTypeIndex < NumActions; ++CurTypeIndex)
	{
		TSharedPtr<FEdGraphSchemaAction> CurrentAction = AllActions->GetSchemaAction(CurTypeIndex);

		// If we're filtering, search check to see if we need to show this action
		bool bShowAction = true;
		float EachWeight = TNumericLimits<float>::Lowest();
		const bool bFuzzyFiltering = ActionSchema->AllowsFuzzyActionFiltering(*CurrentAction);

		// only filter out per strict rule if the schema doesn't rely on fuzzy filter weights
		if (!bFuzzyFiltering)
		{
			const FString& SearchText = CurrentAction->GetFullSearchText();
			for (int32 FilterIndex = 0; (FilterIndex < FilterTerms.Num()) && bShowAction; ++FilterIndex)
			{
				const bool bMatchesTerm = (SearchText.Contains(FilterTerms[FilterIndex], ESearchCase::CaseSensitive) || (SearchText.Contains(SanitizedFilterTerms[FilterIndex], ESearchCase::CaseSensitive) == true));
				bShowAction = bMatchesTerm;
			}

			if (!bShowAction)
			{
				continue;
			}
		}

		bool bAddedAndSelected = false;
		if (bRequiresFiltering)
		{
			// Get the 'weight' of this in relation to the filter
			EachWeight = ActionSchema->GetActionFilteredWeight(*CurrentAction, FilterTerms, SanitizedFilterTerms, DraggedFromPins);

			// skip this action if the weight is too low
			if (bFuzzyFiltering && EachWeight <= SMALL_NUMBER)
	        {
			    continue;
            }
			
			// If this action has a greater relevance than others, cache its index.
			if (EachWeight > BestMatchCount)
			{
				BestMatchCount = EachWeight;
				BestMatchIndex = CurTypeIndex;

				// as our currently best scoring entry, add and select the node:
				if (bIsPartialBuild)
				{
					SelectedAction = FilteredRootAction->AddChildAlphabetical(CurrentAction);
				}
				else
				{
					SelectedAction = FilteredRootAction->AddChild(CurrentAction);
				}
				bAddedAndSelected = true;
			}
		}

		if (!bAddedAndSelected) // if the node was not added and selected, then just add it to the root:
		{
			if (bIsPartialBuild)
			{
				FilteredRootAction->AddChildAlphabetical(CurrentAction);
			}
			else
			{
				FilteredRootAction->AddChild(CurrentAction);
			}
		}
	}

	SelectedSuggestionScore = BestMatchCount;
	SelectedSuggestionSourceIndex = BestMatchIndex;
}

void SGraphActionMenu::SelectPreviousAction(int32 Num)
{
	// search backwards Num entries for a previous action, stop if we reach the first action:
	bIsKeyboardNavigating = true;
	int32 SelectedIndex = INDEX_NONE;
	const TArray< TSharedPtr<FGraphActionNode> >& CurrentFilteredActionNodes = GetFilteredActionNodes(&SelectedIndex);
	SelectedIndex = FMath::Max(0, SelectedIndex - Num);
	SelectedAction = CurrentFilteredActionNodes.Num() > 0 ? CurrentFilteredActionNodes[SelectedIndex] : TSharedPtr<FGraphActionNode>();;
	int32 NextIndex = DisplayIndex - Num;
	DisplayIndex = NextIndex < UE::GraphEditor::Private::PREFERRED_TOP_INDEX ? UE::GraphEditor::Private::PREFERRED_BOTTOM_INDEX : NextIndex; // if we bump below 2, loop over to 10, causing a scroll
}

void SGraphActionMenu::SelectNextAction(int32 Num)
{
	// search forwards Num entries for a next action, stop if we reach the first action:
	bIsKeyboardNavigating = true;
	int32 SelectedIndex = INDEX_NONE;
	const TArray< TSharedPtr<FGraphActionNode> >& CurrentFilteredActionNodes = GetFilteredActionNodes(&SelectedIndex);
	SelectedIndex = FMath::Min(CurrentFilteredActionNodes.Num() - 1, SelectedIndex + Num);
	SelectedAction = CurrentFilteredActionNodes.Num() > 0 ? CurrentFilteredActionNodes[SelectedIndex] : TSharedPtr<FGraphActionNode>();;
	int32 NextIndex = DisplayIndex + Num;
	DisplayIndex = NextIndex > UE::GraphEditor::Private::PREFERRED_BOTTOM_INDEX ? UE::GraphEditor::Private::PREFERRED_TOP_INDEX : NextIndex; // if we bump below 2, loop over to 10, causing a scroll
}

void SGraphActionMenu::SelectFirstAction()
{
	bIsKeyboardNavigating = true;
	DisplayIndex = UE::GraphEditor::Private::PREFERRED_TOP_INDEX;
	SelectedAction = GetFirstAction();
}

void SGraphActionMenu::SelectLastAction()
{
	bIsKeyboardNavigating = true;
	DisplayIndex = UE::GraphEditor::Private::PREFERRED_TOP_INDEX;
	// find the last unfiltered action:
	const TArray< TSharedPtr<FGraphActionNode> >& CurrentFilteredActionNodes = GetFilteredActionNodes();
	SelectedAction = CurrentFilteredActionNodes.Num() > 0 ? CurrentFilteredActionNodes.Last() : TSharedPtr<FGraphActionNode>();
}

TSharedPtr<FGraphActionNode> SGraphActionMenu::GetFirstAction()
{
	const TArray< TSharedPtr<FGraphActionNode> >& Nodes = GetFilteredActionNodes();
	return Nodes.Num() > 0 ? FilteredActionNodes[0] : TSharedPtr<FGraphActionNode>();
}

const TArray< TSharedPtr<FGraphActionNode> >& SGraphActionMenu::GetFilteredActionNodes(int32* OutSelectedIndex)
{
	// We could cache this, but for now I'm calculating it every time it is requested - this
	// possibility of caching is the reason none of these methods are const
	FilteredActionNodes.Reset();
	FilteredRootAction->GetLeafNodes(FilteredActionNodes);
	ensureMsgf(GetTotalLeafNodes() == FilteredActionNodes.Num(), TEXT("FilteredActionNodes and GetTotalLeafNodes should match"));
	// find selected item's leaf index:
	if (OutSelectedIndex && SelectedAction)
	{
		for (int32 Idx = 0; Idx < FilteredActionNodes.Num(); ++Idx)
		{
			if (FilteredActionNodes[Idx] == SelectedAction)
			{
				*OutSelectedIndex = Idx;
			}
		}
	}
	return FilteredActionNodes;
}

int32 SGraphActionMenu::GetTotalLeafNodes() const
{
	return FilteredRootAction->GetTotalLeafNodes();
}


/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
