// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphEditorActionMenu.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformMath.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Types/SlateStructs.h"
#include "Widgets/Layout/SBox.h"

SGraphEditorActionMenu::~SGraphEditorActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
}

void SGraphEditorActionMenu::Construct( const FArguments& InArgs )
{


	this->GraphObj = InArgs._GraphObj;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->NewNodePosition = InArgs._NewNodePosition;
	this->OnClosedCallback = InArgs._OnClosedCallback;
	this->AutoExpandActionMenu = InArgs._AutoExpandActionMenu;
	this->OnCreateWidgetForAction = InArgs._OnCreateWidgetForAction;

	// Build the widget layout
	SBorder::Construct( SBorder::FArguments()
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		.Padding(5.0f)
		[
			// Achieving fixed width by nesting items within a fixed width box.
			SNew(SBox)
			.WidthOverride(400.0f)
			.HeightOverride(400.0f)
			[
				SAssignNew(GraphActionMenu, SGraphActionMenu)
				.OnActionSelected(this, &SGraphEditorActionMenu::OnActionSelected)
				.OnCollectAllActions(this, &SGraphEditorActionMenu::CollectAllActions)
				.AutoExpandActionMenu(AutoExpandActionMenu)
				.DraggedFromPins(DraggedFromPins)
				.GraphObj(GraphObj)
				.OnCreateWidgetForAction(OnCreateWidgetForAction)
			]
		]
	);
}

void SGraphEditorActionMenu::RefreshAllActions()
{
	if (GraphActionMenu)
	{
		constexpr bool bPreserveExpansion = true;
		constexpr bool bHandleOnSelectionEvent = false;
		GraphActionMenu->RefreshAllActions(bPreserveExpansion, bHandleOnSelectionEvent);
	}
}

void SGraphEditorActionMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	// Build up the context object
	FGraphContextMenuBuilder ContextMenuBuilder(GraphObj);
	if (DraggedFromPins.Num() > 0)
	{
		ContextMenuBuilder.FromPin = DraggedFromPins[0];
	}

	// Determine all possible actions
	GraphObj->GetSchema()->GetGraphContextActions(ContextMenuBuilder);

	// Copy the added options back to the main list
	//@TODO: Avoid this copy
	OutAllActions.Append(ContextMenuBuilder);
}

TSharedRef<SEditableTextBox> SGraphEditorActionMenu::GetFilterTextBox()
{
	return GraphActionMenu->GetFilterTextBox();
}


void SGraphEditorActionMenu::OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType )
{
	if (InSelectionType == ESelectInfo::OnMouseClick  || InSelectionType == ESelectInfo::OnKeyPress || SelectedAction.Num() == 0)
	{
		bool bDoDismissMenus = true;

		if ( GraphObj != NULL )
		{
			for ( int32 ActionIndex = 0; ActionIndex < SelectedAction.Num(); ActionIndex++ )
			{
				TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedAction[ActionIndex];

				if ( CurrentAction.IsValid() )
				{
					if ( bDoDismissMenus )
					{
						FSlateApplication::Get().DismissAllMenus();
						bDoDismissMenus = false;
					}

					CurrentAction->PerformAction(GraphObj, DraggedFromPins, NewNodePosition);
				}
			}
		}
	}
}
