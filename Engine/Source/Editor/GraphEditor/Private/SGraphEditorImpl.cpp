// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphEditorImpl.h"

#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "MaterialGraph/MaterialGraphNode_Root.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "GenericPlatform/GenericApplication.h"
#include "GraphEditAction.h"
#include "GraphEditorActions.h"
#include "GraphEditorModule.h"
#include "GraphEditorSettings.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Modules/ModuleManager.h"
#include "SGraphEditorActionMenu.h"
#include "SGraphPanel.h"
#include "SNodePanel.h"
#include "ScopedTransaction.h"
#include "SlateGlobals.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuMisc.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "HAL/IConsoleManager.h"
#include "RenderUtils.h"
#include "Materials/Material.h"
#include "Widgets/SInvalidationPanel.h"

class FActiveTimerHandle;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "GraphEditorModule"

FVector2f GetNodeSize(const SGraphEditor& GraphEditor, const UEdGraphNode* Node)
{
	FSlateRect Rect;
	if (GraphEditor.GetBoundsForNode(Node, Rect, 0.f))
	{
		return Rect.GetSize();
	}

	return Node->GetSize();
}

/////////////////////////////////////////////////////
// FAlignmentHelper

FAlignmentData FAlignmentHelper::GetAlignmentDataForNode(UEdGraphNode* Node)
{
	float PropertyOffset = 0.f;

	const float NodeSize = Orientation == Orient_Horizontal ? GetNodeSize(*GraphEditor, Node).X : GetNodeSize(*GraphEditor, Node).Y;
	switch (AlignType)
	{
	case EAlignType::Minimum:	PropertyOffset = 0.f;			break;
	case EAlignType::Middle:	PropertyOffset = NodeSize * .5f;	break;
	case EAlignType::Maximum:	PropertyOffset = NodeSize;		break;
	}
	int32* Property = Orientation == Orient_Horizontal ? &Node->NodePosX : &Node->NodePosY;
	return FAlignmentData(Node, *Property, PropertyOffset);
}

/////////////////////////////////////////////////////
// SGraphEditorImpl

UE::Slate::FDeprecateVector2DResult SGraphEditorImpl::GetPasteLocation2f() const
{
	return GraphPanel->GetPastePosition();
}

bool SGraphEditorImpl::IsNodeTitleVisible( const UEdGraphNode* Node, bool bEnsureVisible )
{
	return GraphPanel->IsNodeTitleVisible(Node, bEnsureVisible);
}

void SGraphEditorImpl::JumpToNode( const UEdGraphNode* JumpToMe, bool bRequestRename, bool bSelectNode )
{
	GraphPanel->JumpToNode(JumpToMe, bRequestRename, bSelectNode);
	FocusLockedEditorHere();
}


void SGraphEditorImpl::JumpToPin( const UEdGraphPin* JumpToMe )
{
	GraphPanel->JumpToPin(JumpToMe);
	FocusLockedEditorHere();
}


bool SGraphEditorImpl::SupportsKeyboardFocus() const
{
	return true;
}

FReply SGraphEditorImpl::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	OnFocused.ExecuteIfBound(SharedThis(this));
	return FReply::Handled();
}

FReply SGraphEditorImpl::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if(MouseEvent.IsMouseButtonDown(EKeys::ThumbMouseButton))
	{
		OnNavigateHistoryBack.ExecuteIfBound();
	}
	else if(MouseEvent.IsMouseButtonDown(EKeys::ThumbMouseButton2))
	{
		OnNavigateHistoryForward.ExecuteIfBound();
	}
	return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Mouse);
}

FReply SGraphEditorImpl::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!MouseEvent.GetCursorDelta().IsZero())
	{
		NumNodesAddedSinceLastPointerPosition = 0;
	}

	return SGraphEditor::OnMouseMove(MyGeometry, MouseEvent);
}

FReply SGraphEditorImpl::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	int32 NumNodes = GetCurrentGraph()->Nodes.Num();
	if (Commands->ProcessCommandBindings( InKeyEvent ) )
	{
		bool bPasteOperation = InKeyEvent.IsControlDown() && InKeyEvent.GetKey() == EKeys::V;

		if(	!bPasteOperation && GetCurrentGraph()->Nodes.Num() > NumNodes )
		{
			OnNodeSpawnedByKeymap.ExecuteIfBound();
		}
		return FReply::Handled();
	}
	else
	{
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}
}

void SGraphEditorImpl::NotifyGraphChanged()
{
	GetCurrentGraph()->NotifyGraphChanged();
}

void SGraphEditorImpl::OnGraphChanged(const FEdGraphEditAction& InAction)
{
	const bool bWasAddAction = (InAction.Action & GRAPHACTION_AddNode) != 0;
	if (bWasAddAction)
	{
		NumNodesAddedSinceLastPointerPosition++;
	}
}

void SGraphEditorImpl::GetPinContextMenuActionsForSchema(UToolMenu* InMenu) const
{
	FToolMenuSection& Section = InMenu->FindOrAddSection("EdGraphSchemaPinActions");
	Section.InitSection("EdGraphSchemaPinActions", LOCTEXT("PinActionsMenuHeader", "Pin Actions"), FToolMenuInsert());

	// Select Output/Input Nodes
	{
		FToolUIAction SelectConnectedNodesAction;
		SelectConnectedNodesAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SGraphEditorImpl::ExecuteSelectConnectedNodesFromPin);
		SelectConnectedNodesAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &SGraphEditorImpl::IsSelectConnectedNodesFromPinVisible, EEdGraphPinDirection::EGPD_Output);
		TSharedPtr<FUICommandInfo> SelectAllOutputNodesCommand = FGraphEditorCommands::Get().SelectAllOutputNodes;
		Section.AddMenuEntry(
			SelectAllOutputNodesCommand->GetCommandName(),
			SelectAllOutputNodesCommand->GetLabel(),
			SelectAllOutputNodesCommand->GetDescription(),
			SelectAllOutputNodesCommand->GetIcon(),
			SelectConnectedNodesAction
		);

		SelectConnectedNodesAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &SGraphEditorImpl::IsSelectConnectedNodesFromPinVisible, EEdGraphPinDirection::EGPD_Input);
		TSharedPtr<FUICommandInfo> SelectAllInputNodesCommand = FGraphEditorCommands::Get().SelectAllInputNodes;
		Section.AddMenuEntry(
			SelectAllInputNodesCommand->GetCommandName(),
			SelectAllInputNodesCommand->GetLabel(),
			SelectAllInputNodesCommand->GetDescription(),
			SelectAllInputNodesCommand->GetIcon(),
			SelectConnectedNodesAction
		);
	}

	auto GetMenuEntryForPin = [](const UEdGraphPin* TargetPin, const FToolUIAction& Action, const FText& SingleDescFormat, const FText& MultiDescFormat, TMap< FString, uint32 >& LinkTitleCount)
	{
		FText Title = TargetPin->GetOwningNode()->GetNodeTitle(ENodeTitleType::ListView);
		const FText PinDisplayName = TargetPin->GetDisplayName();
		if (!PinDisplayName.IsEmpty())
		{
			// Add name of connection if possible
			FFormatNamedArguments Args;
			Args.Add(TEXT("NodeTitle"), Title);
			Args.Add(TEXT("PinName"), PinDisplayName);
			Title = FText::Format(LOCTEXT("JumpToDescPin", "{NodeTitle} ({PinName})"), Args);
		}

		uint32& Count = LinkTitleCount.FindOrAdd(Title.ToString());

		FText Description;
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), Title);
		Args.Add(TEXT("NumberOfNodes"), Count);

		if (Count == 0)
		{
			Description = FText::Format(SingleDescFormat, Args);
		}
		else
		{
			Description = FText::Format(MultiDescFormat, Args);
		}
		++Count;

		return FToolMenuEntry::InitMenuEntry(
			NAME_None,
			Description,
			Description,
			FSlateIcon(),
			Action);
	};

	// Break all pin links
	{
		FToolUIAction BreakPinLinksAction;
		BreakPinLinksAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SGraphEditorImpl::ExecuteBreakPinLinks);
		BreakPinLinksAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &SGraphEditorImpl::IsBreakPinLinksVisible);

		TSharedPtr<FUICommandInfo> BreakPinLinksCommand = FGraphEditorCommands::Get().BreakPinLinks;
		Section.AddMenuEntry(
			BreakPinLinksCommand->GetCommandName(),
			BreakPinLinksCommand->GetLabel(),
			BreakPinLinksCommand->GetDescription(),
			BreakPinLinksCommand->GetIcon(),
			BreakPinLinksAction
		);
	}

	// Break Specific Links
	{
		FToolUIAction BreakLinksMenuVisibility;
		BreakLinksMenuVisibility.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &SGraphEditorImpl::IsBreakPinLinksVisible);

		Section.AddSubMenu("BreakLinkTo",
			LOCTEXT("BreakLinkTo", "Break Link To..."),
			LOCTEXT("BreakSpecificLinks", "Break a specific link..."),
			FNewToolMenuDelegate::CreateLambda([this, GetMenuEntryForPin](UToolMenu* InToolMenu)
			{
				UGraphNodeContextMenuContext* NodeContext = InToolMenu->FindContext<UGraphNodeContextMenuContext>();
				if (NodeContext && NodeContext->Pin)
				{
					FText SingleDescFormat = LOCTEXT("BreakDesc", "Break Link to {NodeTitle}");
					FText MultiDescFormat = LOCTEXT("BreakDescMulti", "Break Link to {NodeTitle} ({NumberOfNodes})");

					TMap< FString, uint32 > LinkTitleCount;
					for (UEdGraphPin* TargetPin : NodeContext->Pin->LinkedTo)
					{
						FToolUIAction BreakSpecificLinkAction;
						BreakSpecificLinkAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this, TargetPin](const FToolMenuContext& InMenuContext)
						{
							UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>();
							check(NodeContext && NodeContext->Pin);
							NodeContext->Pin->GetSchema()->BreakSinglePinLink(const_cast<UEdGraphPin*>(NodeContext->Pin), TargetPin);
						});

						InToolMenu->AddMenuEntry(NAME_None, GetMenuEntryForPin(TargetPin, BreakSpecificLinkAction, SingleDescFormat, MultiDescFormat, LinkTitleCount));
					}
				}
			}),
			BreakLinksMenuVisibility,
			EUserInterfaceActionType::Button);

		// Break This Link
		{
			FToolUIAction BreakThisLinkAction;
			BreakThisLinkAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SGraphEditorImpl::ExecuteBreakPinLinks);
			BreakThisLinkAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &SGraphEditorImpl::IsBreakThisLinkVisible);

			TSharedPtr<FUICommandInfo> BreakThisLinkCommand = FGraphEditorCommands::Get().BreakThisLink;
			Section.AddMenuEntry(
				BreakThisLinkCommand->GetCommandName(),
				BreakThisLinkCommand->GetLabel(),
				BreakThisLinkCommand->GetDescription(),
				BreakThisLinkCommand->GetIcon(),
				BreakThisLinkAction
			);
		}
	}

	FToolUIAction PinActionSubMenuVisibiliity;
	PinActionSubMenuVisibiliity.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &SGraphEditorImpl::HasAnyLinkedPins);
	
	// Jump to specific connections
	{
		Section.AddSubMenu("JumpToConnection",
			LOCTEXT("JumpToConnection", "Jump to Connection..."),
			LOCTEXT("JumpToSpecificConnection", "Jump to specific connection..."),
			FNewToolMenuDelegate::CreateLambda([this, GetMenuEntryForPin](UToolMenu* InToolMenu)
			{
				UGraphNodeContextMenuContext* NodeContext = InToolMenu->FindContext<UGraphNodeContextMenuContext>();
				check(NodeContext&& NodeContext->Pin);

				const SGraphEditor* ThisAsGraphEditor = this;
				FText SingleDescFormat = LOCTEXT("JumpDesc", "Jump to {NodeTitle}");
				FText MultiDescFormat = LOCTEXT("JumpDescMulti", "Jump to {NodeTitle} ({NumberOfNodes})");
				TMap< FString, uint32 > LinkTitleCount;
				for (UEdGraphPin* TargetPin : NodeContext->Pin->LinkedTo)
				{
					FToolUIAction JumpToConnectionAction;
					JumpToConnectionAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([ThisAsGraphEditor, TargetPin](const FToolMenuContext& InMenuContext)
					{
						const_cast<SGraphEditor*>(ThisAsGraphEditor)->JumpToPin(TargetPin);
					});

					InToolMenu->AddMenuEntry(NAME_None, GetMenuEntryForPin(TargetPin, JumpToConnectionAction, SingleDescFormat, MultiDescFormat, LinkTitleCount));
				}
			}),
			PinActionSubMenuVisibiliity,
			EUserInterfaceActionType::Button);
	}

	// Straighten Connections menu items
	{
		Section.AddSubMenu("StraightenPinConnection",
			LOCTEXT("StraightenConnection", "Straighten Connection..."),
			LOCTEXT("StraightenSpecificConnection", "Straighten specific connection..."),
			FNewToolMenuDelegate::CreateLambda([this, GetMenuEntryForPin](UToolMenu* InToolMenu)
			{
				const UGraphNodeContextMenuContext* NodeContext = InToolMenu->FindContext<UGraphNodeContextMenuContext>();
				
				// Add straighten all connected pins
				{
					FToolUIAction StraightenConnectionsAction;
					StraightenConnectionsAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& Context)
					{
						UGraphNodeContextMenuContext* NodeContext = Context.FindContext<UGraphNodeContextMenuContext>();
						StraightenConnections(const_cast<UEdGraphPin*>(NodeContext->Pin), nullptr);
					});
					StraightenConnectionsAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSP(this, &SGraphEditorImpl::HasAnyLinkedPins);

					InToolMenu->AddMenuEntry(
						NAME_None,
						FToolMenuEntry::InitMenuEntry(NAME_None,
							LOCTEXT("StraightenAllConnections", "Straighten All Pin Connections"),
							LOCTEXT("StraightenAllConnectionsTooltip", "Straightens all connected pins"),
							FGraphEditorCommands::Get().StraightenConnections->GetIcon(),
							StraightenConnectionsAction)
					);
				}

				check(NodeContext && NodeContext->Pin);

				// Add individual pin connections
				FText SingleDescFormat = LOCTEXT("StraightenDesc", "Straighten Connection to {NodeTitle}");
				FText MultiDescFormat = LOCTEXT("StraightenDescMulti", "Straighten Connection to {NodeTitle} ({NumberOfNodes})");
				TMap< FString, uint32 > LinkTitleCount;
				for (UEdGraphPin* TargetPin : NodeContext->Pin->LinkedTo)
				{
					FToolUIAction StraightenConnectionAction;
					StraightenConnectionAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this, TargetPin](const FToolMenuContext& InMenuContext)
						{
							UGraphNodeContextMenuContext* NodeContext = InMenuContext.FindContext<UGraphNodeContextMenuContext>();
							if (NodeContext && NodeContext->Pin)
							{
								this->StraightenConnections(const_cast<UEdGraphPin*>(NodeContext->Pin), const_cast<UEdGraphPin*>(TargetPin));
							}
						});

					InToolMenu->AddMenuEntry(NAME_None, GetMenuEntryForPin(TargetPin, StraightenConnectionAction, SingleDescFormat, MultiDescFormat, LinkTitleCount));
				}
			}),
			PinActionSubMenuVisibiliity,
			EUserInterfaceActionType::Button);
	}

	// Add any additional menu options from the asset toolkit that owns this graph editor
	UAssetEditorToolkitMenuContext* AssetToolkitContext = InMenu->FindContext<UAssetEditorToolkitMenuContext>();
	if (AssetToolkitContext && AssetToolkitContext->Toolkit.IsValid())
	{
		AssetToolkitContext->Toolkit.Pin()->AddGraphEditorPinActionsToContextMenu(Section);
	}
}

void SGraphEditorImpl::ExecuteBreakPinLinks(const FToolMenuContext& InContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InContext.FindContext<UGraphNodeContextMenuContext>();
	if (NodeContext && NodeContext->Pin)
	{
		const UEdGraphSchema* Schema = NodeContext->Pin->GetSchema();
		Schema->BreakPinLinks(const_cast<UEdGraphPin&>(*(NodeContext->Pin)), true);
	}
}

bool SGraphEditorImpl::IsBreakPinLinksVisible(const FToolMenuContext& InContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InContext.FindContext<UGraphNodeContextMenuContext>();
	if (NodeContext && NodeContext->Pin)
	{
		return !NodeContext->bIsDebugging && (NodeContext->Pin->LinkedTo.Num() > 1);
	}

	return false;
}

bool SGraphEditorImpl::IsBreakThisLinkVisible(const FToolMenuContext& InContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InContext.FindContext<UGraphNodeContextMenuContext>();
	if (NodeContext && NodeContext->Pin)
	{
		return !NodeContext->bIsDebugging && (NodeContext->Pin->LinkedTo.Num() == 1);
	}

	return false;
}

bool SGraphEditorImpl::HasAnyLinkedPins(const FToolMenuContext& InContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InContext.FindContext<UGraphNodeContextMenuContext>();
	if (NodeContext && NodeContext->Pin)
	{
		return (NodeContext->Pin->LinkedTo.Num() > 0);
	}

	return false;
}

void SGraphEditorImpl::ExecuteSelectConnectedNodesFromPin(const FToolMenuContext& InContext) const
{
	UGraphNodeContextMenuContext* NodeContext = InContext.FindContext<UGraphNodeContextMenuContext>();
	if (NodeContext && NodeContext->Pin)
	{
		SelectAllNodesInDirection(NodeContext->Pin);
	}
}

void SGraphEditorImpl::SelectAllNodesInDirection(const UEdGraphPin* InGraphPin) const
{
	/** Traverses the node graph out from the specified pin, logging each node that it visits along the way. */
	struct FDirectionalNodeVisitor
	{
		FDirectionalNodeVisitor(const UEdGraphPin* StartingPin, EEdGraphPinDirection TargetDirection)
			: Direction(TargetDirection)
		{
			TraversePin(StartingPin);
		}

		/** If the pin is the right direction, visits each of its attached nodes */
		void TraversePin(const UEdGraphPin* Pin)
		{
			if (Pin->Direction == Direction)
			{
				for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
				{
					VisitNode(LinkedPin->GetOwningNode());
				}
			}
		}

		/** If the node has already been visited, does nothing. Otherwise it traverses each of its pins. */
		void VisitNode(const UEdGraphNode* Node)
		{
			bool bAlreadyVisited = false;
			VisitedNodes.Add(Node, &bAlreadyVisited);

			if (!bAlreadyVisited)
			{
				for (UEdGraphPin* Pin : Node->Pins)
				{
					TraversePin(Pin);
				}
			}
		}

		EEdGraphPinDirection Direction;
		TSet<const UEdGraphNode*>  VisitedNodes;
	};

	FDirectionalNodeVisitor NodeVisitor(InGraphPin, InGraphPin->Direction);
	for (const UEdGraphNode* Node : NodeVisitor.VisitedNodes)
	{
		const_cast<SGraphEditorImpl*>(this)->SetNodeSelection(const_cast<UEdGraphNode*>(Node), true);
	}
}

bool SGraphEditorImpl::IsSelectConnectedNodesFromPinVisible(const FToolMenuContext& InContext, EEdGraphPinDirection DirectionToSelect) const
{
	UGraphNodeContextMenuContext* NodeContext = InContext.FindContext<UGraphNodeContextMenuContext>();
	if (NodeContext && NodeContext->Pin)
	{
		return (NodeContext->Pin->Direction == DirectionToSelect) && NodeContext->Pin->HasAnyConnections();
	}

	return false;
}

//void SGraphEditorImpl::GraphEd_OnPanelUpdated()
//{
//	
//}

const TSet<class UObject*>& SGraphEditorImpl::GetSelectedNodes() const
{
	return GraphPanel->SelectionManager.GetSelectedNodes();
}

void SGraphEditorImpl::ClearSelectionSet()
{
	GraphPanel->SelectionManager.ClearSelectionSet();
}

void SGraphEditorImpl::SetNodeSelection(UEdGraphNode* Node, bool bSelect)
{
	GraphPanel->SelectionManager.SetNodeSelection(Node, bSelect);
}

void SGraphEditorImpl::SelectAllNodes()
{
	FGraphPanelSelectionSet NewSet;
	for (int32 NodeIndex = 0; NodeIndex < EdGraphObj->Nodes.Num(); ++NodeIndex)
	{
		UEdGraphNode* Node = EdGraphObj->Nodes[NodeIndex];
		if (Node)
		{
			ensureMsgf(Node->IsValidLowLevel(), TEXT("Node is invalid"));
			NewSet.Add(Node);
		}
	}
	GraphPanel->SelectionManager.SetSelectionSet(NewSet);
}

UEdGraphPin* SGraphEditorImpl::GetGraphPinForMenu()
{
	return GraphPinForMenu.Get();
}

UEdGraphNode* SGraphEditorImpl::GetGraphNodeForMenu()
{
	return GraphNodeForMenu.Get();
}

void SGraphEditorImpl::ZoomToFit(bool bOnlySelection)
{
	GraphPanel->ZoomToFit(bOnlySelection);
}
bool SGraphEditorImpl::GetBoundsForSelectedNodes( class FSlateRect& Rect, float Padding )
{
	return GraphPanel->GetBoundsForSelectedNodes(Rect, Padding);
}

bool SGraphEditorImpl::GetBoundsForNode( const UEdGraphNode* InNode, class FSlateRect& Rect, float Padding) const
{
	FVector2f TopLeft, BottomRight;
	if (GraphPanel->GetBoundsForNode(InNode, TopLeft, BottomRight, Padding))
	{
		Rect.Left = TopLeft.X;
		Rect.Top = TopLeft.Y;
		Rect.Bottom = BottomRight.Y;
		Rect.Right = BottomRight.X;
		return true;
	}
	return false;
}

void SGraphEditorImpl::StraightenConnections()
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().StraightenConnections->GetLabel());
	GraphPanel->StraightenConnections();
}

void SGraphEditorImpl::StraightenConnections(UEdGraphPin* SourcePin, UEdGraphPin* PinToAlign) const
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().StraightenConnections->GetLabel());
	GraphPanel->StraightenConnections(SourcePin, PinToAlign);
}

void SGraphEditorImpl::RefreshNode(UEdGraphNode& Node)
{
	GraphPanel->RefreshNode(Node);
}

void SGraphEditorImpl::Construct( const FArguments& InArgs )
{
	Commands = MakeShareable( new FUICommandList() );
	IsEditable = InArgs._IsEditable;
	DisplayAsReadOnly = InArgs._DisplayAsReadOnly;
	Appearance = InArgs._Appearance;
	TitleBar	= InArgs._TitleBar;
	bAutoExpandActionMenu = InArgs._AutoExpandActionMenu;
	ShowGraphStateOverlay = InArgs._ShowGraphStateOverlay;

	OnNavigateHistoryBack = InArgs._OnNavigateHistoryBack;
	OnNavigateHistoryForward = InArgs._OnNavigateHistoryForward;
	OnNodeSpawnedByKeymap = InArgs._GraphEvents.OnNodeSpawnedByKeymap;
	OnTickDelegate = InArgs._OnTick;

	NumNodesAddedSinceLastPointerPosition = 0;

	// Make sure that the editor knows about what kinds
	// of commands GraphEditor can do.
	FGraphEditorCommands::Register();

	// Tell GraphEditor how to handle all the known commands
	{
		Commands->MapAction( FGraphEditorCommands::Get().ReconstructNodes,
			FExecuteAction::CreateSP( this, &SGraphEditorImpl::ReconstructNodes ),
			FCanExecuteAction::CreateSP( this, &SGraphEditorImpl::CanReconstructNodes )
		);

		Commands->MapAction( FGraphEditorCommands::Get().BreakNodeLinks,
			FExecuteAction::CreateSP( this, &SGraphEditorImpl::BreakNodeLinks ),
			FCanExecuteAction::CreateSP( this, &SGraphEditorImpl::CanBreakNodeLinks )
		);

		Commands->MapAction(FGraphEditorCommands::Get().SummonCreateNodeMenu,
			FExecuteAction::CreateSP(this, &SGraphEditorImpl::SummonCreateNodeMenu),
			FCanExecuteAction::CreateSP(this, &SGraphEditorImpl::CanSummonCreateNodeMenu)
		);

		Commands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
			FExecuteAction::CreateSP(this, &SGraphEditorImpl::StraightenConnections)
		);

		// Append any additional commands that a consumer of GraphEditor wants us to be aware of.
		const TSharedPtr<FUICommandList>& AdditionalCommands = InArgs._AdditionalCommands;
		if ( AdditionalCommands.IsValid() )
		{
			Commands->Append( AdditionalCommands.ToSharedRef() );
		}

	}

	bResetMenuContext = false;
	GraphPinForMenu.SetPin(nullptr);
	EdGraphObj = InArgs._GraphToEdit;

	OnFocused = InArgs._GraphEvents.OnFocused;
	OnCreateActionMenuAtLocation = InArgs._GraphEvents.OnCreateActionMenuAtLocation;
	OnCreateNodeOrPinMenu = InArgs._GraphEvents.OnCreateNodeOrPinMenu;
	
	struct Local
	{
		static FText GetCornerText(TAttribute<FGraphAppearanceInfo> Appearance, FText DefaultText)
		{
			FText OverrideText = Appearance.Get().CornerText;
			return !OverrideText.IsEmpty() ? OverrideText : DefaultText;
		}

		static FText GetPIENotifyText(TAttribute<FGraphAppearanceInfo> Appearance, FText DefaultText)
		{
			FText OverrideText = Appearance.Get().PIENotifyText;
			return !OverrideText.IsEmpty() ? OverrideText : DefaultText;
		}

		static FText GetReadOnlyText(TAttribute<FGraphAppearanceInfo>Appearance, FText DefaultText)
		{
			FText OverrideText = Appearance.Get().ReadOnlyText;
			return !OverrideText.IsEmpty() ? OverrideText : DefaultText;
		}

		static FText GetWarningText(TAttribute<FGraphAppearanceInfo> Appearance, FText DefaultText)
		{
			FText OverrideText = Appearance.Get().WarningText;
			return !OverrideText.IsEmpty() ? OverrideText : DefaultText;
		}
	};

	FText DefaultCornerText = Appearance.Get().CornerText;
	TAttribute<FText> CornerText = Appearance.IsBound() ?
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&Local::GetCornerText, Appearance, DefaultCornerText)) :
		TAttribute<FText>(DefaultCornerText);
	
	FText DefaultPIENotify(LOCTEXT("GraphSimulatingText", "SIMULATING"));
	TAttribute<FText> PIENotifyText = Appearance.IsBound() ?
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&Local::GetPIENotifyText, Appearance, DefaultPIENotify)) :
		TAttribute<FText>(DefaultPIENotify);

	FText DefaultReadOnlyText(LOCTEXT("GraphReadOnlyText", "READ-ONLY"));
	TAttribute<FText> ReadOnlyText = Appearance.IsBound() ?
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&Local::GetReadOnlyText, Appearance, DefaultReadOnlyText)) :
		TAttribute<FText>(DefaultReadOnlyText);

	FText DefaultWarningText;
	TAttribute<FText> WarningText = Appearance.IsBound() ?
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&Local::GetWarningText, Appearance, DefaultWarningText)) :
		TAttribute<FText>(DefaultWarningText);

	TSharedPtr<SOverlay> OverlayWidget;

	GraphPanel = SNew(SGraphPanel)
		.GraphObj( EdGraphObj )
		.DiffResults( InArgs._DiffResults)
		.FocusedDiffResult( InArgs._FocusedDiffResult)
		.OnGetContextMenuFor( this, &SGraphEditorImpl::GraphEd_OnGetContextMenuFor )
		.OnSelectionChanged( InArgs._GraphEvents.OnSelectionChanged )
		.OnNodeDoubleClicked( InArgs._GraphEvents.OnNodeDoubleClicked )
		.IsEditable( this, &SGraphEditorImpl::IsGraphEditable )
		.DisplayAsReadOnly( this, &SGraphEditorImpl::DisplayGraphAsReadOnly )
		.OnDropActors(InArgs._GraphEvents.OnDropActors)
		.OnDropStreamingLevels( InArgs._GraphEvents.OnDropStreamingLevels )
		.OnVerifyTextCommit( InArgs._GraphEvents.OnVerifyTextCommit )
		.OnTextCommitted( InArgs._GraphEvents.OnTextCommitted )
		.OnSpawnNodeByShortcutAtLocation( InArgs._GraphEvents.OnSpawnNodeByShortcutAtLocation)
		//.OnUpdateGraphPanel( this, &SGraphEditorImpl::GraphEd_OnPanelUpdated )
		.OnDisallowedPinConnection( InArgs._GraphEvents.OnDisallowedPinConnection )
		.ShowGraphStateOverlay(InArgs._ShowGraphStateOverlay)
		.AllowConnectionSlicing(InArgs._AllowConnectionSlicing)
		.OnDoubleClicked(InArgs._GraphEvents.OnDoubleClicked)
		.OnMouseButtonDown(InArgs._GraphEvents.OnMouseButtonDown)
		.OnNodeSingleClicked(InArgs._GraphEvents.OnNodeSingleClicked);

	TSharedPtr<SWidget> GraphPanelWidget;
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	if (InArgs._GraphSupportsInvalidation)
	{
		GraphPanelWidget = SNew(SInvalidationPanel)
		.UseDynamicInvalidation(true)

		[
			GraphPanel.ToSharedRef()
		];
	}
	else
	{
#endif
		GraphPanelWidget = GraphPanel;
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	}
#endif

	this->ChildSlot
	[
		SAssignNew(OverlayWidget, SOverlay)

		// The graph panel
		+SOverlay::Slot()
		.Expose(GraphPanelSlot)
		[
			GraphPanelWidget.ToSharedRef()
		]

		// Indicator of current zoom level
		+SOverlay::Slot()
		.Padding(5)
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.TextStyle( FAppStyle::Get(), "Graph.ZoomText" )
			.Text( this, &SGraphEditorImpl::GetZoomText )
			.ColorAndOpacity( this, &SGraphEditorImpl::GetZoomTextColorAndOpacity )
		]

		// Title bar - optional
		+SOverlay::Slot()
		.VAlign(VAlign_Top)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				InArgs._TitleBar.IsValid() ? InArgs._TitleBar.ToSharedRef() : (TSharedRef<SWidget>)SNullWidget::NullWidget
			]
			+ SVerticalBox::Slot()
			.Padding(20.f, 20.f, 20.f, 0.f)
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Center)
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(FMargin(10.f, 4.f))
				.BorderImage(FAppStyle::GetBrush(TEXT("Graph.InstructionBackground")))
				.BorderBackgroundColor(this, &SGraphEditorImpl::InstructionBorderColor)
				.HAlign(HAlign_Center)
				.ColorAndOpacity(this, &SGraphEditorImpl::InstructionTextTint)
				.Visibility(this, &SGraphEditorImpl::InstructionTextVisibility)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "Graph.InstructionText")
					.Text(this, &SGraphEditorImpl::GetInstructionText)
				]
			]
		]

		// Bottom-left corner text for Substrate
		+SOverlay::Slot()
		.Padding(10)
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Visibility(EVisibility::Visible)
			.TextStyle(FAppStyle::Get(), "Graph.WarningText")
			.Text(WarningText)
		]

		// Bottom-right corner text indicating the type of tool
		+SOverlay::Slot()
		.Padding(10)
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Visibility( EVisibility::HitTestInvisible )
			.TextStyle( FAppStyle::Get(), "Graph.CornerText" )
			.Text(CornerText)
		]

		// Top-right corner text indicating PIE is active
		+SOverlay::Slot()
		.Padding(20)
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Visibility(this, &SGraphEditorImpl::PIENotification)
			.TextStyle( FAppStyle::Get(), "Graph.SimulatingText" )
			.Text( PIENotifyText )
		]

		// Top-right corner text indicating read only when not simulating
		+ SOverlay::Slot()
		.Padding(20)
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Visibility(this, &SGraphEditorImpl::ReadOnlyVisibility)
			.TextStyle(FAppStyle::Get(), "Graph.CornerText")
			.Text(ReadOnlyText)
		]

		// Bottom-right corner text for notification list position
		+SOverlay::Slot()
		.Padding(15.f)
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		[
			SAssignNew(NotificationListPtr, SNotificationList)
			.Visibility(EVisibility::Visible)
		]
	];

	GraphPanel->RestoreViewSettings(FVector2f::ZeroVector, -1.0f);

	NotifyGraphChanged();
}

EVisibility SGraphEditorImpl::PIENotification( ) const
{
	if(ShowGraphStateOverlay.Get() && (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != NULL))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}
	
SGraphEditorImpl::~SGraphEditorImpl()
{
	if (FocusEditorTimer.IsValid())
	{
		UnRegisterActiveTimer(FocusEditorTimer.Pin().ToSharedRef());
	}
}

void SGraphEditorImpl::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bResetMenuContext)
	{
		GraphPinForMenu.SetPin(nullptr);
		GraphNodeForMenu.Reset();
		bResetMenuContext = false;
	}

	// If locked to another graph editor, and our panel has moved, synchronise the locked graph editor accordingly
	if ((EdGraphObj != NULL) && GraphPanel.IsValid())
	{
		if(GraphPanel->HasMoved() && IsLocked())
		{
			FocusLockedEditorHere();
		}
	}

	if (OnTickDelegate.IsBound())
	{
		OnTickDelegate.Execute(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
}

void SGraphEditorImpl::OnClosedActionMenu()
{
	GraphPanel->OnStopMakingConnection(/*bForceStop=*/ true);
}

void SGraphEditorImpl::AddContextMenuCommentSection(UToolMenu* InMenu) 
{
		UGraphNodeContextMenuContext* Context = InMenu->FindContext<UGraphNodeContextMenuContext>();
		if (!Context)
		{
			return;
		}

		const UEdGraphSchema* GraphSchema = Context->Graph->GetSchema();
		if (!GraphSchema || GraphSchema->GetParentContextMenuName() == NAME_None)
		{
			return;
		}

		// Helper to do the node comment editing
		struct Local
		{
			// Called by the EditableText widget to get the current comment for the node
			static FString GetNodeComment(TWeakObjectPtr<UEdGraphNode> NodeWeakPtr)
			{
				if (UEdGraphNode* SelectedNode = NodeWeakPtr.Get())
				{
					return SelectedNode->NodeComment;
				}
				return FString();
			}

			// Called by the EditableText widget when the user types a new comment for the selected node
			static void OnNodeCommentTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo, TWeakObjectPtr<UEdGraphNode> NodeWeakPtr)
			{
				// Apply the change to the selected actor
				UEdGraphNode* SelectedNode = NodeWeakPtr.Get();
				FString NewString = NewText.ToString();
				if (SelectedNode && !SelectedNode->NodeComment.Equals(NewString, ESearchCase::CaseSensitive))
				{
					// send property changed events
					const FScopedTransaction Transaction(LOCTEXT("EditNodeComment", "Change Node Comment"));
					SelectedNode->Modify();
					FProperty* NodeCommentProperty = FindFProperty<FProperty>(SelectedNode->GetClass(), "NodeComment");
					if (NodeCommentProperty != nullptr)
					{
						SelectedNode->PreEditChange(NodeCommentProperty);

						SelectedNode->NodeComment = NewString;
						SelectedNode->SetMakeCommentBubbleVisible(true);

						FPropertyChangedEvent NodeCommentPropertyChangedEvent(NodeCommentProperty);
						SelectedNode->PostEditChangeProperty(NodeCommentPropertyChangedEvent);
					}
				}

				// Only dismiss all menus if the text was committed via some user action.
				// ETextCommit::Default implies that focus was switched by some other means. If this is because a submenu was opened, we don't want to close all the menus as a consequence.
				if (CommitInfo != ETextCommit::Default)
				{
					FSlateApplication::Get().DismissAllMenus();
				}
			}
		};

		if (!Context->Pin)
		{
			int32 SelectionCount = GraphSchema->GetNodeSelectionCount(Context->Graph);

			if (SelectionCount == 1)
			{
				// Node comment area
				TSharedRef<SHorizontalBox> NodeCommentBox = SNew(SHorizontalBox);

				{
					FToolMenuSection& Section = InMenu->AddSection("GraphNodeComment", LOCTEXT("NodeCommentMenuHeader", "Node Comment"));
					Section.AddEntry(FToolMenuEntry::InitWidget("NodeCommentBox", NodeCommentBox, FText::GetEmpty()));
				}
				TWeakObjectPtr<UEdGraphNode> SelectedNodeWeakPtr = MakeWeakObjectPtr(const_cast<UEdGraphNode*>(ToRawPtr(Context->Node)));

				FText NodeCommentText;
				if (UEdGraphNode* SelectedNode = SelectedNodeWeakPtr.Get())
				{
					NodeCommentText = FText::FromString(SelectedNode->NodeComment);
				}

				const FSlateBrush* NodeIcon = FCoreStyle::Get().GetDefaultBrush();//@TODO: FActorIconFinder::FindIconForActor(SelectedActors(0).Get());

				// Comment label
				NodeCommentBox->AddSlot()
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					.MaxWidth(250.0f)
					.Padding(FMargin(10.0f, 0.0f))
					[
						SNew(SMultiLineEditableTextBox)
						.Text(NodeCommentText)
						.ToolTipText(LOCTEXT("NodeComment_ToolTip", "Comment for this node"))
						.OnTextCommitted_Static(&Local::OnNodeCommentTextCommitted, SelectedNodeWeakPtr)
						.SelectAllTextWhenFocused(true)
						.RevertTextOnEscape(true)
						.AutoWrapText(true)						
						.ModiferKeyForNewLine(EModifierKey::Control)
					];
			}
			else if (SelectionCount > 1)
			{
				struct SCommentUtility
				{
					static void CreateComment(const UEdGraphSchema* Schema, UEdGraph* Graph)
					{
						if (Schema && Graph)
						{
							TSharedPtr<FEdGraphSchemaAction> Action = Schema->GetCreateCommentAction();

							if (Action.IsValid())
							{
								Action->PerformAction(Graph, nullptr, FVector2f());
							}
						}
					}
				};

				FToolMenuSection& Section = InMenu->AddSection("SchemaActionComment", LOCTEXT("MultiCommentHeader", "Comment Group"));
				Section.AddMenuEntry(
					"MultiCommentDesc",
					LOCTEXT("MultiCommentDesc", "Create Comment from Selection"),
					LOCTEXT("CommentToolTip", "Create a resizable comment box around selection."),
					FSlateIcon(),
					FExecuteAction::CreateStatic(SCommentUtility::CreateComment, GraphSchema, const_cast<UEdGraph*>(ToRawPtr(Context->Graph))
				));
			}
		}
}

FName SGraphEditorImpl::GetNodeParentContextMenuName(UClass* InClass)
{
	if (InClass && InClass != UEdGraphNode::StaticClass())
	{
		if (InClass->GetDefaultObject<UEdGraphNode>()->IncludeParentNodeContextMenu())
		{
			if (UClass* SuperClass = InClass->GetSuperClass())
			{
				return GetNodeContextMenuName(SuperClass);
			}
		}
	}

	return NAME_None;
}

FName SGraphEditorImpl::GetNodeContextMenuName(UClass* InClass)
{
	return FName(*(FString(TEXT("GraphEditor.GraphNodeContextMenu.")) + InClass->GetName()));
}

void SGraphEditorImpl::RegisterContextMenu(const UEdGraphSchema* Schema, FToolMenuContext& MenuContext) const
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	const FName SchemaMenuName = Schema->GetContextMenuName();
	UGraphNodeContextMenuContext* Context = MenuContext.FindContext<UGraphNodeContextMenuContext>();

	// Root menu
	// "GraphEditor.GraphContextMenu.Common"
	// contains: GraphSchema->GetContextMenuActions(Menu, Context)
	const FName CommonRootMenuName = "GraphEditor.GraphContextMenu.Common";
	if (!ToolMenus->IsMenuRegistered(CommonRootMenuName))
	{
		ToolMenus->RegisterMenu(CommonRootMenuName);
	}

	bool bDidRegisterGraphSchemaMenu = false;
	const FName EdGraphSchemaContextMenuName = UEdGraphSchema::GetContextMenuName(UEdGraphSchema::StaticClass());
	if (!ToolMenus->IsMenuRegistered(EdGraphSchemaContextMenuName))
	{
		ToolMenus->RegisterMenu(EdGraphSchemaContextMenuName);
		bDidRegisterGraphSchemaMenu = true;
	}

	// Menus for subclasses of EdGraphSchema
	for (UClass* CurrentClass = Schema->GetClass(); CurrentClass && CurrentClass->IsChildOf(UEdGraphSchema::StaticClass()); CurrentClass = CurrentClass->GetSuperClass())
	{
		const UEdGraphSchema* CurrentSchema = CurrentClass->GetDefaultObject<UEdGraphSchema>();
		const FName CheckMenuName = CurrentSchema->GetContextMenuName();

		// Some subclasses of UEdGraphSchema chose not to include UEdGraphSchema's menu
		// Note: menu "GraphEditor.GraphContextMenu.Common" calls GraphSchema->GetContextMenuActions() and adds entry for node comment
		const FName CheckParentName = CurrentSchema->GetParentContextMenuName();

		if (!ToolMenus->IsMenuRegistered(CheckMenuName))
		{
			FName ParentNameToUse = CheckParentName;

			// Connect final menu in chain to the common root
			if (ParentNameToUse == NAME_None)
			{
				ParentNameToUse = CommonRootMenuName;
			}

			ToolMenus->RegisterMenu(CheckMenuName, ParentNameToUse);
		}

		if (CheckParentName == NAME_None)
		{
			break;
		}
	}

	// Now register node menus, which will belong to their schemas
	bool bDidRegisterNodeMenu = false;
	if (Context->Node)
	{
		for (UClass* CurrentClass = Context->Node->GetClass(); CurrentClass && CurrentClass->IsChildOf(UEdGraphNode::StaticClass()); CurrentClass = CurrentClass->GetSuperClass())
		{
			const FName CheckMenuName = GetNodeContextMenuName(CurrentClass);
			const FName CheckParentName = GetNodeParentContextMenuName(CurrentClass);

			if (!ToolMenus->IsMenuRegistered(CheckMenuName))
			{
				FName ParentNameToUse = CheckParentName;

				// Connect final menu in chain to schema's chain of menus
				if (CheckParentName == NAME_None)
				{
					ParentNameToUse = EdGraphSchemaContextMenuName;
				}

				ToolMenus->RegisterMenu(CheckMenuName, ParentNameToUse);
				bDidRegisterNodeMenu = true;
			}

			if (CheckParentName == NAME_None)
			{
				break;
			}
		}
	}

	// Now that all the possible sections have been registered, we can add the dynamic section for the custom schema node actions to override
	if (bDidRegisterGraphSchemaMenu)
	{
		UToolMenu* Menu = ToolMenus->FindMenu(EdGraphSchemaContextMenuName);

		Menu->AddDynamicSection("EdGraphSchemaPinActions", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				UGraphNodeContextMenuContext* NodeContext = InMenu->FindContext<UGraphNodeContextMenuContext>();
				if (NodeContext && NodeContext->Graph && NodeContext->Pin)
				{
					if (TSharedPtr<SGraphEditorImpl> GraphEditor = StaticCastSharedPtr<SGraphEditorImpl>(FindGraphEditorForGraph(NodeContext->Graph)))
					{
						GraphEditor->GetPinContextMenuActionsForSchema(InMenu);
					}
				}
			}));

		Menu->AddDynamicSection("GetContextMenuActions", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if (UGraphNodeContextMenuContext* ContextObject = InMenu->FindContext<UGraphNodeContextMenuContext>())
			{
				if (const UEdGraphSchema* GraphSchema = ContextObject->Graph->GetSchema())
				{
					GraphSchema->GetContextMenuActions(InMenu, ContextObject);
				}
			}
		}));

		Menu->AddDynamicSection("EdGraphSchema", FNewToolMenuDelegate::CreateStatic(&SGraphEditorImpl::AddContextMenuCommentSection)); 

		if(Substrate::IsSubstrateEnabled())
		{
			Menu->AddDynamicSection("SubstrateRootActions", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				if (UGraphNodeContextMenuContext * ContextObject = InMenu->FindContext<UGraphNodeContextMenuContext>())
				{
					if (const UEdGraphSchema* GraphSchema = ContextObject->Graph->GetSchema())
					{
						if (ContextObject->Node->IsA<UMaterialGraphNode_Root>() && !ContextObject->Pin)
						{
							const UMaterialGraphNode_Root* RootNode = Cast<UMaterialGraphNode_Root>(ContextObject->Node);

							TSharedRef<SButton> SubstrateConvertButton = SNew(SButton)
								.Text(LOCTEXT("SubstrateConvertButtonLabel", "\u26A0 Convert To Substrate \u26A0"))
								.ToolTipText(LOCTEXT("SubstrateConvertButtonToolTip", "Convert the root node to a Substrate graph. This material will then use Substrate more advanced features and topology processes.\nTo go back to the previous Non-Substrate material, you can use CTRL+Z or you will need to manually relink the graph."))
								.ButtonStyle(FAppStyle::Get(), "Button")
								.OnClicked_Lambda([RootNode]()
								{
									if (RootNode && RootNode->Material)
									{
										RootNode->Material->UserRequestsConvertMaterialToSubstrateMaterial();
									}
									return FReply::Handled();
								});
						
							FToolMenuSection& SubstrateSection = InMenu->AddSection("Substrate", LOCTEXT("SubstrateMenuHeader", "Substrate"));
							SubstrateSection.AddEntry(FToolMenuEntry::InitWidget("NodeCommentBox", SubstrateConvertButton, FText::GetEmpty()));
						}
					}
				}
			}));
		}

	}

	if (bDidRegisterNodeMenu)
	{
		UToolMenu* Menu = ToolMenus->FindMenu(GetNodeContextMenuName(Context->Node->GetClass()));
		 
		Menu->AddDynamicSection("GetNodeContextMenuActions", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			UGraphNodeContextMenuContext* Context = InMenu->FindContext<UGraphNodeContextMenuContext>();
			if (Context && Context->Node)
			{
				Context->Node->GetNodeContextMenuActions(InMenu, Context);
			}
		}));
	}
}

UToolMenu* SGraphEditorImpl::GenerateContextMenu(const UEdGraphSchema* Schema, FToolMenuContext& MenuContext) const
{
	// Register all the menu's needed
	RegisterContextMenu(Schema, MenuContext);

	UToolMenus* ToolMenus = UToolMenus::Get();
	UGraphNodeContextMenuContext* Context = MenuContext.FindContext<UGraphNodeContextMenuContext>();

	FName MenuName = NAME_None;
	if (Context->Node)
	{
		MenuName = GetNodeContextMenuName(Context->Node->GetClass());
	}
	else
	{
		MenuName = Schema->GetContextMenuName();
	}

	return ToolMenus->GenerateMenu(MenuName, MenuContext);
}

FActionMenuContent SGraphEditorImpl::GraphEd_OnGetContextMenuFor(const FGraphContextMenuArguments& SpawnInfo)
{
	FActionMenuContent Result;

	if (EdGraphObj != NULL)
	{
		Result = FActionMenuContent( SNew(STextBlock) .Text( NSLOCTEXT("GraphEditor", "NoNodes", "No Nodes") ) );

		const UEdGraphSchema* Schema = EdGraphObj->GetSchema();
		check(Schema);

		GraphPinForMenu.SetPin(SpawnInfo.GraphPin);
		GraphNodeForMenu = SpawnInfo.GraphNode;
		
		if ((SpawnInfo.GraphPin != NULL) || (SpawnInfo.GraphNode != NULL))
		{
			// Get all menu extenders for this context menu from the graph editor module
			FGraphEditorModule& GraphEditorModule = FModuleManager::GetModuleChecked<FGraphEditorModule>( TEXT("GraphEditor") );
			TArray<FGraphEditorModule::FGraphEditorMenuExtender_SelectedNode> MenuExtenderDelegates = GraphEditorModule.GetAllGraphEditorContextMenuExtender();

			TArray<TSharedPtr<FExtender>> Extenders;
			for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
			{
				if (MenuExtenderDelegates[i].IsBound())
				{
					Extenders.Add(MenuExtenderDelegates[i].Execute(this->Commands.ToSharedRef(), EdGraphObj, SpawnInfo.GraphNode, SpawnInfo.GraphPin, !IsEditable.Get()));
				}
			}
			TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

			if (OnCreateNodeOrPinMenu.IsBound())
			{
				// Show the menu for the pin or node under the cursor
				const bool bShouldCloseAfterAction = true;
				FMenuBuilder MenuBuilder( bShouldCloseAfterAction, this->Commands, MenuExtender );

				Result = OnCreateNodeOrPinMenu.Execute(EdGraphObj, SpawnInfo.GraphNode, SpawnInfo.GraphPin, &MenuBuilder, !IsEditable.Get());
			}
			else
			{
				UGraphNodeContextMenuContext* ContextObject = NewObject<UGraphNodeContextMenuContext>();
				ContextObject->Init(EdGraphObj, SpawnInfo.GraphNode, SpawnInfo.GraphPin, !IsEditable.Get());

				FToolMenuContext Context(this->Commands, MenuExtender, ContextObject);

				UAssetEditorToolkitMenuContext* ToolkitMenuContext = NewObject<UAssetEditorToolkitMenuContext>();
				ToolkitMenuContext->Toolkit = AssetEditorToolkit;
				Context.AddObject(ToolkitMenuContext);

				if (TSharedPtr<FAssetEditorToolkit> SharedToolKit = AssetEditorToolkit.Pin())
				{
					SharedToolKit->InitToolMenuContext(Context);
				}

				// Need to additionally pass through the asset toolkit to hook up those commands?

				UToolMenus* ToolMenus = UToolMenus::Get();
				UToolMenu* GeneratedMenu = GenerateContextMenu(Schema, Context);
				Result = FActionMenuContent(ToolMenus->GenerateWidget(GeneratedMenu));
			}
		}
		else if (IsEditable.Get())
		{
			if (EdGraphObj->GetSchema() != NULL )
			{
				if(OnCreateActionMenuAtLocation.IsBound())
				{
					Result = OnCreateActionMenuAtLocation.Execute(
						EdGraphObj, 
						SpawnInfo.NodeAddPosition,
						SpawnInfo.DragFromPins,
						bAutoExpandActionMenu, 
						SGraphEditor::FActionMenuClosed::CreateSP(this, &SGraphEditorImpl::OnClosedActionMenu)
						);
				}
				else
				{
					TSharedRef<SGraphEditorActionMenu> Menu =	
						SNew(SGraphEditorActionMenu)
						.GraphObj( EdGraphObj )
						.NewNodePosition(SpawnInfo.NodeAddPosition)
						.DraggedFromPins(SpawnInfo.DragFromPins)
						.AutoExpandActionMenu(bAutoExpandActionMenu)
						.OnClosedCallback( SGraphEditor::FActionMenuClosed::CreateSP(this, &SGraphEditorImpl::OnClosedActionMenu)
						);

					Result = FActionMenuContent( Menu, Menu->GetFilterTextBox() );
				}

				if (SpawnInfo.DragFromPins.Num() > 0)
				{
					GraphPanel->PreservePinPreviewUntilForced();
				}
			}
		}
		else
		{
			Result = FActionMenuContent( SNew(STextBlock)  .Text( NSLOCTEXT("GraphEditor", "CannotCreateWhileDebugging", "Cannot create new nodes in a read only graph") ) );
		}
	}
	else
	{
		Result = FActionMenuContent( SNew(STextBlock) .Text( NSLOCTEXT("GraphEditor", "GraphObjectIsNull", "Graph Object is Null") ) );
	}

	Result.OnMenuDismissed.AddLambda([this]()
	{
		bResetMenuContext = true;
	});

	return Result;
}

bool SGraphEditorImpl::CanReconstructNodes() const
{
	return IsGraphEditable() && (GraphPanel->SelectionManager.AreAnyNodesSelected());
}

bool SGraphEditorImpl::CanBreakNodeLinks() const
{
	return IsGraphEditable() && (GraphPanel->SelectionManager.AreAnyNodesSelected());
}

bool SGraphEditorImpl::CanSummonCreateNodeMenu() const
{
	return IsGraphEditable() && GraphPanel->IsHovered() && GetDefault<UGraphEditorSettings>()->bOpenCreateMenuOnBlankGraphAreas;
}

void SGraphEditorImpl::ReconstructNodes()
{
	const UEdGraphSchema* Schema = this->EdGraphObj->GetSchema();
	{
		FScopedTransaction const Transaction(LOCTEXT("ReconstructNodeTransaction", "Refresh Node(s)"));

		for (FGraphPanelSelectionSet::TConstIterator NodeIt( GraphPanel->SelectionManager.GetSelectedNodes() ); NodeIt; ++NodeIt)
		{
			if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
			{
				const bool bCurDisableOrphanSaving = Node->bDisableOrphanPinSaving;
				Node->bDisableOrphanPinSaving = true;

				Schema->ReconstructNode(*Node);
				Node->ClearCompilerMessage();

				Node->bDisableOrphanPinSaving = bCurDisableOrphanSaving;
			}
		}
	}
	NotifyGraphChanged();
}
	
void SGraphEditorImpl::BreakNodeLinks()
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakNodeLinks", "Break Node Links") );

	for (FGraphPanelSelectionSet::TConstIterator NodeIt( GraphPanel->SelectionManager.GetSelectedNodes() ); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
		{
			const UEdGraphSchema* Schema = Node->GetSchema();
			Schema->BreakNodeLinks(*Node);
		}
	}
}

void SGraphEditorImpl::SummonCreateNodeMenu()
{
	GraphPanel->SummonCreateNodeMenuFromUICommand(NumNodesAddedSinceLastPointerPosition);
}

FText SGraphEditorImpl::GetZoomText() const
{
	return GraphPanel->GetZoomText();
}

FSlateColor SGraphEditorImpl::GetZoomTextColorAndOpacity() const
{
	return GraphPanel->GetZoomTextColorAndOpacity();
}

bool SGraphEditorImpl::IsGraphEditable() const
{
	return (EdGraphObj != NULL) && IsEditable.Get();
}

bool SGraphEditorImpl::DisplayGraphAsReadOnly() const
{
	return (EdGraphObj != NULL) && DisplayAsReadOnly.Get();
}

bool SGraphEditorImpl::IsLocked() const
{
	for( auto LockedGraph : LockedGraphs )
	{
		if( LockedGraph.IsValid() )
		{
			return true;
		}
	}
	return false;
}

TSharedPtr<SWidget> SGraphEditorImpl::GetTitleBar() const
{
	return TitleBar;
}

void SGraphEditorImpl::SetViewLocation( const FVector2f& Location, float ZoomAmount, const FGuid& BookmarkId ) 
{
	if( GraphPanel.IsValid() &&  EdGraphObj && (!IsLocked() || !GraphPanel->HasDeferredObjectFocus()))
	{
		GraphPanel->RestoreViewSettings(Location, ZoomAmount, BookmarkId);
	}
}

void SGraphEditorImpl::GetViewLocation( FVector2f& Location, float& ZoomAmount ) 
{
	if( GraphPanel.IsValid() &&  EdGraphObj && (!IsLocked() || !GraphPanel->HasDeferredObjectFocus()))
	{
		Location = GraphPanel->GetViewOffset();
		ZoomAmount = GraphPanel->GetZoomAmount();
	}
}

void SGraphEditorImpl::GetViewBookmark( FGuid& BookmarkId )
{
	if (GraphPanel.IsValid())
	{
		BookmarkId = GraphPanel->GetViewBookmarkId();
	}
}

void SGraphEditorImpl::LockToGraphEditor( TWeakPtr<SGraphEditor> Other ) 
{
	if( !LockedGraphs.Contains(Other) )
	{
		LockedGraphs.Push(Other);
	}

	if (GraphPanel.IsValid())
	{
		FocusLockedEditorHere();
	}
}

void SGraphEditorImpl::UnlockFromGraphEditor( TWeakPtr<SGraphEditor> Other )
{
	check(Other.IsValid());
	int idx = LockedGraphs.Find(Other);
	if (idx != INDEX_NONE)
	{
		LockedGraphs.RemoveAtSwap(idx);
	}
}

void SGraphEditorImpl::AddNotification( FNotificationInfo& Info, bool bSuccess )
{
	// set up common notification properties
	Info.bUseLargeFont = true;

	TSharedPtr<SNotificationItem> Notification = NotificationListPtr->AddNotification(Info);
	if ( Notification.IsValid() )
	{
		Notification->SetVisibility(EVisibility::HitTestInvisible);
		Notification->SetCompletionState( bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail );
	}
}

TSharedPtr<SNotificationItem> SGraphEditorImpl::AddNotification(FNotificationInfo& Info)
{
	// set up common notification properties
	Info.bUseLargeFont = true;

	TSharedPtr<SNotificationItem> Notification = NotificationListPtr->AddNotification(Info);
	if (Notification.IsValid())
	{
		Notification->SetVisibility(EVisibility::HitTestInvisible);
		return Notification;
	}
	return nullptr;
}

EActiveTimerReturnType SGraphEditorImpl::HandleFocusEditorDeferred(double InCurrentTime, float InDeltaTime)
{
	
	// If GraphPanel is going to pan to a target but hasn't yet, wait until it does so we don't miss it
	if (GraphPanel->HasDeferredZoomDestination())
	{
		return EActiveTimerReturnType::Continue;
	}
	
	for( int i = 0; i < LockedGraphs.Num(); ++i )
	{
		TSharedPtr<SGraphEditor> LockedGraph = LockedGraphs[i].Pin();
		if (LockedGraph != TSharedPtr<SGraphEditor>())
		{
			// If the locked graph is going to pan to a target but hasn't yet, wait until it does so we don't miss it
			if (LockedGraph->GetGraphPanel()->HasDeferredZoomDestination())
			{
				return EActiveTimerReturnType::Continue;
			}

			FVector2f TopLeft, BottomRight;

			// If the locked graph was instructed to pan to a destination, let it ignore the lock to reach that destination.
			// this way we can support diffs of moved nodes.
			if (LockedGraph->GetGraphPanel()->GetZoomTargetRect(TopLeft, BottomRight))
			{
				continue;
			}
			
			// Send the locked graph to the same place as this graph
			if (GraphPanel->GetZoomTargetRect(TopLeft, BottomRight))
			{
				LockedGraph->GetGraphPanel()->JumpToRect(TopLeft, BottomRight);
			}
			else
			{
				LockedGraph->SetViewLocation(GraphPanel->GetViewOffset(), GraphPanel->GetZoomAmount());
			}
		}
		else
		{
			LockedGraphs.RemoveAtSwap(i--);
		}
	}
	return EActiveTimerReturnType::Stop;
}

void SGraphEditorImpl::FocusLockedEditorHere()
{
	if (!FocusEditorTimer.IsValid())
	{
		FocusEditorTimer = RegisterActiveTimer(
		0.f,
		FWidgetActiveTimerDelegate::CreateSP(this, &SGraphEditorImpl::HandleFocusEditorDeferred)
		);
	}
	
}

void SGraphEditorImpl::SetPinVisibility( SGraphEditor::EPinVisibility InVisibility ) 
{
	if( GraphPanel.IsValid())
	{
		SGraphEditor::EPinVisibility CachedVisibility = GraphPanel->GetPinVisibility();
		GraphPanel->SetPinVisibility(InVisibility);
		if(CachedVisibility != InVisibility)
		{
			NotifyGraphChanged();
		}
	}
}

TSharedRef<FActiveTimerHandle> SGraphEditorImpl::RegisterActiveTimer(float TickPeriod, FWidgetActiveTimerDelegate TickFunction)
{
	return SWidget::RegisterActiveTimer(TickPeriod, TickFunction);
}

EVisibility SGraphEditorImpl::ReadOnlyVisibility() const
{
	if(ShowGraphStateOverlay.Get() && PIENotification() == EVisibility::Hidden && !IsEditable.Get())
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

FText SGraphEditorImpl::GetInstructionText() const
{
	if (Appearance.IsBound())
	{
		return Appearance.Get().InstructionText;
	}
	return FText::GetEmpty();
}

EVisibility SGraphEditorImpl::InstructionTextVisibility() const
{
	if (!GetInstructionText().IsEmpty() && (GetInstructionTextFade() > 0.0f))
	{
		return EVisibility::HitTestInvisible;
	}
	return EVisibility::Hidden;
}

float SGraphEditorImpl::GetInstructionTextFade() const
{
	float InstructionOpacity = 1.0f;
	if (Appearance.IsBound())
	{
		InstructionOpacity = Appearance.Get().InstructionFade.Get();
	}
	return InstructionOpacity;
}

FLinearColor SGraphEditorImpl::InstructionTextTint() const
{
	return FLinearColor(1.f, 1.f, 1.f, GetInstructionTextFade());
}

FSlateColor SGraphEditorImpl::InstructionBorderColor() const
{
	FLinearColor BorderColor(0.1f, 0.1f, 0.1f, 0.7f);
	BorderColor.A *= GetInstructionTextFade();
	return BorderColor;
}

void SGraphEditorImpl::CaptureKeyboard()
{
	FSlateApplication::Get().SetKeyboardFocus(GraphPanel);
}

void SGraphEditorImpl::SetNodeFactory(const TSharedRef<class FGraphNodeFactory>& NewNodeFactory)
{
	GraphPanel->SetNodeFactory(NewNodeFactory);
}


void SGraphEditorImpl::OnAlignTop()
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().AlignNodesTop->GetLabel());

	FAlignmentHelper Helper(SharedThis(this), Orient_Vertical, EAlignType::Minimum);
	Helper.Align();
}

void SGraphEditorImpl::OnAlignMiddle()
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().AlignNodesMiddle->GetLabel());

	FAlignmentHelper Helper(SharedThis(this), Orient_Vertical, EAlignType::Middle);
	Helper.Align();
}

void SGraphEditorImpl::OnAlignBottom()
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().AlignNodesBottom->GetLabel());

	FAlignmentHelper Helper(SharedThis(this), Orient_Vertical, EAlignType::Maximum);
	Helper.Align();
}

void SGraphEditorImpl::OnAlignLeft()
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().AlignNodesLeft->GetLabel());

	FAlignmentHelper Helper(SharedThis(this), Orient_Horizontal, EAlignType::Minimum);
	Helper.Align();
}

void SGraphEditorImpl::OnAlignCenter()
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().AlignNodesCenter->GetLabel());

	FAlignmentHelper Helper(SharedThis(this), Orient_Horizontal, EAlignType::Middle);
	Helper.Align();
}

void SGraphEditorImpl::OnAlignRight()
{
	const FScopedTransaction Transaction(FGraphEditorCommands::Get().AlignNodesRight->GetLabel());

	FAlignmentHelper Helper(SharedThis(this), Orient_Horizontal, EAlignType::Maximum);
	Helper.Align();
}

void SGraphEditorImpl::OnStraightenConnections()
{
	StraightenConnections();
}

/** Distribute the specified array of node data evenly */
void DistributeNodes(TArray<FAlignmentData>& InData, bool bIsHorizontal)
{
	// Sort the data
	InData.Sort([](const FAlignmentData& A, const FAlignmentData& B) {
		return A.TargetProperty + FMath::RoundToInt(A.TargetOffset / 2) < B.TargetProperty + FMath::RoundToInt(B.TargetOffset / 2);
	});

	// Measure the available space
	float TotalWidthOfNodes = 0.f;
	for (int32 Index = 1; Index < InData.Num() - 1; ++Index)
	{
		TotalWidthOfNodes += InData[Index].TargetOffset;
	}

	const float SpaceToDistributeIn = static_cast<float>(InData.Last().TargetProperty) - InData[0].GetTarget();
	const float PaddingAmount = ((SpaceToDistributeIn - TotalWidthOfNodes) / static_cast<float>(InData.Num() - 1));

	float TargetPosition = InData[0].GetTarget() + PaddingAmount;

	// Now set all the properties on the target
	if (InData.Num() > 1)
	{
		UEdGraph* Graph = InData[0].Node->GetGraph();
		if (Graph)
		{
			const UEdGraphSchema* Schema = Graph->GetSchema();

			// similar to FAlignmentHelper::Align(), first try using GraphSchema to move the nodes if applicable
			if (Schema)
			{
				for (int32 Index = 1; Index < InData.Num() - 1; ++Index)
				{
					FAlignmentData& Entry = InData[Index];

					FVector2f Target2DPosition = Entry.Node->GetPosition();
 
					if (bIsHorizontal)
					{
						Target2DPosition.X = TargetPosition;
					}
					else
					{ 
						Target2DPosition.Y = TargetPosition;
					}

					Schema->SetNodePosition(Entry.Node, Target2DPosition);

					TargetPosition = Entry.GetTarget() + PaddingAmount;
				} 

				return;
			}
		}

		// fall back to the old approach if there isn't a schema
		for (int32 Index = 1; Index < InData.Num() - 1; ++Index)
		{
			FAlignmentData& Entry = InData[Index];

			Entry.Node->Modify();
			Entry.TargetProperty = UE::LWC::FloatToIntCastChecked<int32>(TargetPosition);

			TargetPosition = Entry.GetTarget() + PaddingAmount;
		}
	}
}

void SGraphEditorImpl::OnDistributeNodesH()
{
	TArray<FAlignmentData> AlignData;
	for (UObject* It : GetSelectedNodes())
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(It))
		{
			AlignData.Add(FAlignmentData(Node, Node->NodePosX, GetNodeSize(*this, Node).X));
		}
	}

	if (AlignData.Num() > 2)
	{
		const FScopedTransaction Transaction(FGraphEditorCommands::Get().DistributeNodesHorizontally->GetLabel());
		DistributeNodes(AlignData, true);
	}
}

void SGraphEditorImpl::OnDistributeNodesV()
{
	TArray<FAlignmentData> AlignData;
	for (UObject* It : GetSelectedNodes())
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(It))
		{
			AlignData.Add(FAlignmentData(Node, Node->NodePosY, GetNodeSize(*this, Node).Y));
		}
	}

	if (AlignData.Num() > 2)
	{
		const FScopedTransaction Transaction(FGraphEditorCommands::Get().DistributeNodesVertically->GetLabel());
		DistributeNodes(AlignData, false);
	}
}

/** Distribute the specified array of node data evenly and align left when aligning vertically, and align top when aligning horizontally */
void StackNodes(TArray<FAlignmentData>& InData, bool bIsHorizontal)
{
	// Sort the data
	InData.Sort([](const FAlignmentData& A, const FAlignmentData& B) {
		return A.TargetProperty < B.TargetProperty;
	});

	float TargetPosition = static_cast<float>(InData[0].TargetProperty);
	float StackAxisPosition = 0.0f;
	float PaddingAmount = static_cast<float>(SNodePanel::GetSnapGridSize());

	if (bIsHorizontal)
	{
		StackAxisPosition = InData[0].Node->GetNodePosY();
		PaddingAmount *= 2.0f;
	}
	else
	{
		StackAxisPosition = InData[0].Node->GetNodePosX();
	}

	// Now set all the properties on the target
	if (InData.Num() > 1)
	{
		UEdGraph* Graph = InData[0].Node->GetGraph();
		if (Graph)
		{
			const UEdGraphSchema* Schema = Graph->GetSchema();

			if (Schema)
			{
				for (int32 Index = 0; Index < InData.Num(); ++Index)
				{
					FAlignmentData& Entry = InData[Index];

					FVector2f AdjustedTargetPosition = Entry.Node->GetPosition();

					if (bIsHorizontal)
					{
						AdjustedTargetPosition.X = TargetPosition;
						AdjustedTargetPosition.Y = StackAxisPosition;
					}
					else
					{
						AdjustedTargetPosition.X = StackAxisPosition;
						AdjustedTargetPosition.Y = TargetPosition;
					}

					Schema->SetNodePosition(Entry.Node, AdjustedTargetPosition);

					TargetPosition = Entry.GetTarget() + PaddingAmount;
				}

				return;
			}
		}

		// fall back to the old approach if there isn't a schema
		for (int32 Index = 0; Index < InData.Num() - 1; ++Index)
		{
			FAlignmentData& Entry = InData[Index];

			Entry.Node->Modify();
			Entry.TargetProperty = UE::LWC::FloatToIntCastChecked<int32>(TargetPosition);

			TargetPosition = Entry.GetTarget() + PaddingAmount;
		}
	}
}

void SGraphEditorImpl::OnStackNodesH()
{
	TArray<FAlignmentData> AlignData;
	for (UObject* It : GetSelectedNodes())
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(It))
		{
			AlignData.Add(FAlignmentData(Node, Node->NodePosX, GetNodeSize(*this, Node).X));
		}
	}

	if (AlignData.Num() > 1)
	{
		const FScopedTransaction Transaction(FGraphEditorCommands::Get().StackNodesHorizontally->GetLabel());
		StackNodes(AlignData, true);
	}
}

void SGraphEditorImpl::OnStackNodesV()
{
	TArray<FAlignmentData> AlignData;
	for (UObject* It : GetSelectedNodes())
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(It))
		{
			AlignData.Add(FAlignmentData(Node, Node->NodePosY, GetNodeSize(*this, Node).Y));
		}
	}

	if (AlignData.Num() > 1)
	{
		const FScopedTransaction Transaction(FGraphEditorCommands::Get().StackNodesVertically->GetLabel());
		StackNodes(AlignData, false);
	}
}

int32 SGraphEditorImpl::GetNumberOfSelectedNodes() const
{
	return GetSelectedNodes().Num();
}

UEdGraphNode* SGraphEditorImpl::GetSingleSelectedNode() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	return (SelectedNodes.Num() == 1) ? Cast<UEdGraphNode>(*SelectedNodes.CreateConstIterator()) : nullptr;
}

SGraphPanel* SGraphEditorImpl::GetGraphPanel() const
{
	return GraphPanel.Get();
}

void SGraphEditorImpl::SetOnTick(FOnGraphEditorTickDelegate InOnTickDelegate)
{
	OnTickDelegate = InOnTickDelegate;
}

/////////////////////////////////////////////////////



#undef LOCTEXT_NAMESPACE 