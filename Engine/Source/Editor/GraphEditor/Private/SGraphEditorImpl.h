// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h" // for FEdGraphPinReference
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SOverlay.h"

class FActiveTimerHandle;
class FUICommandList;
class SGraphPanel;
class SNotificationList;
class SWidget;
class UClass;
class UToolMenu;
struct FDiffSingleResult;
struct FEdGraphEditAction;
struct FFocusEvent;
struct FGeometry;
struct FGraphContextMenuArguments;
struct FKeyEvent;
struct FNotificationInfo;
struct FPointerEvent;
struct FToolMenuContext;


/** Struct used for generically aligning nodes */
struct FAlignmentData
{
	FAlignmentData(UEdGraphNode* InNode, int32& InTargetProperty, float InTargetOffset)
		: Node(InNode), TargetProperty(InTargetProperty), TargetOffset(InTargetOffset)
	{}

	/** The node to position */
	UEdGraphNode* Node;
	/** The property within the node to read/write */
	int32& TargetProperty;
	/** The offset from the property to consider for alignment */
	float TargetOffset;

	/** Get the destination target from this alignment data (property + offset) */
	float GetTarget() const
	{
		return float(TargetProperty) + TargetOffset;
	}
};

enum class EAlignType : uint8
{
	Minimum, Middle, Maximum
};

/** Helper class for aligning nodes */
struct FAlignmentHelper
{
	/** Construct from a graph editor, an orientation, and an alignment type */
	FAlignmentHelper(TSharedRef<SGraphEditor> InGraphEditor, EOrientation InOrientation, EAlignType InAlignType)
		: GraphEditor(MoveTemp(InGraphEditor))
	{
		// We align to the node that was clicked on, if available (not when invoked from a key shortcut)
		CardinalNode = GraphEditor->GetGraphNodeForMenu();

		Orientation = InOrientation;
		AlignType = InAlignType;

		// Collect all the alignment data for all the selected nodes
		for (UObject* It : GraphEditor->GetSelectedNodes())
		{
			if (UEdGraphNode* Node = Cast<UEdGraphNode>(It))
			{
				AlignmentData.Add(GetAlignmentDataForNode(Node));
			}
		}

		// Sort the data based on target - important for future algorithms
		AlignmentData.Sort([](const FAlignmentData& A, const FAlignmentData& B) {
			return A.GetTarget() < B.GetTarget();
		});
	}

	/** Align all the nodes */
	void Align()
	{
		if (AlignmentData.Num() > 1)
		{
			UEdGraph* Graph = AlignmentData[0].Node->GetGraph();

			if (Graph)
			{
				const UEdGraphSchema* Schema = Graph->GetSchema();

				if (Schema)
				{ 
					float Target = DetermineAlignmentTarget();
					
					for (FAlignmentData& Entry : AlignmentData)
					{
						float TargetProperty = Target - Entry.TargetOffset; 
						FVector2f TargetPosition = Entry.Node->GetPosition();

						if (Orientation == EOrientation::Orient_Horizontal)
						{
							TargetPosition.X = TargetProperty;
						}
						else
						{ 
							TargetPosition.Y = TargetProperty;
						}

						Schema->SetNodePosition(Entry.Node, TargetPosition);
					}
				}
			}
		}
	}

private:

	/** Collect alignment data for a given node, based on our settings */
	FAlignmentData GetAlignmentDataForNode(UEdGraphNode* Node);

	/** Determine the horizontal/vertical position that all nodes should align to */
	float DetermineAlignmentTarget()
	{
		if (CardinalNode)
		{
			return GetAlignmentDataForNode(CardinalNode).GetTarget();
		}

		if (AlignType == EAlignType::Minimum)
		{
			float Target = TNumericLimits<float>::Max();
			for (const FAlignmentData& Entry : AlignmentData)
			{
				Target = FMath::Min(Target, Entry.GetTarget());
			}
			return Target;
		}
		else if (AlignType == EAlignType::Maximum)
		{
			float Target = TNumericLimits<float>::Lowest();
			for (const FAlignmentData& Entry : AlignmentData)
			{
				Target = FMath::Max(Target, Entry.GetTarget());
			}
			return Target;
		}
		else
		{
			// Use the mean
			float SumTotal = 0.f;
			for (const FAlignmentData& Entry : AlignmentData)
			{
				SumTotal += Entry.GetTarget();
			}
			return SumTotal / static_cast<float>(AlignmentData.Num());
		}
	}

	/** The graph editor */
	TSharedRef<SGraphEditor> GraphEditor;
	/** Whether we are aligning horizontally/vertically */
	EOrientation Orientation;
	/** Whether we are aligning to the minimum/middle/maximum bounds */
	EAlignType AlignType;
	/** The cardinal node that all other nodes should align to (possibly null) */
	UEdGraphNode* CardinalNode;
	/** Generated alignment data */
	TArray<FAlignmentData> AlignmentData;
};



/////////////////////////////////////////////////////
// SGraphEditorImpl

class SGraphEditorImpl : public SGraphEditor
{
public:
	SLATE_BEGIN_ARGS( SGraphEditorImpl )
		: _AdditionalCommands( TSharedPtr<FUICommandList>() )
		, _IsEditable(true)
		, _DisplayAsReadOnly(false)
		, _AllowConnectionSlicing(true)
		, _GraphSupportsInvalidation(false)
		{}


		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, AdditionalCommands)
		SLATE_ATTRIBUTE( bool, IsEditable )
		SLATE_ATTRIBUTE( bool, DisplayAsReadOnly )
		SLATE_ARGUMENT( TSharedPtr<SWidget>, TitleBar )
		SLATE_ATTRIBUTE( FGraphAppearanceInfo, Appearance )
		SLATE_ARGUMENT( UEdGraph*, GraphToEdit )
		SLATE_ARGUMENT(TSharedPtr<TArray<FDiffSingleResult>>, DiffResults )
		SLATE_ATTRIBUTE(int32, FocusedDiffResult )
		SLATE_ARGUMENT(SGraphEditor::FGraphEditorEvents, GraphEvents)
		SLATE_ARGUMENT(bool, AutoExpandActionMenu)
		SLATE_EVENT(FSimpleDelegate, OnNavigateHistoryBack)
		SLATE_EVENT(FSimpleDelegate, OnNavigateHistoryForward)
		SLATE_EVENT(FOnGraphEditorTickDelegate, OnTick);

		/** Show overlay elements for the graph state such as the PIE and read-only borders and text */
		SLATE_ATTRIBUTE(bool, ShowGraphStateOverlay)
		/** Allow graph connections to be sliced and removed */
		SLATE_ATTRIBUTE(bool, AllowConnectionSlicing)

		/** If set to true, the GraphPanel will be surrounded with a Dynamic Invalidation Panel */
		SLATE_ARGUMENT(bool, GraphSupportsInvalidation)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );
private:
	TSharedPtr< FUICommandList > Commands;
	mutable TSet<UEdGraphNode*> SelectedNodeCache;

	/** The panel contains the GraphNode widgets, draws the connections, etc */
	SOverlay::FOverlaySlot* GraphPanelSlot;
	TSharedPtr<SGraphPanel> GraphPanel;
	TSharedPtr<SWidget>	TitleBar;

	FEdGraphPinReference  GraphPinForMenu;
	TWeakObjectPtr<UEdGraphNode> GraphNodeForMenu;
	bool bResetMenuContext;

	/** Info on the appearance */
	TAttribute<FGraphAppearanceInfo> Appearance;

	SGraphEditor::FOnFocused OnFocused;
	SGraphEditor::FOnCreateActionMenuAtLocation OnCreateActionMenuAtLocation;
	SGraphEditor::FOnCreateNodeOrPinMenu OnCreateNodeOrPinMenu;

	TAttribute<bool> IsEditable;

	/** Attribute for displaying the graph as read-only, which is a visual state only where IsEditable is a functional state */
	TAttribute<bool> DisplayAsReadOnly;

	bool bAutoExpandActionMenu;

	/** Whether to show the state (read only / PIE etc) Overlay on the panel */
	TAttribute<bool> ShowGraphStateOverlay;

	//FOnViewChanged	OnViewChanged;
	TArray< TWeakPtr<SGraphEditor> > LockedGraphs;

	/** Function to check whether PIE is active to display "Simulating" text in graph panel*/
	EVisibility PIENotification( ) const;

	/** Function to check whether we should show read-only text in the panel */
	EVisibility ReadOnlyVisibility() const;

	/** Returns dynamic text, meant to passively instruct the user on what to do in the graph */
	FText GetInstructionText() const;
	/** Function to check whether we should show instruction text to the user */
	EVisibility InstructionTextVisibility() const;
	/** Returns a 0.0 to 1.0 value, denoting the instruction text's fade percent */
	float GetInstructionTextFade() const;
	/** A dynamic tint for the instruction text (allows us to nicely fade it in/out) */
	FLinearColor InstructionTextTint() const;
	/** Determines the color of the box containing the instruction text */
	FSlateColor InstructionBorderColor() const;

	/** Notification list to pass messages to editor users  */
	TSharedPtr<SNotificationList> NotificationListPtr;

	/** Callback to navigate backward in the history */
	FSimpleDelegate OnNavigateHistoryBack;
	/** Callback to navigate forward in the history */
	FSimpleDelegate OnNavigateHistoryForward;

	/** Invoked when a node is created by a keymap */
	FOnNodeSpawnedByKeymap OnNodeSpawnedByKeymap;

	/**
	 * Invoked during the tick of the graph editor. 
	 * Can be used for custom layout code / node UI relevant code
	 */
	FOnGraphEditorTickDelegate OnTickDelegate;

public:
	virtual ~SGraphEditorImpl();

	void OnClosedActionMenu();

	FActionMenuContent GraphEd_OnGetContextMenuFor(const FGraphContextMenuArguments& SpawnInfo);

	//void GraphEd_OnPanelUpdated();

	// SWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	void FocusLockedEditorHere();

	virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual bool SupportsKeyboardFocus() const override;
	// End of SWidget interface

	// SGraphEditor interface
	virtual const TSet<class UObject*>& GetSelectedNodes() const override;
	virtual void ClearSelectionSet() override;
	virtual void SetNodeSelection(UEdGraphNode* Node, bool bSelect) override;
	virtual void SelectAllNodes() override;

	virtual UE::Slate::FDeprecateVector2DResult GetPasteLocation2f() const override;
	virtual bool IsNodeTitleVisible( const UEdGraphNode* Node, bool bRequestRename ) override;
	virtual void JumpToNode( const UEdGraphNode* JumpToMe, bool bRequestRename = false, bool bSelectNode = true ) override;
	virtual void JumpToPin( const UEdGraphPin* JumpToMe ) override;
	virtual UEdGraphPin* GetGraphPinForMenu() override;
	virtual UEdGraphNode* GetGraphNodeForMenu() override;
	virtual void ZoomToFit(bool bOnlySelection) override;
	virtual bool GetBoundsForSelectedNodes( class FSlateRect& Rect, float Padding) override;
	virtual bool GetBoundsForNode( const UEdGraphNode* InNode, class FSlateRect& Rect, float Padding) const override;
	virtual void NotifyGraphChanged() override;
	virtual TSharedPtr<SWidget> GetTitleBar() const override;
	virtual void SetViewLocation(const FVector2f& Location, float ZoomAmount, const FGuid& BookmarkId = FGuid()) override;
	virtual void GetViewLocation(FVector2f& Location, float& ZoomAmount) override;
	virtual void GetViewBookmark(FGuid& BookmarkId) override;
	virtual void LockToGraphEditor(TWeakPtr<SGraphEditor> Other) override;
	virtual void UnlockFromGraphEditor(TWeakPtr<SGraphEditor> Other) override;
	virtual void AddNotification ( FNotificationInfo& Info, bool bSuccess ) override;
	virtual TSharedPtr<SNotificationItem> AddNotification(FNotificationInfo& Info) override;
	virtual void SetPinVisibility(SGraphEditor::EPinVisibility Visibility) override;
	virtual TSharedRef<FActiveTimerHandle> RegisterActiveTimer(float TickPeriod, FWidgetActiveTimerDelegate TickFunction) override;
	virtual void StraightenConnections() override;
	virtual void StraightenConnections(UEdGraphPin* SourcePin, UEdGraphPin* PinToAlign) const override;
	virtual void RefreshNode(UEdGraphNode& Node) override;
	virtual void CaptureKeyboard() override;
	virtual void SetNodeFactory(const TSharedRef<class FGraphNodeFactory>& NewNodeFactory) override;
	virtual void OnAlignTop() override;
	virtual void OnAlignMiddle() override;
	virtual void OnAlignBottom() override;
	virtual void OnAlignLeft() override;
	virtual void OnAlignCenter() override;
	virtual void OnAlignRight() override;

	virtual void OnStraightenConnections() override;

	virtual void OnDistributeNodesH() override;
	virtual void OnDistributeNodesV() override;

	virtual void OnStackNodesH() override;
	virtual void OnStackNodesV() override;

	virtual int32 GetNumberOfSelectedNodes() const override;

	virtual UEdGraphNode* GetSingleSelectedNode() const override;

	virtual SGraphPanel* GetGraphPanel() const override;

	virtual void SetOnTick(FOnGraphEditorTickDelegate InOnTickDelegate) override;
	
	// End of SGraphEditor interface
protected:
	//
	// COMMAND HANDLING
	// 
	bool CanReconstructNodes() const;
	bool CanBreakNodeLinks() const;
	bool CanSummonCreateNodeMenu() const;

	void ReconstructNodes();
	void BreakNodeLinks();

	void SummonCreateNodeMenu();

	// SGraphEditor interface
	virtual void OnGraphChanged( const FEdGraphEditAction& InAction ) override;
	// End of SGraphEditorInterface

private:
	FText GetZoomText() const;

	FSlateColor GetZoomTextColorAndOpacity() const;

	bool IsGraphEditable() const;

	/** Helper function to decide whether to display the graph in a read-only state */
	bool DisplayGraphAsReadOnly() const;

	bool IsLocked() const;
	
	void RegisterContextMenu(const class UEdGraphSchema* Schema, struct FToolMenuContext& MenuContext) const;
	class UToolMenu* GenerateContextMenu(const class UEdGraphSchema* Schema, struct FToolMenuContext& MenuContext) const;

	static FName GetNodeParentContextMenuName(UClass* InClass);
	static FName GetNodeContextMenuName(UClass* InClass);
	static void AddContextMenuCommentSection(UToolMenu* InMenu);

	void GetPinContextMenuActionsForSchema(UToolMenu* InMenu) const;
	void ExecuteBreakPinLinks(const FToolMenuContext& InContext) const;
	bool IsBreakPinLinksVisible(const FToolMenuContext& InContext) const;
	bool IsBreakThisLinkVisible(const FToolMenuContext& InContext) const;
	bool HasAnyLinkedPins(const FToolMenuContext& InContext) const;

	void ExecuteSelectConnectedNodesFromPin(const FToolMenuContext& InContext) const;
	void SelectAllNodesInDirection(const UEdGraphPin* InGraphPin) const;
	bool IsSelectConnectedNodesFromPinVisible(const FToolMenuContext& InContext, EEdGraphPinDirection DirectionToSelect) const;

	EActiveTimerReturnType HandleFocusEditorDeferred(double InCurrentTime, float InDeltaTime);
private:
	TWeakPtr<FActiveTimerHandle> FocusEditorTimer;
	uint32 NumNodesAddedSinceLastPointerPosition;
};

