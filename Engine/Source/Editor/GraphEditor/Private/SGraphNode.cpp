// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphNode.h"

#include "BlueprintEditorSettings.h"
#include "BoneDragDropOp.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/TextLayout.h"
#include "GameFramework/Actor.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorDragDropAction.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "IDocumentation.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_Literal.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Logging/TokenizedMessage.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "NodeFactory.h"
#include "SCommentBubble.h"
#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "SLevelOfDetailBranchNode.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "Textures/SlateIcon.h"
#include "TutorialMetaData.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class UBlueprint;
struct FSlateBrush;

/////////////////////////////////////////////////////
// SNodeTitle

void SNodeTitle::Construct(const FArguments& InArgs, UEdGraphNode* InNode)
{
	GraphNode = InNode;

	ExtraLineStyle = InArgs._ExtraLineStyle;

	StyleSet = InArgs._StyleSet;

	CachedSize = FVector2f::ZeroVector;

	// If the user set the text, use it, otherwise use the node title by default
	if(InArgs._Text.IsSet())
	{
		TitleText = InArgs._Text;
	}
	else
	{
		TitleText = TAttribute<FText>(this, &SNodeTitle::GetNodeTitle);
	}
	NodeTitleCache.SetCachedText(TitleText.Get(), GraphNode.Get());
	RebuildWidget();
}

void SNodeTitle::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	CachedSize = AllottedGeometry.GetLocalSize();

	// Checks to see if the cached string is valid, and if not, updates it.
	if (NodeTitleCache.IsOutOfDate(GraphNode.Get()))
	{
		// Do not rebuild if the text was empty and is still empty
		if (!(NodeTitleCache.GetCachedText().IsEmpty() && TitleText.Get().IsEmpty()))
		{
			NodeTitleCache.SetCachedText(TitleText.Get(), GraphNode.Get());
			RebuildWidget();
		}
	}
}

FText SNodeTitle::GetNodeTitle() const
{
	if (GetDefault<UBlueprintEditorSettings>()->bBlueprintNodeUniqueNames && GraphNode.IsValid())
	{
		return FText::FromName(GraphNode->GetFName());
	}
	else
	{
		return GraphNode.IsValid()
			? GraphNode->GetNodeTitle(ENodeTitleType::FullTitle)
			: NSLOCTEXT("GraphEditor", "NullNode", "Null Node");
	}
}

FText SNodeTitle::GetHeadTitle() const
{
	return (GraphNode.IsValid() && GraphNode->GetCanRenameNode()) ? GraphNode->GetNodeTitle(ENodeTitleType::EditableTitle) : CachedHeadTitle;
}

UE::Slate::FDeprecateVector2DResult SNodeTitle::GetTitleSize() const
{
	return CachedSize;
}

void SNodeTitle::MarkDirty()
{
	NodeTitleCache.MarkDirty();
}

void SNodeTitle::RebuildWidget()
{
	// Create the box to contain the lines
	TSharedPtr<SVerticalBox> VerticalBox;
	this->ChildSlot
	[
		SAssignNew(VerticalBox, SVerticalBox)
	];

	// Break the title into lines
	TArray<FString> Lines;
	const FString CachedTitleString = NodeTitleCache.GetCachedText().ToString().Replace(TEXT("\r"), TEXT(""));
	CachedTitleString.ParseIntoArray(Lines, TEXT("\n"), false);

	if (Lines.Num())
	{
		CachedHeadTitle = FText::FromString(Lines[0]);
	}

	// Pad the height of multi-line node titles to be a multiple of the graph snap grid taller than
	// single-line nodes, so the pins will still line up if you place the node N cell snaps above
	if (Lines.Num() > 1)
	{
		// Note: This code a little fragile, and will need to be updated if the font or padding of titles
		// changes in the future, but the failure mode is just a slight misalignment.
		const int32 EstimatedExtraHeight = (Lines.Num() - 1) * 14;

		const int32 SnapSize = (int32)SNodePanel::GetSnapGridSize();
		const int32 PadSize = SnapSize - (EstimatedExtraHeight % SnapSize);

		if (PadSize < SnapSize)
		{
			VerticalBox->AddSlot()
			[
				SNew(SSpacer)
				.Size(FVector2D(1.0f, PadSize))
			];
		}
	}

	// Make a separate widget for each line, using a less obvious style for subsequent lines
	for (int32 Index = 1; Index < Lines.Num(); ++Index)
	{
		VerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.TextStyle( StyleSet, ExtraLineStyle )
			.Text(FText::FromString(Lines[Index]))
		];
	}
}


/////////////////////////////////////////////////////
// SGraphNode

// Check whether drag and drop functionality is permitted on the given node
bool SGraphNode::CanAllowInteractionUsingDragDropOp( const UEdGraphNode* GraphNodePtr, const TSharedPtr<FActorDragDropOp>& DragDropOp )
{
	bool bReturn = false;

	//Allow interaction only if this node is a literal type object.
	//Only change actor reference if a single actor reference is dragged from the outliner.
	if( GraphNodePtr->IsA( UK2Node_Literal::StaticClass() )  && DragDropOp->Actors.Num() == 1 )
	{
		bReturn = true;
	}
	return bReturn;
}

void SGraphNode::SetIsEditable(TAttribute<bool> InIsEditable)
{
	IsEditable = InIsEditable;
}

bool SGraphNode::IsNodeEditable() const
{
	bool bIsEditable = OwnerGraphPanelPtr.IsValid() ? OwnerGraphPanelPtr.Pin()->IsGraphEditable() : true;
	return IsEditable.Get() && bIsEditable;
}

/** Set event when node is double clicked */
void SGraphNode::SetDoubleClickEvent(FSingleNodeEvent InDoubleClickEvent)
{
	OnDoubleClick = InDoubleClickEvent;
}

void SGraphNode::SetVerifyTextCommitEvent(FOnNodeVerifyTextCommit InOnVerifyTextCommit)
{
	OnVerifyTextCommit = InOnVerifyTextCommit;
}

void SGraphNode::SetTextCommittedEvent(FOnNodeTextCommitted InOnTextCommitted)
{
	OnTextCommitted = InOnTextCommitted;
}

void SGraphNode::OnCommentTextCommitted(const FText& NewComment, ETextCommit::Type CommitInfo)
{
	GetNodeObj()->OnUpdateCommentText(NewComment.ToString());
}

void SGraphNode::OnCommentBubbleToggled(bool bInCommentBubbleVisible)
{
	GetNodeObj()->OnCommentBubbleToggled(bInCommentBubbleVisible);
}

void SGraphNode::SetDisallowedPinConnectionEvent(SGraphEditor::FOnDisallowedPinConnection InOnDisallowedPinConnection)
{
	OnDisallowedPinConnection = InOnDisallowedPinConnection;
}

void SGraphNode::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return;
	}

	// Is someone dragging a connection?
	if (Operation->IsOfType<FGraphEditorDragDropAction>())
	{
		// Inform the Drag and Drop operation that we are hovering over this pin.
		TSharedPtr<FGraphEditorDragDropAction> DragConnectionOp = StaticCastSharedPtr<FGraphEditorDragDropAction>(Operation);
		DragConnectionOp->SetHoveredNode( SharedThis(this) );
	}
	else if (Operation->IsOfType<FActorDragDropGraphEdOp>())
	{
		TSharedPtr<FActorDragDropGraphEdOp> DragConnectionOp = StaticCastSharedPtr<FActorDragDropGraphEdOp>(Operation);
		if( GraphNode->IsA( UK2Node_Literal::StaticClass() ) )
		{
			//Show tool tip only if a single actor is dragged
			if( DragConnectionOp->Actors.Num() == 1 )
			{
				UK2Node_Literal* LiteralNode = CastChecked< UK2Node_Literal > ( GraphNode );

				//Check whether this node is already referencing the same actor dragged from outliner
				if( LiteralNode->GetObjectRef() != DragConnectionOp->Actors[0].Get() )
				{
					DragConnectionOp->SetToolTip(FActorDragDropGraphEdOp::ToolTip_Compatible);
				}
			}
			else
			{
				//For more that one actor dragged on to a literal node, show tooltip as incompatible
				DragConnectionOp->SetToolTip(FActorDragDropGraphEdOp::ToolTip_MultipleSelection_Incompatible);
			}
		}
		else
		{
			DragConnectionOp->SetToolTip( (DragConnectionOp->Actors.Num() == 1) ? FActorDragDropGraphEdOp::ToolTip_Incompatible : FActorDragDropGraphEdOp::ToolTip_MultipleSelection_Incompatible);
		}
	}
	else if (Operation->IsOfType<FBoneDragDropOp>())
	{
		//@TODO: A2REMOVAL: No support for A3 nodes handling this drag-drop op yet!
	}
}

void SGraphNode::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return;
	}

	// Is someone dragging a connection?
	if (Operation->IsOfType<FGraphEditorDragDropAction>())
	{
		// Inform the Drag and Drop operation that we are not hovering any pins
		TSharedPtr<FGraphEditorDragDropAction> DragConnectionOp = StaticCastSharedPtr<FGraphEditorDragDropAction>(Operation);
		DragConnectionOp->SetHoveredNode( TSharedPtr<SGraphNode>(NULL) );
	}
	else if (Operation->IsOfType<FActorDragDropGraphEdOp>())
	{
		//Default tool tip
		TSharedPtr<FActorDragDropGraphEdOp> DragConnectionOp = StaticCastSharedPtr<FActorDragDropGraphEdOp>(Operation);
		DragConnectionOp->ResetToDefaultToolTip();
	}
	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> AssetOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
		AssetOp->ResetToDefaultToolTip();
	}
	else if (Operation->IsOfType<FBoneDragDropOp>())
	{
		//@TODO: A2REMOVAL: No support for A3 nodes handling this drag-drop op yet!
	}
}

FReply SGraphNode::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FAssetDragDropOp> AssetOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (AssetOp.IsValid())
	{
		if(GraphNode != NULL && GraphNode->GetSchema() != NULL)
		{
			bool bOkIcon = false;
			FString TooltipText;
			if (AssetOp->HasAssets())
			{
				GraphNode->GetSchema()->GetAssetsNodeHoverMessage(AssetOp->GetAssets(), GraphNode, TooltipText, bOkIcon);
			}
			bool bReadOnly = OwnerGraphPanelPtr.IsValid() ? !OwnerGraphPanelPtr.Pin()->IsGraphEditable() : false;
			bOkIcon = bReadOnly ? false : bOkIcon;
			const FSlateBrush* TooltipIcon = bOkIcon ? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")) : FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));;
			AssetOp->SetToolTip(FText::FromString(TooltipText), TooltipIcon);
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

/** Given a coordinate in SGraphPanel space (i.e. panel widget space), return the same coordinate in graph space while taking zoom and panning into account */
UE::Slate::FDeprecateVector2DResult SGraphNode::NodeCoordToGraphCoord( const UE::Slate::FDeprecateVector2DParameter& NodeSpaceCoordinate ) const
{
	TSharedPtr<SGraphPanel> OwnerCanvas = OwnerGraphPanelPtr.Pin();
	if (OwnerCanvas.IsValid())
	{
		//@TODO: NodeSpaceCoordinate != PanelCoordinate
		FVector2f PanelSpaceCoordinate = NodeSpaceCoordinate;
		return OwnerCanvas->PanelCoordToGraphCoord( PanelSpaceCoordinate );
	}
	else
	{
		return FVector2f::ZeroVector;
	}
}

FReply SGraphNode::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	bool bReadOnly = OwnerGraphPanelPtr.IsValid() ? !OwnerGraphPanelPtr.Pin()->IsGraphEditable() : false;
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid() || bReadOnly)
	{
		return FReply::Unhandled();
	}

	// Is someone dropping a connection onto this node?
	if (Operation->IsOfType<FGraphEditorDragDropAction>())
	{
		TSharedPtr<FGraphEditorDragDropAction> DragConnectionOp = StaticCastSharedPtr<FGraphEditorDragDropAction>(Operation);

		const FVector2f NodeAddPosition = NodeCoordToGraphCoord( MyGeometry.AbsoluteToLocal( DragDropEvent.GetScreenSpacePosition() ) );

		FReply Result = DragConnectionOp->DroppedOnNode(DragDropEvent.GetScreenSpacePosition(), NodeAddPosition);
		
		if (Result.IsEventHandled() && (GraphNode != NULL))
		{
			GraphNode->GetGraph()->NotifyGraphChanged();
		}
		return Result;
	}
	else if (Operation->IsOfType<FActorDragDropGraphEdOp>())
	{
		TSharedPtr<FActorDragDropGraphEdOp> DragConnectionOp = StaticCastSharedPtr<FActorDragDropGraphEdOp>(Operation);
		if( CanAllowInteractionUsingDragDropOp( GraphNode, DragConnectionOp ) )
		{
			UK2Node_Literal* LiteralNode = CastChecked< UK2Node_Literal > ( GraphNode );

			//Check whether this node is already referencing the same actor
			if( LiteralNode->GetObjectRef() != DragConnectionOp->Actors[0].Get() )
			{
				//Replace literal node's object reference
				LiteralNode->SetObjectRef( DragConnectionOp->Actors[ 0 ].Get() );

				UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(CastChecked<UEdGraph>(GraphNode->GetOuter()));
				if (Blueprint != nullptr)
				{
					FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
				}
			}
		}
		return FReply::Handled();
	}
	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		UEdGraphNode* Node = GetNodeObj();
		if(Node != NULL && Node->GetSchema() != NULL)
		{
			TSharedPtr<FAssetDragDropOp> AssetOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
			if (AssetOp->HasAssets())
			{
				Node->GetSchema()->DroppedAssetsOnNode(AssetOp->GetAssets(), DragDropEvent.GetScreenSpacePosition(), Node);
			}
		}
		return FReply::Handled();
	} 
	else if (Operation->IsOfType<FBoneDragDropOp>())
	{
		//@TODO: A2REMOVAL: No support for A3 nodes handling this drag-drop op yet!
	}
	return FReply::Unhandled();
}

/**
 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
 *
 * @param MyGeometry The Geometry of the widget receiving the event
 * @param MouseEvent Information about the input event
 *
 * @return Whether the event was handled along with possible requests for the system to take action.
 */
FReply SGraphNode::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return FReply::Unhandled();
}

// The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
FReply SGraphNode::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return FReply::Unhandled();
}

// Called when a mouse button is double clicked.  Override this in derived classes
FReply SGraphNode::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	if(InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		OnDoubleClick.ExecuteIfBound(GraphNode);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

TSharedPtr<IToolTip> SGraphNode::GetToolTip()
{
	TSharedPtr<IToolTip> CurrentTooltip = SWidget::GetToolTip();
	if (!CurrentTooltip.IsValid())
	{
		TSharedPtr<SToolTip> ComplexTooltip = GetComplexTooltip();
		if (ComplexTooltip.IsValid())
		{
			SetToolTip(ComplexTooltip);
			bProvidedComplexTooltip = true;
		}
	}

	return SWidget::GetToolTip();
}

void SGraphNode::OnToolTipClosing()
{
	if (bProvidedComplexTooltip)
	{
		SetToolTip(nullptr);
		bProvidedComplexTooltip = false;
	}
}

void SGraphNode::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	CachedUnscaledPosition = AllottedGeometry.AbsolutePosition/AllottedGeometry.Scale;

	SNodePanel::SNode::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const bool bNeedToUpdateCommentBubble = GetNodeObj()->ShouldMakeCommentBubbleVisible();

	if (IsHovered() || bNeedToUpdateCommentBubble)
	{
		if (FNodeSlot* CommentSlot = GetSlot(ENodeZone::TopCenter))
		{
			TSharedPtr<SCommentBubble> CommentBubble = StaticCastSharedRef<SCommentBubble>(CommentSlot->GetWidget());
			if (CommentBubble.IsValid())
			{
				if (bNeedToUpdateCommentBubble)
				{
					CommentBubble->SetCommentBubbleVisibility(true);
					GetNodeObj()->SetMakeCommentBubbleVisible(false);
				}
				else
				{
					CommentBubble->TickVisibility(InCurrentTime, InDeltaTime);
				}
			}
		}
	}
}

bool SGraphNode::IsSelectedExclusively() const
{
	TSharedPtr<SGraphPanel> OwnerPanel = OwnerGraphPanelPtr.Pin();
	if (!OwnerPanel.IsValid())
	{
		return false;
	}

	if (!OwnerPanel->HasKeyboardFocus() || OwnerPanel->SelectionManager.GetSelectedNodes().Num() > 1)
	{
		return false;
	}

	return OwnerPanel->SelectionManager.IsNodeSelected(GraphNode);
}

/** @param OwnerPanel  The GraphPanel that this node belongs to */
void SGraphNode::SetOwner( const TSharedRef<SGraphPanel>& OwnerPanel )
{
	check( !OwnerGraphPanelPtr.IsValid() );
	SetParentPanel(OwnerPanel);
	OwnerGraphPanelPtr = OwnerPanel;
	GraphNode->DEPRECATED_NodeWidget = SharedThis(this);

	/*Once we have an owner, and if hide Unused pins is enabled, we need to remake our pins to drop the hidden ones*/
	if(OwnerGraphPanelPtr.Pin()->GetPinVisibility() != SGraphEditor::Pin_Show 
		&& LeftNodeBox.IsValid()
		&& RightNodeBox.IsValid())
	{
		if (this->LeftNodeBox.IsValid())
		{
			this->LeftNodeBox->ClearChildren();
		}
		if (this->RightNodeBox.IsValid())
		{
			this->RightNodeBox->ClearChildren();
		}
		CreatePinWidgets();
	}

	if (TitleLODBranchNode.IsValid())
	{
		TitleLODBranchNode->RefreshLODSlotContent();
	}
	
	for (TSharedRef<SGraphPin> Pin : InputPins)
	{
		Pin->RefreshLOD();
	}
	for (TSharedRef<SGraphPin> Pin : OutputPins)
	{
		Pin->RefreshLOD();
	}
}

/** @param NewPosition  The Node should be relocated to this position in the graph panel */
void SGraphNode::MoveTo( const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty )
{
	if ( !NodeFilter.Find( SharedThis( this )))
	{
		if (GraphNode && !RequiresSecondPassLayout())
		{
			NodeFilter.Add( SharedThis( this ) );
			GraphNode->Modify(bMarkDirty);
			GraphNode->SetPosition(NewPosition);
			Invalidate(EInvalidateWidgetReason::RenderTransform);
		}
	}
}

/** @return the Node's position within the graph */
FVector2f SGraphNode::GetPosition2f() const
{
	return GraphNode->GetPosition();
}

FVector2f SGraphNode::GetDesiredSizeForMarquee2f() const
{
	return ControlFlowSequence ? UE::Slate::CastToVector2f(ControlFlowSequence->GetChildren()->GetChildAt(0)->GetCachedGeometry().GetLocalSize())
		: SNodePanel::SNode::GetDesiredSizeForMarquee2f();
}

FString SGraphNode::GetEditableNodeTitle() const
{
	if (GraphNode != nullptr)
	{
		// Trying to catch a non-reproducible crash in this function
		check(GraphNode->IsValidLowLevel());
		return GraphNode->GetNodeTitle(ENodeTitleType::EditableTitle).ToString();
	}

	return NSLOCTEXT("GraphEditor", "NullNode", "Null Node").ToString();
}

FText SGraphNode::GetEditableNodeTitleAsText() const
{
	FString NewString = GetEditableNodeTitle();
	return FText::FromString(NewString);
}

FString SGraphNode::GetNodeComment() const
{
	return GetNodeObj()->NodeComment;
}

UObject* SGraphNode::GetObjectBeingDisplayed() const
{
	return GetNodeObj();
}

FSlateColor SGraphNode::GetNodeTitleColor() const
{
	FLinearColor ReturnTitleColor = GraphNode->IsDeprecated() ? FLinearColor::Red : GetNodeObj()->GetNodeTitleColor();

	if(!GraphNode->IsNodeEnabled() || GraphNode->IsDisplayAsDisabledForced() || GraphNode->IsNodeUnrelated())
	{
		ReturnTitleColor *= FLinearColor(0.5f, 0.5f, 0.5f, 0.4f);
	}
	else
	{
		ReturnTitleColor.A = FadeCurve.GetLerp();
	}
	return ReturnTitleColor;
}

FSlateColor SGraphNode::GetNodeBodyColor() const
{
	FLinearColor ReturnBodyColor = GraphNode->GetNodeBodyTintColor();
	if(!GraphNode->IsNodeEnabled() || GraphNode->IsDisplayAsDisabledForced() || GraphNode->IsNodeUnrelated())
	{
		ReturnBodyColor *= FLinearColor(1.0f, 1.0f, 1.0f, 0.5f); 
	}
	return ReturnBodyColor;
}

const FSlateBrush* SGraphNode::GetNodeBodyBrush() const
{
	static const FSlateBrush* Brush = FAppStyle::GetBrush("Graph.Node.Body");
	return Brush;
}

FSlateColor SGraphNode::GetNodeTitleIconColor() const
{
	FLinearColor ReturnIconColor = IconColor;
	if(!GraphNode->IsNodeEnabled() || GraphNode->IsDisplayAsDisabledForced() || GraphNode->IsNodeUnrelated())
	{
		ReturnIconColor *= FLinearColor(1.0f, 1.0f, 1.0f, 0.3f); 
	}
	return ReturnIconColor;
}

FLinearColor SGraphNode::GetNodeTitleTextColor() const
{
	return GetNodeObj()->GetNodeTitleTextColor();
}

FSlateColor SGraphNode::GetNodeCommentColor() const
{
	return GetNodeObj()->GetNodeCommentColor();
}

/** @return the tooltip to display when over the node */
FText SGraphNode::GetNodeTooltip() const
{
	if (GraphNode != NULL)
	{
		// If any of our child pins have an interactive tooltip, we shouldn't have a tooltip
		for (UEdGraphPin* Pin : GraphNode->GetAllPins())
		{
			TSharedPtr<SGraphPin> PinWidget = FindWidgetForPin(Pin);

			if (PinWidget.IsValid() && PinWidget->HasInteractiveTooltip())
			{
				return FText::GetEmpty();
			}
		}

		// Display the native title of the node when alt is held
		if(FSlateApplication::Get().GetModifierKeys().IsAltDown())
		{
			return FText::FromString(GraphNode->GetNodeTitle(ENodeTitleType::ListView).BuildSourceString());
		}

		FText TooltipText = GraphNode->GetTooltipText();

		if (UEdGraph* Graph = GraphNode->GetGraph())
		{
			// If the node resides in an intermediate graph, show the UObject name for debug purposes
			if (Graph->HasAnyFlags(RF_Transient))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("NodeName"), FText::FromString(GraphNode->GetName()));
				Args.Add(TEXT("TooltipText"), TooltipText);
				TooltipText = FText::Format(NSLOCTEXT("GraphEditor", "GraphNodeTooltip", "{NodeName}\n\n{TooltipText}"), Args);
			}
		}

		if (TooltipText.IsEmpty())
		{
			TooltipText =  GraphNode->GetNodeTitle(ENodeTitleType::FullTitle);
		}

		return TooltipText;
	}
	else
	{
		return NSLOCTEXT("GraphEditor", "InvalidGraphNode", "<Invalid graph node>");
	}
}


/** @return the node being observed by this widget*/
UEdGraphNode* SGraphNode::GetNodeObj() const
{
	return GraphNode;
}

TSharedRef<SGraphNode> SGraphNode::GetNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return StaticCastSharedRef<SGraphNode>(AsShared());
}

TSharedPtr<SGraphPanel> SGraphNode::GetOwnerPanel() const
{
	return OwnerGraphPanelPtr.Pin();
}

void SGraphNode::UpdateErrorInfo()
{
	//Check for node errors/warnings
	if (GraphNode->bHasCompilerMessage)
	{
		if (GraphNode->ErrorType <= EMessageSeverity::Error)
		{
			ErrorMsg = FString( TEXT("ERROR!") );
			ErrorColor = FAppStyle::GetColor("ErrorReporting.BackgroundColor");
		}
		else if (GraphNode->ErrorType <= EMessageSeverity::Warning)
		{
			ErrorMsg = FString( TEXT("WARNING!") );
			ErrorColor = FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor");
		}
		else
		{
			ErrorMsg = FString( TEXT("NOTE") );
			ErrorColor = FAppStyle::GetColor("InfoReporting.BackgroundColor");
		}
	}
	else if (!GraphNode->NodeUpgradeMessage.IsEmpty())
	{
		ErrorMsg = FString(TEXT("UPGRADE NOTE"));
		ErrorColor = FAppStyle::GetColor("InfoReporting.BackgroundColor");
	}
	else 
	{
		ErrorColor = FLinearColor(0,0,0);
		ErrorMsg.Empty();
	}

	VisualWarningMsg = FString(TEXT("WARNING!"));
	VisualWarningColor = FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor");
}

void SGraphNode::SetupErrorReporting()
{
	UpdateErrorInfo();

	if( !ErrorReporting.IsValid() )
	{
		TSharedPtr<SErrorText> ErrorTextWidget;

		// generate widget
		SAssignNew(ErrorTextWidget, SErrorText)
			.BackgroundColor( this, &SGraphNode::GetErrorColor )
			.ToolTipText( this, &SGraphNode::GetErrorMsgToolTip );

		ErrorReporting = ErrorTextWidget;
	}
	ErrorReporting->SetError(ErrorMsg);

	if (!VisualWarningReporting.IsValid())
	{
		TSharedPtr<SErrorText> ErrorTextWidget;

		// generate widget
		SAssignNew(ErrorTextWidget, SErrorText)
			.Visibility(this, &SGraphNode::VisualWarningVisibility)
			.BackgroundColor(this, &SGraphNode::GetVisualWarningColor)
			.ToolTipText(this, &SGraphNode::GetVisualWarningMsgToolTip);

		VisualWarningReporting = ErrorTextWidget;
	}
	VisualWarningReporting->SetError(VisualWarningMsg);
}

TSharedRef<SWidget> SGraphNode::CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle)
{
	SAssignNew(InlineEditableText, SInlineEditableTextBlock)
		.Style(FAppStyle::Get(), "Graph.Node.NodeTitleInlineEditableText")
		.Text(NodeTitle.Get(), &SNodeTitle::GetHeadTitle)
		.OnVerifyTextChanged(this, &SGraphNode::OnVerifyNameTextChanged)
		.OnTextCommitted(this, &SGraphNode::OnNameTextCommited)
		.IsReadOnly(this, &SGraphNode::IsNameReadOnly)
		.IsSelected(this, &SGraphNode::IsSelectedExclusively)
		.OverflowPolicy(GetNameOverflowPolicy());
	InlineEditableText->SetColorAndOpacity(TAttribute<FLinearColor>::Create(TAttribute<FLinearColor>::FGetter::CreateSP(this, &SGraphNode::GetNodeTitleTextColor)));

	return InlineEditableText.ToSharedRef();
}

TSharedRef<SWidget> SGraphNode::CreateTitleRightWidget()
{
	return SNullWidget::NullWidget;
}

/**
 * Update this GraphNode to match the data that it is observing
 */
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SGraphNode::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	//
	//             ______________________
	//            |      TITLE AREA      |
	//            +-------+------+-------+
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |      | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	TSharedPtr<SVerticalBox> MainVerticalBox;
	SetupErrorReporting();

	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	TSharedRef<SOverlay> DefaultTitleAreaWidget =
		SNew(SOverlay)
		+SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SImage)
			.Image( FAppStyle::GetBrush("Graph.Node.TitleGloss") )
			.ColorAndOpacity( this, &SGraphNode::GetNodeTitleIconColor )
		]
		+SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("Graph.Node.ColorSpill") )
			.Padding(TitleBorderMargin)
			.BorderBackgroundColor(this, &SGraphNode::GetNodeTitleColor)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Top)
					.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
					.AutoWidth()
					[
						CreateIconWidget()
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							CreateTitleWidget(NodeTitle)
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							NodeTitle.ToSharedRef()
						]
					]
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(0, 0, 5, 0)
				.AutoWidth()
				[
					CreateTitleRightWidget()
				]
			]
		]
		+SOverlay::Slot()
		.VAlign(VAlign_Top)
		[
			SNew(SBorder)
			.Visibility(EVisibility::HitTestInvisible)			
			.BorderImage( FAppStyle::GetBrush( "Graph.Node.TitleHighlight" ) )
			.BorderBackgroundColor( this, &SGraphNode::GetNodeTitleIconColor )
			[
				SNew(SSpacer)
				.Size(FVector2D(20,20))
			]
		];

	SetDefaultTitleAreaWidget(DefaultTitleAreaWidget);

	SAssignNew(TitleLODBranchNode, SLevelOfDetailBranchNode)
	.UseLowDetailSlot(this, &SGraphNode::UseLowDetailNodeTitles)
	.LowDetail()
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Graph.Node.ColorSpill") )
		.Padding( FMargin(75.0f, 22.0f) ) // Saving enough space for a 'typical' title so the transition isn't quite so abrupt
		.BorderBackgroundColor( this, &SGraphNode::GetNodeTitleColor )
	]
	.HighDetail()
	[
		DefaultTitleAreaWidget
	];

	
	if (!SWidget::GetToolTip().IsValid())
	{
		TSharedRef<SToolTip> DefaultToolTip = IDocumentation::Get()->CreateToolTip( TAttribute< FText >( this, &SGraphNode::GetNodeTooltip ), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName() );
		SetToolTip(DefaultToolTip);
	}

	// Setup a meta tag for this node
	FGraphNodeMetaData TagMeta(TEXT("Graphnode"));
	PopulateMetaTag(&TagMeta);
	
	TSharedPtr<SVerticalBox> InnerVerticalBox = SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.Padding(Settings->GetNonPinNodeBodyPadding())
		[
			TitleLODBranchNode.ToSharedRef()
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			CreateNodeContentArea()
		];

	TSharedPtr<SWidget> EnabledStateWidget = GetEnabledStateWidget();
	if (EnabledStateWidget.IsValid())
	{
		InnerVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.Padding(FMargin(2, 0))
			[
				EnabledStateWidget.ToSharedRef()
			];
	}

	InnerVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(Settings->GetNonPinNodeBodyPadding())
		[
			ErrorReporting->AsWidget()
		];

	InnerVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(Settings->GetNonPinNodeBodyPadding())
		[
			VisualWarningReporting->AsWidget()
		];


	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SAssignNew(MainVerticalBox, SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SOverlay)
				.AddMetaData<FGraphNodeMetaData>(TagMeta)
				+SOverlay::Slot()
				.Padding(Settings->GetNonPinNodeBodyPadding())
				[
					SNew(SImage)
					.Image(this, &SGraphNode::GetNodeBodyBrush)
					.ColorAndOpacity(this, &SGraphNode::GetNodeBodyColor)
				]
				+SOverlay::Slot()
				[
					InnerVerticalBox.ToSharedRef()
				]
			]			
		];

	bool SupportsBubble = true;
	if (GraphNode != nullptr)
	{
		SupportsBubble = GraphNode->SupportsCommentBubble();
	}

	if (SupportsBubble)
	{
		// Create comment bubble
		TSharedPtr<SCommentBubble> CommentBubble;
		const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

		SAssignNew(CommentBubble, SCommentBubble)
			.GraphNode(GraphNode)
			.Text(this, &SGraphNode::GetNodeComment)
			.OnTextCommitted(this, &SGraphNode::OnCommentTextCommitted)
			.OnToggled(this, &SGraphNode::OnCommentBubbleToggled)
			.ColorAndOpacity(CommentColor)
			.AllowPinning(true)
			.EnableTitleBarBubble(true)
			.EnableBubbleCtrls(true)
			.GraphLOD(this, &SGraphNode::GetCurrentLOD)
			.IsGraphNodeHovered(this, &SGraphNode::IsHovered);

		GetOrAddSlot(ENodeZone::TopCenter)
			.SlotOffset2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetOffset2f))
			.SlotSize2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetSize2f))
			.AllowScaling(TAttribute<bool>(CommentBubble.Get(), &SCommentBubble::IsScalingAllowed))
			.VAlign(VAlign_Top)
			[
				CommentBubble.ToSharedRef()
			];
	}

	CreateBelowWidgetControls(MainVerticalBox);
	CreatePinWidgets();
	CreateInputSideAddButton(LeftNodeBox);
	CreateOutputSideAddButton(RightNodeBox);
	CreateBelowPinControls(InnerVerticalBox);
	CreateAdvancedViewArrow(InnerVerticalBox);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


TSharedPtr<SWidget> SGraphNode::GetEnabledStateWidget() const
{
	if ((GraphNode->GetDesiredEnabledState() != ENodeEnabledState::Enabled) && !GraphNode->IsAutomaticallyPlacedGhostNode())
	{
		const bool bDevelopmentOnly = GraphNode->GetDesiredEnabledState() == ENodeEnabledState::DevelopmentOnly;
		const FText StatusMessage = bDevelopmentOnly ? NSLOCTEXT("SGraphNode", "DevelopmentOnly", "Development Only") : NSLOCTEXT("SGraphNode", "DisabledNode", "Disabled");
		const FText StatusMessageTooltip = bDevelopmentOnly ?
			NSLOCTEXT("SGraphNode", "DevelopmentOnlyTooltip", "This node will only be executed in the editor and in Development builds in a packaged game (it will be treated as disabled in Shipping or Test builds cooked from a commandlet)") :
			NSLOCTEXT("SGraphNode", "DisabledNodeTooltip", "This node is currently disabled and will not be executed");

		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(bDevelopmentOnly ? "Graph.Node.DevelopmentBanner" : "Graph.Node.DisabledBanner"))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(STextBlock)
				.Text(StatusMessage)
				.ToolTipText(StatusMessageTooltip)
				.Justification(ETextJustify::Center)
				.ColorAndOpacity(FLinearColor::White)
				.ShadowOffset(FVector2D::UnitVector)
				.Visibility(EVisibility::Visible)
			];
	}

	return TSharedPtr<SWidget>();
}

TSharedRef<SWidget> SGraphNode::CreateNodeContentArea()
{
	// NODE CONTENT AREA
	return SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding( FMargin(0,3) )
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(1.0f)
			[
				// LEFT
				SAssignNew(LeftNodeBox, SVerticalBox)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				// RIGHT
				SAssignNew(RightNodeBox, SVerticalBox)
			]
		];
}

/** Returns visibility of AdvancedViewButton */
EVisibility SGraphNode::AdvancedViewArrowVisibility() const
{
	const bool bShowAdvancedViewArrow = GraphNode && (ENodeAdvancedPins::NoPins != GraphNode->AdvancedPinDisplay);
	return bShowAdvancedViewArrow ? EVisibility::Visible : EVisibility::Collapsed;
}

void SGraphNode::OnAdvancedViewChanged( const ECheckBoxState NewCheckedState )
{
	if(GraphNode && (ENodeAdvancedPins::NoPins != GraphNode->AdvancedPinDisplay))
	{
		const bool bAdvancedPinsHidden = (NewCheckedState != ECheckBoxState::Checked);
		GraphNode->AdvancedPinDisplay = bAdvancedPinsHidden ? ENodeAdvancedPins::Hidden : ENodeAdvancedPins::Shown;
	}
}

ECheckBoxState SGraphNode::IsAdvancedViewChecked() const
{
	const bool bAdvancedPinsHidden = GraphNode && (ENodeAdvancedPins::Hidden == GraphNode->AdvancedPinDisplay);
	return bAdvancedPinsHidden ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

const FSlateBrush* SGraphNode::GetAdvancedViewArrow() const
{
	const bool bAdvancedPinsHidden = GraphNode && (ENodeAdvancedPins::Hidden == GraphNode->AdvancedPinDisplay);
	return FAppStyle::GetBrush(bAdvancedPinsHidden ? TEXT("Icons.ChevronDown") : TEXT("Icons.ChevronUp"));
}

/** Create widget to show/hide advanced pins */
void SGraphNode::CreateAdvancedViewArrow(TSharedPtr<SVerticalBox> MainBox)
{
	const bool bHidePins = OwnerGraphPanelPtr.IsValid() && (OwnerGraphPanelPtr.Pin()->GetPinVisibility() != SGraphEditor::Pin_Show);
	const bool bAnyAdvancedPin = GraphNode && (ENodeAdvancedPins::NoPins != GraphNode->AdvancedPinDisplay);
	if(!bHidePins && GraphNode && MainBox.IsValid())
	{
		MainBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.Padding(3, 0, 3, 3)
		[
			SNew(SCheckBox)
			.Visibility(this, &SGraphNode::AdvancedViewArrowVisibility)
			.OnCheckStateChanged( this, &SGraphNode::OnAdvancedViewChanged )
			.IsChecked( this, &SGraphNode::IsAdvancedViewChecked )
			.Cursor(EMouseCursor::Default)
			.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					. Image(this, &SGraphNode::GetAdvancedViewArrow)
				]
			]
		];
	}
}

bool SGraphNode::ShouldPinBeHidden(const UEdGraphPin* InPin) const
{
	bool bHideNoConnectionPins = false;
	bool bHideNoConnectionNoDefaultPins = false;

	// Not allowed to hide exec pins 
	const bool bCanHidePin = (InPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec);

	if (OwnerGraphPanelPtr.IsValid() && bCanHidePin)
	{
		bHideNoConnectionPins = OwnerGraphPanelPtr.Pin()->GetPinVisibility() == SGraphEditor::Pin_HideNoConnection;
		bHideNoConnectionNoDefaultPins = OwnerGraphPanelPtr.Pin()->GetPinVisibility() == SGraphEditor::Pin_HideNoConnectionNoDefault;
	}

	const bool bIsOutputPin = InPin->Direction == EGPD_Output;
	const bool bPinHasDefaultValue = !InPin->DefaultValue.IsEmpty() || (InPin->DefaultObject != NULL);
	const bool bIsSelfTarget = (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) && (InPin->PinName == UEdGraphSchema_K2::PN_Self);
	const bool bPinHasValidDefault = !bIsOutputPin && (bPinHasDefaultValue || bIsSelfTarget);
	const bool bPinHasConections = InPin->LinkedTo.Num() > 0;

	const bool bPinDesiresToBeHidden = InPin->bHidden || (bHideNoConnectionPins && !bPinHasConections) || (bHideNoConnectionNoDefaultPins && !bPinHasConections && !bPinHasValidDefault);

	// No matter how strong the desire, a pin with connections can never be hidden!
	const bool bShowPin = !bPinDesiresToBeHidden || bPinHasConections;

	return bShowPin;
}

void SGraphNode::CreateStandardPinWidget(UEdGraphPin* CurPin)
{
	const bool bShowPin = ShouldPinBeHidden(CurPin);

	if (bShowPin)
	{
		TSharedPtr<SGraphPin> NewPin = CreatePinWidget(CurPin);
		check(NewPin.IsValid());

		this->AddPin(NewPin.ToSharedRef());
	}
}

void SGraphNode::CreatePinWidgets()
{
	// Create Pin widgets for each of the pins.
	for (int32 PinIndex = 0; PinIndex < GraphNode->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* CurPin = GraphNode->Pins[PinIndex];

		if ( !ensureMsgf(CurPin->GetOuter() == GraphNode
			, TEXT("Graph node ('%s' - %s) has an invalid %s pin: '%s'; (with a bad %s outer: '%s'); skiping creation of a widget for this pin.")
			, *GraphNode->GetNodeTitle(ENodeTitleType::ListView).ToString()
			, *GraphNode->GetPathName()
			, (CurPin->Direction == EEdGraphPinDirection::EGPD_Input) ? TEXT("input") : TEXT("output")
			,  CurPin->PinFriendlyName.IsEmpty() ? *CurPin->PinName.ToString() : *CurPin->PinFriendlyName.ToString()
			,  CurPin->GetOuter() ? *CurPin->GetOuter()->GetClass()->GetName() : TEXT("UNKNOWN")
			,  CurPin->GetOuter() ? *CurPin->GetOuter()->GetPathName() : TEXT("NULL")) )
		{
			continue;
		}

		CreateStandardPinWidget(CurPin);
	}
}

TSharedPtr<SGraphPin> SGraphNode::CreatePinWidget(UEdGraphPin* Pin) const
{
	return FNodeFactory::CreatePinWidget(Pin);
}

void SGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{	
	PinToAdd->SetOwner(SharedThis(this));

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = (PinObj != nullptr) && PinObj->bAdvancedView;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility( TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced) );
	}

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(Settings->GetInputPinPadding())
		[
			PinToAdd
		];
		InputPins.Add(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		RightNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(Settings->GetOutputPinPadding())
		[
			PinToAdd
		];
		OutputPins.Add(PinToAdd);
	}
}

/**
 * Get all the pins found on this node.
 *
 * @param AllPins  The set of pins found on this node.
 */
void SGraphNode::GetPins( TSet< TSharedRef<SWidget> >& AllPins ) const
{

	for( int32 PinIndex=0; PinIndex < this->InputPins.Num(); ++PinIndex )
	{
		AllPins.Add(InputPins[PinIndex]);
	}


	for( int32 PinIndex=0; PinIndex < this->OutputPins.Num(); ++PinIndex )
	{
		AllPins.Add(OutputPins[PinIndex]);
	}

}

void SGraphNode::GetPins( TArray< TSharedRef<SWidget> >& AllPins ) const
{

	for( int32 PinIndex=0; PinIndex < this->InputPins.Num(); ++PinIndex )
	{
		AllPins.Add(InputPins[PinIndex]);
	}


	for( int32 PinIndex=0; PinIndex < this->OutputPins.Num(); ++PinIndex )
	{
		AllPins.Add(OutputPins[PinIndex]);
	}

}

TSharedPtr<SGraphPin> SGraphNode::GetHoveredPin( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) const
{
	// We just need to find the one WidgetToFind among our descendants.
	TSet< TSharedRef<SWidget> > MyPins;
	{
		GetPins( MyPins );
	}
	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;

	FindChildGeometries(MyGeometry, MyPins, Result);
	
	if ( Result.Num() > 0 )
	{
		FArrangedChildren ArrangedPins(EVisibility::Visible);
		Result.GenerateValueArray( ArrangedPins.GetInternalArray() );
		int32 HoveredPinIndex = SWidget::FindChildUnderMouse( ArrangedPins, MouseEvent );
		if ( HoveredPinIndex != INDEX_NONE )
		{
			return StaticCastSharedRef<SGraphPin>(ArrangedPins[HoveredPinIndex].Widget);
		}
	}
	
	return TSharedPtr<SGraphPin>();
}

TSharedPtr<SGraphPin> SGraphNode::FindWidgetForPin( UEdGraphPin* ThePin ) const
{
	// Search input or output pins?
	const TArray< TSharedRef<SGraphPin> > &PinsToSearch = (ThePin->Direction == EGPD_Input) ? InputPins : OutputPins;

	// Actually search for the widget
	for( int32 PinIndex=0; PinIndex < PinsToSearch.Num(); ++PinIndex )
	{
		if ( PinsToSearch[PinIndex]->GetPinObj() == ThePin )
		{
			return PinsToSearch[PinIndex];
		}
	}

	return TSharedPtr<SGraphPin>(NULL);
}

void SGraphNode::PlaySpawnEffect()
{
	SpawnAnim.Play( this->AsShared() );
}

UE::Slate::FDeprecateVector2DResult SGraphNode::GetContentScale() const
{
	const float CurZoomValue = ZoomCurve.GetLerp();
	return FVector2f( CurZoomValue, CurZoomValue );
}

FLinearColor SGraphNode::GetColorAndOpacity() const
{
	return FLinearColor(1,1,1,FadeCurve.GetLerp());
}

FLinearColor SGraphNode::GetPinLabelColorAndOpacity() const
{
	return FLinearColor(0,0,0,FadeCurve.GetLerp());
}


SGraphNode::SGraphNode()
	: IsEditable(true)
	, bProvidedComplexTooltip(false)
	, bRenameIsPending( false )
	, ErrorColor( FLinearColor::White )
	, CachedUnscaledPosition( FVector2f::ZeroVector )
	, Settings( GetDefault<UGraphEditorSettings>() )
{
	// Set up animation
	{
		ZoomCurve = SpawnAnim.AddCurve(0, 0.1f);
		FadeCurve = SpawnAnim.AddCurve(0.15f, 0.15f);
		SpawnAnim.JumpToEnd();
	}
}

void SGraphNode::PositionThisNodeBetweenOtherNodes(const TMap< UObject*, TSharedRef<SNode> >& NodeToWidgetLookup, UEdGraphNode* PreviousNode, UEdGraphNode* NextNode, float HeightAboveWire) const
{
	if ((PreviousNode != NULL) && (NextNode != NULL))
	{
		TSet<UEdGraphNode*> PrevNodes;
		PrevNodes.Add(PreviousNode);

		TSet<UEdGraphNode*> NextNodes;
		NextNodes.Add(NextNode);

		PositionThisNodeBetweenOtherNodes(NodeToWidgetLookup, PrevNodes, NextNodes, HeightAboveWire);
	}
}

void SGraphNode::PositionThisNodeBetweenOtherNodes(const TMap< UObject*, TSharedRef<SNode> >& NodeToWidgetLookup, TSet<UEdGraphNode*>& PreviousNodes, TSet<UEdGraphNode*>& NextNodes, float HeightAboveWire) const
{
	// Find the previous position centroid
	FVector2f PrevPos(0.0f, 0.0f);
	for (auto NodeIt = PreviousNodes.CreateConstIterator(); NodeIt; ++NodeIt)
	{
		UEdGraphNode* PreviousNode = *NodeIt;
		const FVector2f CornerPos = PreviousNode->GetPosition();
		PrevPos += CornerPos + NodeToWidgetLookup.FindChecked(PreviousNode)->GetDesiredSize() * 0.5f;
	}

	// Find the next position centroid
	FVector2f NextPos(0.0f, 0.0f);
	for (auto NodeIt = NextNodes.CreateConstIterator(); NodeIt; ++NodeIt)
	{
		UEdGraphNode* NextNode = *NodeIt;
		const FVector2f CornerPos = NextNode->GetPosition();
		NextPos += CornerPos + NodeToWidgetLookup.FindChecked(NextNode)->GetDesiredSize() * 0.5f;
	}

	PositionThisNodeBetweenOtherNodes(PrevPos, NextPos, HeightAboveWire);
}

void SGraphNode::PositionThisNodeBetweenOtherNodes(const FVector2f& PrevPos, const FVector2f& NextPos, float HeightAboveWire) const
{
	const FVector2f DesiredNodeSize = GetDesiredSize();

	FVector2f DeltaPos(NextPos - PrevPos);
	if (DeltaPos.IsNearlyZero())
	{
		DeltaPos = FVector2f(10.0f, 0.0f);
	}

	const FVector2f Normal = FVector2f(DeltaPos.Y, -DeltaPos.X).GetSafeNormal();

	const FVector2f SlidingCapsuleBias = FVector2f::ZeroVector;//(0.5f * FMath::Sin(Normal.X * (float)HALF_PI) * DesiredNodeSize.X, 0.0f);

	const FVector2f NewCenter = PrevPos + (0.5f * DeltaPos) + (HeightAboveWire * Normal) + SlidingCapsuleBias;

	// Now we need to adjust the new center by the node size and zoom factor
	const FVector2f NewCorner = NewCenter - (0.5f * DesiredNodeSize);

	GraphNode->SetPosition(NewCorner);
}

FText SGraphNode::GetErrorMsgToolTip() const
{
	FText Result = FText::GetEmpty();
	if (GraphNode != nullptr)
	{
		Result = FText::FromString(GraphNode->ErrorMsg);

		// Append the node's upgrade message, if any.
		if (!GraphNode->NodeUpgradeMessage.IsEmpty())
		{
			if (Result.IsEmpty())
			{
				Result = GraphNode->NodeUpgradeMessage;
			}
			else
			{
				Result = FText::Format(FText::FromString(TEXT("{0}\n\n{1}")), Result, GraphNode->NodeUpgradeMessage);
			}
		}
	}

	return Result;
}

EVisibility SGraphNode::VisualWarningVisibility() const
{
	const bool bShowVisualWarning = GraphNode && GraphNode->ShowVisualWarning();
	return bShowVisualWarning ? EVisibility::Visible : EVisibility::Collapsed;
}

/* Helper function to set the error color for the node */
FSlateColor SGraphNode::GetVisualWarningColor() const
{
	return VisualWarningColor;
}

FText SGraphNode::GetVisualWarningMsgToolTip() const
{
	FText Result = FText::GetEmpty();
	if (GraphNode != nullptr)
	{
		Result = GraphNode->GetVisualWarningTooltipText();
	}

	return Result;
}


bool SGraphNode::IsNameReadOnly() const
{
	return (!GraphNode->GetCanRenameNode() || !IsNodeEditable());
}

bool SGraphNode::OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage)
{
	bool bValid(true);

	if ((GetEditableNodeTitle() != InText.ToString()) && OnVerifyTextCommit.IsBound())
	{
		bValid = OnVerifyTextCommit.Execute(InText, GraphNode, OutErrorMessage);
	}

	if( OutErrorMessage.IsEmpty() )
	{
		OutErrorMessage = FText::FromString(TEXT("Error"));
	}

	//UpdateErrorInfo();
	//ErrorReporting->SetError(ErrorMsg);

	return bValid;
}

void SGraphNode::OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo)
{
	OnTextCommitted.ExecuteIfBound(InText, CommitInfo, GraphNode);
	
	UpdateErrorInfo();
	if (ErrorReporting.IsValid())
	{
		ErrorReporting->SetError(ErrorMsg);
	}
}

void SGraphNode::RequestRename()
{
	if ((GraphNode != nullptr) && GraphNode->GetCanRenameNode())
	{
		bRenameIsPending = true;
	}
}

void SGraphNode::ApplyRename()
{
	if (bRenameIsPending)
	{
		bRenameIsPending = false;
		InlineEditableText->EnterEditingMode();
	}
}

void SGraphNode::CancelRename()
{
	if (InlineEditableText.IsValid())
	{
		if(InlineEditableText->IsInEditMode())
		{
			InlineEditableText->ExitEditingMode();
		}
	}
}

FSlateRect SGraphNode::GetTitleRect() const
{
	const FVector2f NodePosition = GetPosition2f();
	const FVector2f NodeSize = GraphNode ? InlineEditableText->GetDesiredSize() : GetDesiredSize();

	return FSlateRect( NodePosition.X, NodePosition.Y + NodeSize.Y, NodePosition.X + NodeSize.X, NodePosition.Y );
}

void SGraphNode::NotifyDisallowedPinConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	OnDisallowedPinConnection.ExecuteIfBound(PinA, PinB);
}

bool SGraphNode::UseLowDetailNodeTitles() const
{
	if (InlineEditableText.IsValid())
	{
		if (const SGraphPanel* MyOwnerPanel = GetOwnerPanel().Get())
		{
			return (MyOwnerPanel->GetCurrentLOD() <= EGraphRenderingLOD::LowestDetail) && !InlineEditableText->IsInEditMode();
		}
	}

	return false;
}

TSharedRef<SWidget> SGraphNode::AddPinButtonContent(FText PinText, FText PinTooltipText, bool bRightSide, FString DocumentationExcerpt, TSharedPtr<SToolTip> CustomTooltip)
{
	TSharedPtr<SWidget> ButtonContent;
	if(bRightSide)
	{
		SAssignNew(ButtonContent, SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(PinText)
			.ColorAndOpacity(FLinearColor::White)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		. VAlign(VAlign_Center)
		. Padding( 7,0,0,0 )
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
		];
	}
	else
	{
		SAssignNew(ButtonContent, SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		. VAlign(VAlign_Center)
		. Padding( 0,0,7,0 )
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(PinText)
			.ColorAndOpacity(FLinearColor::White)
		];
	}

	TSharedPtr<SToolTip> Tooltip;

	if (CustomTooltip.IsValid())
	{
		Tooltip = CustomTooltip;
	}
	else if (!DocumentationExcerpt.IsEmpty())
	{
		Tooltip = IDocumentation::Get()->CreateToolTip( PinTooltipText, NULL, GraphNode->GetDocumentationLink(), DocumentationExcerpt );
	}

	TSharedRef<SButton> AddPinButton = SNew(SButton)
	.ContentPadding(0.0f)
	.ButtonStyle( FAppStyle::Get(), "NoBorder" )
	.OnClicked( this, &SGraphNode::OnAddPin )
	.IsEnabled( this, &SGraphNode::IsNodeEditable )
	.ToolTipText(PinTooltipText)
	.ToolTip(Tooltip)
	.Visibility(this, &SGraphNode::IsAddPinButtonVisible)
	[
		ButtonContent.ToSharedRef()
	];

	AddPinButton->SetCursor( EMouseCursor::Hand );

	return AddPinButton;
}

EVisibility SGraphNode::IsAddPinButtonVisible() const
{
	bool bIsHidden = false;
	auto OwnerGraphPanel = OwnerGraphPanelPtr.Pin();
	if(OwnerGraphPanel.IsValid())
	{
		bIsHidden |= (SGraphEditor::EPinVisibility::Pin_Show != OwnerGraphPanel->GetPinVisibility());
		bIsHidden |= (OwnerGraphPanel->GetCurrentLOD() <= EGraphRenderingLOD::LowDetail);
	}

	return bIsHidden ? EVisibility::Collapsed : EVisibility::Visible;
}

void SGraphNode::PopulateMetaTag(FGraphNodeMetaData* TagMeta) const
{
	if (GraphNode && TagMeta)
	{
		// We want the name of the blueprint/world as our name - we can find the node from the GUID
		UObject* OutermostObject = GraphNode->GetOutermostObject();
		check(OutermostObject);
		TagMeta->Tag = FName(*FString::Printf(TEXT("GraphNode_%s_%s"), *OutermostObject->GetFullName(), *GraphNode->NodeGuid.ToString()));
		TagMeta->OuterName = OutermostObject->GetFullName();
		TagMeta->GUID = GraphNode->NodeGuid;
		TagMeta->FriendlyName = FString::Printf(TEXT("%s in %s"), *GraphNode->GetNodeTitle(ENodeTitleType::ListView).ToString(), *TagMeta->OuterName);		
	}
}

EGraphRenderingLOD::Type SGraphNode::GetCurrentLOD() const
{
	return OwnerGraphPanelPtr.IsValid() ? OwnerGraphPanelPtr.Pin()->GetCurrentLOD() : EGraphRenderingLOD::DefaultDetail;
}

void SGraphNode::RefreshErrorInfo()
{
	SetupErrorReporting();
}

void SGraphNode::InvalidateGraphData()
{
	TSharedPtr<SGraphPanel> OwnerGraphPanel = OwnerGraphPanelPtr.Pin();
	auto InvalidateGraphPinDataLambda = [&OwnerGraphPanel](TSharedRef<SGraphPin>& Pin)
	{
		Pin->InvalidateGraphData();

		// Ensure that the graph panel's current marked pin reference is cleared if it equates to one of the pins being invalidated.
		if (OwnerGraphPanel.IsValid() && OwnerGraphPanel->MarkedPin.IsValid() && OwnerGraphPanel->MarkedPin.HasSameObject(&Pin.Get()))
		{
			OwnerGraphPanel->MarkedPin.Reset();
		}
	};

	for (TSharedRef<SGraphPin>& Pin : InputPins)
	{
		InvalidateGraphPinDataLambda(Pin);
	}
	for (TSharedRef<SGraphPin>& Pin : OutputPins)
	{
		InvalidateGraphPinDataLambda(Pin);
	}
}

TSharedRef<SWidget> SGraphNode::CreateIconWidget()
{
	// Get node icon
	IconColor = FLinearColor::White;
	const FSlateBrush* IconBrush = nullptr;
	if (GraphNode != nullptr && GraphNode->ShowPaletteIconOnNode())
	{
		IconBrush = GraphNode->GetIconAndTint(IconColor).GetOptionalIcon();
	}
	
	return SNew(SImage)
		.Image(IconBrush)
		.ColorAndOpacity(this, &SGraphNode::GetNodeTitleIconColor);
}

TArray<FArrangedWidget> SGraphNode::GetArrangedSnappedChildNodes(const FGeometry& Geometry)
{
	TArray<FArrangedWidget> FinalResult;
	TSet< TSharedRef<SWidget> > MyPins;
	{
		GetPins( MyPins );
	}
	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
	FindChildGeometries(Geometry, MyPins, Result);

	// walk each pin looking for a snapped child node, if found add it to final result:
	for(const TPair<TSharedRef<SWidget>, FArrangedWidget>& ArrangedPin : Result)
	{
		TSharedRef<SGraphPin> AsPin = StaticCastSharedRef<SGraphPin>(ArrangedPin.Key);
		TOptional<FArrangedWidget> ChildNode = AsPin->GetArrangedSnappedChildNode(ArrangedPin.Value.Geometry);
		if(ChildNode.IsSet())
		{
			FinalResult.Emplace(MoveTemp(ChildNode.GetValue()));
		}
	}

	return FinalResult;
}

TSharedPtr<SGraphNode> SGraphNode::GetSnappedChildForPosition(const FGeometry& Geometry, const FVector2f& Position, FVector2f& OutPositionInNode )
{
	// check for control flow sequence child:
	if(ControlFlowSequence)
	{
		FChildren* ControlFlowChildren = ControlFlowSequence->GetChildren();
		if(ControlFlowChildren->Num() > 1)
		{
			TSet< TSharedRef<SWidget> > SubNodes;
			for(int32 I = 1; I < ControlFlowChildren->Num(); ++I)
			{
				SubNodes.Add( ControlFlowChildren->GetChildAt(I) );
			}
			TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
			FindChildGeometries(Geometry, SubNodes, Result);
		
			for( const TPair<TSharedRef<SWidget>, FArrangedWidget>& SubNode : Result)
			{
				if(SubNode.Value.Geometry.IsUnderLocation( Position ))
				{
					return StaticCastSharedRef<SGraphNode>(SubNode.Key)->GetSnappedChildForPosition(SubNode.Value.Geometry, Position, OutPositionInNode);
				}
			}
		}
	}

	TSet< TSharedRef<SWidget> > MyPins;
	{
		GetPins( MyPins );
	}
	
	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
	FindChildGeometries(Geometry, MyPins, Result);
	
	if ( Result.Num() > 0 )
	{
		FArrangedChildren ArrangedPins(EVisibility::Visible);
		Result.GenerateValueArray( ArrangedPins.GetInternalArray() );
		int32 HoveredPinIndex = SWidget::FindChildUnderPosition( ArrangedPins, Position );
		if ( HoveredPinIndex != INDEX_NONE )
		{
			TSharedRef<SGraphPin> HoveredPin = StaticCastSharedRef<SGraphPin>(
				ArrangedPins[HoveredPinIndex].Widget);
			// if the hovered pin has a node then return that:
			TSharedPtr<SGraphNode> NodeSnappedToPin = HoveredPin->GetSnappedChildForPosition(ArrangedPins[HoveredPinIndex].Geometry, Position, OutPositionInNode);
			if(NodeSnappedToPin)
			{
				return NodeSnappedToPin;
			}
		}
	}
	OutPositionInNode = Geometry.AbsoluteToLocal(Position);
	return StaticCastSharedRef<SGraphNode>(AsShared());
}

FGeometry SGraphNode::GetNodeGeometry(const FGeometry& Geometry) const
{
	// if we have a snapped execution child we don't want to include it in our displayed geometry (e.g. dropshadow, selection outline):
	if(ControlFlowSequence )
	{
		TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
		FindChildGeometries(Geometry, {ControlFlowSequence->GetChildren()->GetChildAt(0)}, Result); // this returns the geometry of the node itself, exc. its control flow children
		
		if ( Result.Num() > 0 )
		{
			TMap<TSharedRef<SWidget>, FArrangedWidget>::TIterator Begin(Result);
			return Begin->Value.Geometry;
		}
	}
	return Geometry;
}

void SGraphNode::DrawChildrenNodeShadow( const FGraphPanelSelectionSet& SelectionSet, FSlateWindowElementList& OutDrawElements, int32 NodeShadowsLayerId, const FGeometry& Geometry, FLinearColor Color)
{
	const FVector2f NodeShadowSize = UE::Slate::CastToVector2f(GetDefault<UGraphEditorSettings>()->GetShadowDeltaSize());
	const FSlateBrush* ShadowBrush = GetShadowBrush(SelectionSet.Contains(GraphNode));
	if (ShadowBrush == nullptr)
	{
		return;
	}
	FGeometry ChildGeometry = GetNodeGeometry(Geometry);
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		NodeShadowsLayerId,
		ChildGeometry.ToInflatedPaintGeometry(NodeShadowSize),
		ShadowBrush,
		ESlateDrawEffect::None,
		Color
	);

	TArray<FArrangedWidget> AllDataPinChildren = GetArrangedSnappedChildNodes(Geometry);
	for(const FArrangedWidget& DataPinNode : AllDataPinChildren)
	{
		// bugbugbug: this draws the node shadow behind our parent node, probably need to move it into onpaint:
		StaticCastSharedRef<SGraphNode>(DataPinNode.Widget)->DrawChildrenNodeShadow(SelectionSet, OutDrawElements, NodeShadowsLayerId, DataPinNode.Geometry, Color);
	}

	if(!ControlFlowSequence)
	{
		return;
	}

	FChildren* ControlFlowChildren = ControlFlowSequence->GetChildren();
	if(ControlFlowChildren->Num() <= 1)
	{
		return;
	}

	TSet< TSharedRef<SWidget> > SubNodes;
	for(int32 I = 1; I < ControlFlowChildren->Num(); ++I)
	{
		SubNodes.Add( ControlFlowChildren->GetChildAt(I) );
	}
	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
	FindChildGeometries(Geometry, SubNodes, Result);
		
	if ( Result.Num() > 0 )
	{
		for( const TPair<TSharedRef<SWidget>, FArrangedWidget>& SubNode : Result)
		{
			TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(SubNode.Key);
			ChildNode->DrawChildrenNodeShadow(SelectionSet, OutDrawElements, NodeShadowsLayerId, SubNode.Value.Geometry, Color);
		}
	}
}

void SGraphNode::GetPinsIncludingSnappedChildren( TSet< TSharedRef<SWidget> >& AllPins ) const
{
	ForEachPinOnSnappedChild([&AllPins](const TSharedRef<SGraphPin>& GraphPin) {
		AllPins.Add(GraphPin);
	});
}

void SGraphNode::GetPinsToDraw( TSet< TSharedRef<SWidget> >& AllPins ) const
{
	ForEachPinOnSnappedChild([&AllPins](const TSharedRef<SGraphPin>& GraphPin) {
		if(!GraphPin->IsSnapped())
		{
			AllPins.Add(GraphPin);
		}
	});
}

void SGraphNode::ForEachNodeIncludingChildren(TFunctionRef<void(const TSharedRef<SGraphNode>&)> Ftor)
{
	Ftor(StaticCastSharedRef<SGraphNode>(AsShared()));
	ForEachNodeSnappedToControlFlowPin(	Ftor );
	ForEachNodeSnappedToDataPin(Ftor);
}

void SGraphNode::ForEachPinOnSnappedChild(TFunctionRef<void(const TSharedRef<SGraphPin>&)> Ftor) const
{
	for( TSharedRef<SGraphPin> InputPin : InputPins )
	{
		Ftor(InputPin);
		TSharedPtr<SGraphNode> SnappedNode = InputPin->GetSnappedNode();
		if(SnappedNode)
		{
			SnappedNode->ForEachPinOnSnappedChild(Ftor);
		}
	}
	
	for( TSharedRef<SGraphPin> OutputPin : OutputPins )
	{
		Ftor(OutputPin);
		TSharedPtr<SGraphNode> SnappedNode = OutputPin->GetSnappedNode();
		if(SnappedNode)
		{
			SnappedNode->ForEachPinOnSnappedChild(Ftor);
		}
	}
	
	// recurse into attached control flow pins:
	ForEachNodeSnappedToControlFlowPin([Ftor](const TSharedRef<SGraphNode>& AttachedNode) {
		AttachedNode->ForEachPinOnSnappedChild(Ftor);
	});
}

void SGraphNode::ForEachNodeSnappedToControlFlowPin(TFunctionRef<void(const TSharedRef<SGraphNode>&)> Ftor) const
{
	if(!ControlFlowSequence)
	{
		return;
	}

	FChildren* ControlFlowChildren = ControlFlowSequence->GetChildren();
	if(ControlFlowChildren->Num() <= 1)
	{
		return;
	}

	for(int32 I = 1; I < ControlFlowChildren->Num(); ++I)
	{
		StaticCastSharedRef<SGraphNode>(ControlFlowChildren->GetChildAt(I))->ForEachNodeIncludingChildren(Ftor);
	}
}

void SGraphNode::ForEachNodeSnappedToDataPin(TFunctionRef<void(const TSharedRef<SGraphNode>&)> Ftor) const
{
	for( TSharedRef<SGraphPin> InputPin : InputPins )
	{
		TSharedPtr<SGraphNode> SnappedNode = InputPin->GetSnappedNode();
		if(SnappedNode)
		{
			SnappedNode->ForEachNodeIncludingChildren(Ftor);
		}
	}
	
	for( TSharedRef<SGraphPin> OutputPin : OutputPins )
	{
		TSharedPtr<SGraphNode> SnappedNode = OutputPin->GetSnappedNode();
		if(SnappedNode)
		{
			SnappedNode->ForEachNodeIncludingChildren(Ftor);
		}
	}
}
