// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphNodeComment.h"

#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphNode_Comment.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "SCommentBubble.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Templates/Casts.h"
//#include "TextWrapperHelpers.h"
#include "TutorialMetaData.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

class FDragDropEvent;

namespace SCommentNodeDefs
{

	/** Size of the hit result border for the window borders */
	/* L, T, R, B */
	static const FSlateRect HitResultBorderSize(10,10,10,10);

	/** Minimum resize width for comment */
	static const float MinWidth = 30.0;

	/** Minimum resize height for comment */
	static const float MinHeight = 30.0;

	/** TitleBarColor = CommentColor * TitleBarColorMultiplier */
	static const float TitleBarColorMultiplier = 0.6f;

	/** Titlebar Offset - taken from the widget borders in UpdateGraphNode */
	static const FSlateRect TitleBarOffset(13,8,-3,0);
}


void SGraphNodeComment::Construct(const FArguments& InArgs, UEdGraphNode_Comment* InNode)
{
	this->GraphNode = InNode;
	this->bIsSelected = false;

	// Set up animation
	{
		ZoomCurve = SpawnAnim.AddCurve(0, 0.1f);
		FadeCurve = SpawnAnim.AddCurve(0.15f, 0.15f);
	}

	// Cache these values so they do not force a re-build of the node next tick.
	CachedCommentTitle = GetNodeComment();
	CachedWidth = InNode->NodeWidth;

	this->UpdateGraphNode();

	// Pull out sizes
	UserSize = InNode->GetSize();

	// Cache desired size so we cull correctly. We can do this as our ComputeDesiredSize ignores the layout scale.
	CacheDesiredSize(1.0f);

	MouseZone = CRWZ_NotInWindow;
	bUserIsDragging = false;
}

void SGraphNodeComment::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SGraphNode::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const FString CurrentCommentTitle = GetNodeComment();
	if (CurrentCommentTitle != CachedCommentTitle)
	{
		CachedCommentTitle = CurrentCommentTitle;
	}

	const int32 CurrentWidth = static_cast<int32>(UserSize.X);
	if (CurrentWidth != CachedWidth)
	{
		CachedWidth = CurrentWidth;
	}

	UEdGraphNode_Comment* CommentNode = CastChecked<UEdGraphNode_Comment>(GraphNode);
	if (bCachedBubbleVisibility != CommentNode->bCommentBubbleVisible_InDetailsPanel)
	{
		CommentBubble->UpdateBubble();
		bCachedBubbleVisibility = CommentNode->bCommentBubbleVisible_InDetailsPanel;
	}

	if (CachedFontSize != CommentNode->GetFontSize())
	{
		UpdateGraphNode();
	}
}

FReply SGraphNodeComment::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	return FReply::Unhandled();
}

void SGraphNodeComment::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{

}

float SGraphNodeComment::GetWrapAt() const
{
	return static_cast<float>(CachedWidth) - 32.0f;
}

bool SGraphNodeComment::IsNameReadOnly() const
{
	return !IsEditable.Get() || SGraphNode::IsNameReadOnly();
}

void SGraphNodeComment::UpdateGraphNode()
{
	// No pins in a comment box
	InputPins.Empty();
	OutputPins.Empty();

	// Avoid standard box model too
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	// Remember if we should be showing the bubble
	UEdGraphNode_Comment* CommentNode = CastChecked<UEdGraphNode_Comment>(GraphNode);
	bCachedBubbleVisibility = CommentNode->bCommentBubbleVisible_InDetailsPanel;

	SetupErrorReporting();

	// Setup a meta tag for this node
	FGraphNodeMetaData TagMeta(TEXT("Graphnode"));
	PopulateMetaTag(&TagMeta);

	CachedFontSize = CommentNode->GetFontSize();

	CommentStyle = FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("Graph.CommentBlock.TitleInlineEditableText");
	CommentStyle.EditableTextBoxStyle.TextStyle.Font.Size = static_cast<float>(CachedFontSize);
	CommentStyle.TextStyle.Font.Size = static_cast<float>(CachedFontSize);

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("Kismet.Comment.Background") )
			.ColorAndOpacity( FLinearColor::White )
			.BorderBackgroundColor( this, &SGraphNodeComment::GetCommentBodyColor )
			.Padding(  FMargin(3.0f) )
			.AddMetaData<FGraphNodeMetaData>(TagMeta)
			[
				SNew(SVerticalBox)
				.ToolTipText( this, &SGraphNode::GetNodeTooltip )
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SAssignNew(TitleBar, SBorder)
					.BorderImage( FAppStyle::GetBrush("Graph.Node.TitleBackground") )
					.BorderBackgroundColor( this, &SGraphNodeComment::GetCommentTitleBarColor )
					.Padding( FMargin(10,5,5,3) )
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SAssignNew(InlineEditableText, SInlineEditableTextBlock)
						.Style( &CommentStyle )
						.Text( this, &SGraphNodeComment::GetEditableNodeTitleAsText )
						.OnVerifyTextChanged(this, &SGraphNodeComment::OnVerifyNameTextChanged)
						.OnTextCommitted(this, &SGraphNodeComment::OnNameTextCommited)
						.IsReadOnly( this, &SGraphNodeComment::IsNameReadOnly )
						.IsSelected( this, &SGraphNodeComment::IsSelectedExclusively )
						.WrapTextAt( this, &SGraphNodeComment::GetWrapAt )
						.MultiLine(true)
						.ModiferKeyForNewLine(EModifierKey::Shift)
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(1.0f)
				[
					ErrorReporting->AsWidget()
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					// NODE CONTENT AREA
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush("NoBorder") )
				]
			]
		];

	// Create comment bubble
	CommentBubble = SNew(SCommentBubble)
	.GraphNode(GraphNode)
	.Text(this, &SGraphNodeComment::GetNodeComment)
	.OnTextCommitted(this, &SGraphNodeComment::OnNameTextCommited)
	.ColorAndOpacity(this, &SGraphNodeComment::GetCommentBubbleColor )
	.AllowPinning(true)
	.EnableTitleBarBubble(false)
	.EnableBubbleCtrls(false)
	.GraphLOD(this, &SGraphNode::GetCurrentLOD)
	.InvertLODCulling(true)
	.IsGraphNodeHovered(this, &SGraphNode::IsHovered);

	GetOrAddSlot(ENodeZone::TopCenter)
	.SlotOffset2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetOffset2f))
	.SlotSize2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetSize2f))
	.AllowScaling(TAttribute<bool>(CommentBubble.Get(), &SCommentBubble::IsScalingAllowed))
	.VAlign(VAlign_Top)
	[
		CommentBubble.ToSharedRef()
	];
}

FVector2D SGraphNodeComment::ComputeDesiredSize( float ) const
{
	return FVector2D(UserSize);
}

FString SGraphNodeComment::GetNodeComment() const
{
	const FString Title = GetEditableNodeTitle();;
	return Title;
}

FReply SGraphNodeComment::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	// If user double-clicked in the title bar area
	if(FindMouseZone(InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition())) == CRWZ_TitleBar && IsEditable.Get())
	{
		// Request a rename
		RequestRename();

		// Set the keyboard focus
		if(!HasKeyboardFocus())
		{
			FSlateApplication::Get().SetKeyboardFocus(SharedThis(this), EFocusCause::SetDirectly);
		}

		return FReply::Handled();
	}
	else
	{
		// Otherwise let the graph handle it, to allow spline interactions to work when they overlap with a comment node
		return FReply::Unhandled();
	}
}

FReply SGraphNodeComment::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && bUserIsDragging )
	{
		bUserIsDragging = false;

		// Resize the node
		UserSize.X = FMath::RoundToFloat(UserSize.X);
		UserSize.Y = FMath::RoundToFloat(UserSize.Y);

		GetNodeObj()->ResizeNode(UserSize);
		Invalidate(EInvalidateWidgetReason::Prepass);

		// End resize transaction
		ResizeTransactionPtr.Reset();

		// Update contained child Nodes
		HandleSelection( bIsSelected, true );

		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

int32 SGraphNodeComment::GetSortDepth() const
{
	UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>( GraphNode );
	return CommentNode ? CommentNode->CommentDepth : -1;
}

void SGraphNodeComment::HandleSelection(bool bSelected, bool bUpdateNodesUnderComment) const
{
	const FVector2f NodeSize = GetDesiredSize();
	// we only want to do this after the comment has a valid desired size
	if( !NodeSize.IsZero() )
	{
		if ((!this->bIsSelected && bSelected) || bUpdateNodesUnderComment)
		{
			SGraphNodeComment* Comment = const_cast<SGraphNodeComment*> (this);
			UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(GraphNode);
			if (CommentNode)
			{
				TSharedPtr<SGraphPanel> Panel = Comment->GetOwnerPanel();
				FChildren* PanelChildren = Panel->GetManagedChildren();
				int32 NumChildren = PanelChildren->Num();
				CommentNode->ClearNodesUnderComment();

				for ( int32 NodeIndex=0; NodeIndex < NumChildren; ++NodeIndex )
				{
					const TSharedRef<SGraphNode> SomeNodeWidget = StaticCastSharedRef<SGraphNode>(PanelChildren->GetChildAt(NodeIndex));
					UObject* GraphObject = SomeNodeWidget->GetObjectBeingDisplayed();
					if (GraphObject != CommentNode)
					{
						if (IsNodeUnderComment(CommentNode, SomeNodeWidget))
						{
							CommentNode->AddNodeUnderComment(GraphObject);
						}
					}
				}
			}
		}
		bIsSelected = bSelected;
	}
}

bool SGraphNodeComment::IsNodeUnderComment(UEdGraphNode_Comment* InCommentNode, const TSharedRef<SGraphNode> InNodeWidget) const
{
	const FVector2f NodePosition = GetPosition2f();
	const FVector2f NodeSize = GetDesiredSize();
	const FSlateRect CommentRect(NodePosition.X, NodePosition.Y, NodePosition.X + NodeSize.X, NodePosition.Y + NodeSize.Y);

	const FVector2f InNodePosition = InNodeWidget->GetPosition2f();
	const FVector2f InNodeSize = InNodeWidget->GetDesiredSize();

	const FSlateRect NodeGeometryGraphSpace(InNodePosition.X, InNodePosition.Y, InNodePosition.X + InNodeSize.X, InNodePosition.Y + InNodeSize.Y);
	return FSlateRect::IsRectangleContained(CommentRect, NodeGeometryGraphSpace);
}

const FSlateBrush* SGraphNodeComment::GetShadowBrush(bool bSelected) const
{
	HandleSelection(bSelected);
	return SGraphNode::GetShadowBrush(bSelected);
}

void SGraphNodeComment::GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const
{
	const float Fudge = 3.0f;

	HandleSelection(bSelected);

	FOverlayBrushInfo HandleBrush = FAppStyle::GetBrush( TEXT("Graph.Node.Comment.Handle") );

	HandleBrush.OverlayOffset.X = WidgetSize.X - HandleBrush.Brush->ImageSize.X - Fudge;
	HandleBrush.OverlayOffset.Y = WidgetSize.Y - HandleBrush.Brush->ImageSize.Y - Fudge;

	Brushes.Add(HandleBrush);
	return SGraphNode::GetOverlayBrushes(bSelected, UE::Slate::CastToVector2f(WidgetSize), Brushes);
}

void SGraphNodeComment::MoveTo( const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	FVector2f PositionDelta = NewPosition - GetPosition2f();
	SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);
	// Don't drag note content if either of the shift keys are down.
	FModifierKeysState KeysState = FSlateApplication::Get().GetModifierKeys();
	if(!KeysState.IsShiftDown())
	{
		UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(GraphNode);
		if (CommentNode && CommentNode->MoveMode == ECommentBoxMode::GroupMovement)
		{
			// Now update any nodes which are touching the comment but *not* selected
			// Selected nodes will be moved as part of the normal selection code
			TSharedPtr< SGraphPanel > Panel = GetOwnerPanel();

			for (FCommentNodeSet::TConstIterator NodeIt( CommentNode->GetNodesUnderComment() ); NodeIt; ++NodeIt)
			{
				if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
				{
					if (!Panel->SelectionManager.IsNodeSelected(Node) && !NodeFilter.Find(Node->DEPRECATED_NodeWidget.Pin()))
					{
						NodeFilter.Add(Node->DEPRECATED_NodeWidget.Pin());
						Node->Modify(bMarkDirty);
						Node->SetPosition(Node->GetPosition() + PositionDelta);
					}
				}
			}
		}
	}
}

void SGraphNodeComment::EndUserInteraction() const
{
	// Find any parent comments and their list of child nodes
	const FVector2f NodeSize = GetDesiredSize();
	if( !NodeSize.IsZero() )
	{
		const FVector2f NodePosition = GetPosition2f();
		const FSlateRect CommentRect( NodePosition.X, NodePosition.Y, NodePosition.X + NodeSize.X, NodePosition.Y + NodeSize.Y );

		TSharedPtr<SGraphPanel> Panel = GetOwnerPanel();
		FChildren* PanelChildren = Panel->GetManagedChildren();
		int32 NumChildren = PanelChildren->Num();
		static FString SGraphNodeCommentType = "SGraphNodeComment";

		for ( int32 NodeIndex=0; NodeIndex < NumChildren; ++NodeIndex )
		{
			const TSharedPtr<SGraphNode> SomeNodeWidget = StaticCastSharedRef<SGraphNode>(PanelChildren->GetChildAt(NodeIndex));

			UObject* GraphObject = SomeNodeWidget->GetObjectBeingDisplayed();
			if ( !GraphObject->IsA<UEdGraphNode_Comment>() )
			{
				continue;
			}

			const FVector2f SomeNodePosition = SomeNodeWidget->GetPosition2f();
			const FVector2f SomeNodeSize = SomeNodeWidget->GetDesiredSize();

			const FSlateRect NodeGeometryGraphSpace(SomeNodePosition.X, SomeNodePosition.Y, SomeNodePosition.X + SomeNodeSize.X, SomeNodePosition.Y + SomeNodeSize.Y);
			if (FSlateRect::DoRectanglesIntersect(CommentRect, NodeGeometryGraphSpace))
			{
				// This downcast *should* be valid at this point, since we verified the GraphObject is a comment node
				TSharedPtr<SGraphNodeComment> CommentWidget = StaticCastSharedPtr<SGraphNodeComment>(SomeNodeWidget);
				CommentWidget->HandleSelection(CommentWidget->bIsSelected, true);
			}
		}
	}
}

float SGraphNodeComment::GetTitleBarHeight() const
{
	return TitleBar.IsValid() ? TitleBar->GetDesiredSize().Y : 0.0f;
}

FSlateRect SGraphNodeComment::GetHitTestingBorder() const
{
	return SCommentNodeDefs::HitResultBorderSize;
}

FVector2f SGraphNodeComment::GetNodeMaximumSize2f() const
{
	return FVector2f( UserSize.X + 100.0f, UserSize.Y + 100.0f );
}

FSlateColor SGraphNodeComment::GetCommentBodyColor() const
{
	UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(GraphNode);

	if (CommentNode)
	{
		return CommentNode->CommentColor;
	}
	else
	{
		return FLinearColor::White;
	}
}

FSlateColor SGraphNodeComment::GetCommentTitleBarColor() const
{
	UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(GraphNode);
	if (CommentNode)
	{
		const FLinearColor Color = CommentNode->CommentColor * SCommentNodeDefs::TitleBarColorMultiplier;
		return FLinearColor(Color.R, Color.G, Color.B);
	}
	else
	{
		const FLinearColor Color = FLinearColor::White * SCommentNodeDefs::TitleBarColorMultiplier;
		return FLinearColor(Color.R, Color.G, Color.B);
	}
}

FSlateColor SGraphNodeComment::GetCommentBubbleColor() const
{
	UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(GraphNode);
	if (CommentNode)
	{
		const FLinearColor Color = CommentNode->bColorCommentBubble ?	(CommentNode->CommentColor * SCommentNodeDefs::TitleBarColorMultiplier) :
																		GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;
		return FLinearColor(Color.R, Color.G, Color.B);
	}
	else
	{
		const FLinearColor Color = FLinearColor::White * SCommentNodeDefs::TitleBarColorMultiplier;
		return FLinearColor(Color.R, Color.G, Color.B);
	}
}

bool SGraphNodeComment::CanBeSelected(const FVector2f& MousePositionInNode) const
{
	const EResizableWindowZone InMouseZone = FindMouseZone(MousePositionInNode);
	return CRWZ_TitleBar == InMouseZone;
}

FVector2f SGraphNodeComment::GetDesiredSizeForMarquee2f() const
{
	const float TitleBarHeight = TitleBar.IsValid() ? TitleBar->GetDesiredSize().Y : 0.0f;
	return FVector2f(UserSize.X, TitleBarHeight);
}

FSlateRect SGraphNodeComment::GetTitleRect() const
{
	const FVector2f NodePosition = GetPosition2f();
	FVector2f NodeSize  = TitleBar.IsValid() ? TitleBar->GetDesiredSize() : GetDesiredSize();
	return FSlateRect( NodePosition.X, NodePosition.Y, NodePosition.X + NodeSize.X, NodePosition.Y + NodeSize.Y ) + SCommentNodeDefs::TitleBarOffset;
}

void SGraphNodeComment::PopulateMetaTag(FGraphNodeMetaData* TagMeta) const
{
	if (GraphNode != nullptr)
	{
		// We want the name of the blueprint as our name - we can find the node from the GUID
		UObject* LastOuter = GraphNode->GetOutermostObject();
		TagMeta->Tag = FName(*FString::Printf(TEXT("GraphNode_%s_%s"), *LastOuter->GetFullName(), *GraphNode->NodeGuid.ToString()));
		TagMeta->OuterName = LastOuter->GetFullName();
		TagMeta->GUID = GraphNode->NodeGuid;
		TagMeta->FriendlyName = FString::Printf(TEXT("%s in %s"), *GraphNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString(), *TagMeta->OuterName);
	}
}
