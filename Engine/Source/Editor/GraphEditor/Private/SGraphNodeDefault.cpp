// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphNodeDefault.h"

#include "GenericPlatform/ICursor.h"
#include "Misc/Optional.h"

void SGraphNodeDefault::Construct( const FArguments& InArgs )
{
	this->GraphNode = InArgs._GraphNodeObj;

	this->SetCursor( EMouseCursor::CardinalCross );

	this->UpdateGraphNode();
}
