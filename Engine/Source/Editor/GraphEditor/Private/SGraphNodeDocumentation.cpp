// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphNodeDocumentation.h"

#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformMath.h"
#include "IDocumentation.h"
#include "IDocumentationPage.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "SGraphNode.h"
#include "SLevelOfDetailBranchNode.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "TutorialMetaData.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Colors/SSimpleGradient.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

struct FPointerEvent;

#define LOCTEXT_NAMESPACE "SGraphNodeDocumentation"

namespace GraphNodeDocumentationDefs
{
	/** Size of the hit result border for the window borders */
	static const FSlateRect HitTestBorderSize( 10, 10, 8, 14 );

	/** Minimum size for node */
	static const FVector2f MinNodeSize( 200.0f, 10.0f );

	/** Maximum size for node */
	static const FVector2f MaximumNodeSize( 4000.0f, 10.0f );

	/** Default documentation content size */
	static const FVector2f DefaultContentSize( 600.0f, 400.0f );

	/** Placeholder documentation content size */
	static const FVector2f PlaceholderContentSize( 380.0f, 45.0f );

	/** Default content border */
	static const FMargin DefaultContentBorder( 4.0f, 2.0f, 4.0f, 10.0f );

	/** Line wrap adjustment from node width, to account for scroll bar */
	static const float LineWrapAdjustment = 20.f;

	/** Documentation page gradient colors */
	static const FLinearColor PageGradientStartColor( 0.85f, 0.85f, 0.85f, 1.f );
	static const FLinearColor PageGradientEndColor( 0.75f, 0.75f, 0.75f, 1.f );
}

void SGraphNodeDocumentation::Construct( const FArguments& InArgs, UEdGraphNode* InNode )
{
	GraphNode = InNode;

	// Set up animation
	{
		ZoomCurve = SpawnAnim.AddCurve( 0, 0.1f );
		FadeCurve = SpawnAnim.AddCurve( 0.15f, 0.15f );
	}
	UserSize = InNode->GetSize();
	bUserIsDragging = false;
	UpdateGraphNode();
}

void SGraphNodeDocumentation::UpdateGraphNode()
{
	// No pins in a document box
	InputPins.Empty();
	OutputPins.Empty();

	// Avoid standard box model too
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	SetupErrorReporting();

	// Create Node Title
	TSharedPtr<SNodeTitle> NodeTitle = SNew( SNodeTitle, GraphNode );

	// Setup a meta tag for this node
	FGraphNodeMetaData TagMeta(TEXT("Graphnode"));
	PopulateMetaTag(&TagMeta);

	TSharedRef<SOverlay> DefaultTitleAreaWidget =
		SNew( SOverlay )
		.AddMetaData<FGraphNodeMetaData>(TagMeta)
		+SOverlay::Slot()
		[
			SNew( SImage )
			.Image( FAppStyle::GetBrush( "Graph.Node.TitleGloss" ))
		]
		+SOverlay::Slot()
		.HAlign( HAlign_Left )
		.VAlign( VAlign_Center )
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush( "Graph.Node.ColorSpill" ))
			.Padding( FMargin( 10, 5, 30, 3 ))
			.BorderBackgroundColor( this, &SGraphNodeDocumentation::GetNodeTitleColor )
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew( InlineEditableText,SInlineEditableTextBlock )
					.Style( FAppStyle::Get(), "Graph.Node.NodeTitleInlineEditableText" )
					.Text( this, &SGraphNodeDocumentation::GetDocumentationTitle )
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					NodeTitle.ToSharedRef()
				]
			]
		]
		+SOverlay::Slot()
		.VAlign( VAlign_Top )
		[
			SNew( SBorder )
			.Visibility( EVisibility::HitTestInvisible )			
			.BorderImage( FAppStyle::GetBrush( "Graph.Node.TitleHighlight" ))
			[
				SNew( SSpacer )
				.Size( FVector2D( 20, 20 ))
			]
		];

	// Create Node content
	SAssignNew( TitleBar, SLevelOfDetailBranchNode )
	.UseLowDetailSlot( this, &SGraphNodeDocumentation::UseLowDetailNodeTitles )
	.LowDetail()
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush( "Graph.Node.ColorSpill" ))
		.BorderBackgroundColor( this, &SGraphNodeDocumentation::GetNodeTitleColor )
	]
	.HighDetail()
	[
		DefaultTitleAreaWidget
	];

	// Create Documentation Page
	TSharedPtr<SWidget> DocumentationPage = CreateDocumentationPage();

	TSharedPtr<SVerticalBox> InnerVerticalBox;
	GetOrAddSlot( ENodeZone::Center )
	.HAlign( HAlign_Center )
	.VAlign( VAlign_Center )
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign( HAlign_Fill )
		.VAlign( VAlign_Fill )
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush( "Graph.Node.Body" ))
			.Visibility( this, &SGraphNodeDocumentation::GetWidgetVisibility )
			.Padding( 0.f )
			[
				SAssignNew( InnerVerticalBox, SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign( HAlign_Fill )
				.VAlign( VAlign_Top )
				[
					TitleBar.ToSharedRef()
				]
				+SVerticalBox::Slot()
				.HAlign( HAlign_Left )
				.VAlign( VAlign_Top )
				[
					DocumentationPage.ToSharedRef()
				]
			]
		]
	];
}

FText SGraphNodeDocumentation::GetDocumentationTitle() const
{
	return FText::Format( LOCTEXT( "DocumentationNode",  "UDN - {0}" ), FText::FromString( GraphNode->GetDocumentationExcerptName() ));
}

TSharedPtr<SWidget> SGraphNodeDocumentation::CreateDocumentationPage()
{
	TSharedPtr<SWidget> DocumentationWidget;

	if( IDocumentation::Get()->PageExists( GraphNode->GetDocumentationLink() ))
	{
		TSharedRef< IDocumentationPage > DocumentationPage = IDocumentation::Get()->GetPage( GraphNode->GetDocumentationLink(), NULL );
		TMap< FString, FString > InVariables;
		FExcerpt DesiredExcerpt( GraphNode->GetDocumentationExcerptName(), SNullWidget::NullWidget, InVariables, 0 );

		// Set attributes to control documentation WrapAt and optional width
		DocumentationPage->SetTextWrapAt( TAttribute<float>( this, &SGraphNodeDocumentation::GetDocumentationWrapWidth ) );

		if( DocumentationPage->GetExcerptContent( DesiredExcerpt ))
		{
			// Create Content
			SAssignNew( DocumentationWidget, SBox )
			.WidthOverride( this, &SGraphNodeDocumentation::GetContentWidth )
			.HeightOverride( this, &SGraphNodeDocumentation::GetContentHeight )
			[
				SAssignNew( ContentWidget, SVerticalBox )
				+SVerticalBox::Slot()
				.Padding( GraphNodeDocumentationDefs::DefaultContentBorder )
				[
					SNew( SBorder )
					.HAlign( HAlign_Left )
					.Content()
					[
						SNew( SScrollBox )
						+SScrollBox::Slot()
						[
							SNew(SOverlay)
							+SOverlay::Slot()
							[
								SNew(SSimpleGradient)
								.StartColor(this, &SGraphNodeDocumentation::GetPageGradientStartColor)
								.EndColor(this, &SGraphNodeDocumentation::GetPageGradientEndColor)
							]
							+SOverlay::Slot()
							[
								DesiredExcerpt.Content.ToSharedRef()
							]
						]
					]
				]
			];
			// Find Maximum Content area for resizing
			UserSize = GraphNodeDocumentationDefs::MaximumNodeSize;
			ContentWidget->SlatePrepass();
			DocumentationSize = ContentWidget->GetDesiredSize();
			if( GraphNode->NodeWidth != 0 && GraphNode->NodeHeight != 0 )
			{
				// restore node size
				UserSize = GraphNode->GetSize();
			}
			else
			{
				// set initial size
				UserSize = GraphNodeDocumentationDefs::DefaultContentSize;
				ContentWidget->SlatePrepass();
				UserSize = ContentWidget->GetDesiredSize();
			}
		}
	}
	if( !DocumentationWidget.IsValid() )
	{
		// Create Placeholder
		SAssignNew( DocumentationWidget, SBox )
		.WidthOverride( this, &SGraphNodeDocumentation::GetContentWidth )
		.HeightOverride( this, &SGraphNodeDocumentation::GetContentHeight )
		[
			SAssignNew( ContentWidget, SVerticalBox )
			+SVerticalBox::Slot()
			.Padding( GraphNodeDocumentationDefs::DefaultContentBorder )
			[
				SNew( SBorder )
				.HAlign( HAlign_Left )
				.Content()
				[
					SNew( SScrollBox )
					+SScrollBox::Slot()
					[
						SNew( STextBlock )
						.WrapTextAt( this, &SGraphNodeDocumentation::GetDocumentationWrapWidth )
						.Text( LOCTEXT( "InvalidContentNotification", "No valid content to display.Please choose a valid link and excerpt in the details panel" ))
					]
				]
			]
		];
		// Find Maximum Content area for resizing
		UserSize = GraphNodeDocumentationDefs::PlaceholderContentSize;
		DocumentationSize = GraphNodeDocumentationDefs::PlaceholderContentSize;
	}
	// Cache link/excerpt this widget was based on
	CachedDocumentationLink = GraphNode->GetDocumentationLink();
	CachedDocumentationExcerpt = GraphNode->GetDocumentationExcerptName();
	
	return DocumentationWidget;
}

FOptionalSize SGraphNodeDocumentation::GetContentWidth() const
{
	return UserSize.X;
}

FOptionalSize SGraphNodeDocumentation::GetContentHeight() const
{
	return UserSize.Y;
}

float SGraphNodeDocumentation::GetDocumentationWrapWidth() const
{
	return UserSize.X - GraphNodeDocumentationDefs::LineWrapAdjustment;
}

FVector2D SGraphNodeDocumentation::ComputeDesiredSize( float ) const
{
	return FVector2D( UserSize.X, UserSize.Y + GetTitleBarHeight() );
}

FVector2f SGraphNodeDocumentation::GetNodeMinimumSize2f() const
{
	return GraphNodeDocumentationDefs::MinNodeSize;
}

FVector2f SGraphNodeDocumentation::GetNodeMaximumSize2f() const
{
	return FVector2f( DocumentationSize.X, ContentWidget->GetDesiredSize().Y );
}

FReply SGraphNodeDocumentation::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return FReply::Unhandled();
}

EVisibility SGraphNodeDocumentation::GetWidgetVisibility() const
{
	return ChildWidgetVisibility;
}

float SGraphNodeDocumentation::GetTitleBarHeight() const
{
	return TitleBar.IsValid() ? TitleBar->GetDesiredSize().Y : 0.f;
}

FSlateRect SGraphNodeDocumentation::GetHitTestingBorder() const
{
	return GraphNodeDocumentationDefs::HitTestBorderSize;
}

void SGraphNodeDocumentation::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if( !bUserIsDragging )
	{
		ChildWidgetVisibility = EVisibility::HitTestInvisible;
		FVector2f LocalMouseCoordinates = AllottedGeometry.AbsoluteToLocal( FSlateApplication::Get().GetCursorPos() );

		EResizableWindowZone CurrMouseZone = FindMouseZone( LocalMouseCoordinates );
		
		if( CurrMouseZone == CRWZ_InWindow )
		{
			ChildWidgetVisibility = EVisibility::Visible;
		}
	}
	// Check Cached Links to determine if we need to update the documentation link/excerpt
	if( CachedDocumentationLink != GraphNode->GetDocumentationLink() ||
		CachedDocumentationExcerpt != GraphNode->GetDocumentationExcerptName())
	{
		GraphNode->NodeWidth = 0;
		GraphNode->NodeHeight = 0;
		UpdateGraphNode();
	}
}

FLinearColor SGraphNodeDocumentation::GetPageGradientStartColor() const
{
	FLinearColor Color = GraphNodeDocumentationDefs::PageGradientStartColor;
	return Color;
}

FLinearColor SGraphNodeDocumentation::GetPageGradientEndColor() const
{
	FLinearColor Color = GraphNodeDocumentationDefs::PageGradientEndColor;
	return Color;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
