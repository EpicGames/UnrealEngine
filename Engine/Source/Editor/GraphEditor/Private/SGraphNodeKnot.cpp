// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphNodeKnot.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "DragConnection.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SCommentBubble.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "SNodePanel.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

class FText;
class SWidget;
struct FGeometry;
struct FSlateBrush;

namespace SKnotNodeDefinitions
{
	/** Offset from the left edge to display comment toggle button at */
	static const float KnotCenterButtonAdjust = 3.f;

	/** Offset from the left edge to display comment bubbles at */
	static const float KnotCenterBubbleAdjust = 20.f;

	/** Knot node spacer sizes */
	static const FVector2f NodeSpacerSize(42.0f, 24.0f);
}

class FAmbivalentDirectionDragConnection : public FDragConnection
{
public:
	// FDragDropOperation interface
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;
	// End of FDragDropOperation interface

	UEdGraphPin* GetBestPin() const;

	// FDragConnection interface
	virtual void ValidateGraphPinList(TArray<UEdGraphPin*>& OutValidPins) override;
	// End of FDragConnection interface


	static TSharedRef<FAmbivalentDirectionDragConnection> New(UEdGraphNode* InKnot, const TSharedRef<SGraphPanel>& InGraphPanel, const FDraggedPinTable& InStartingPins)
	{
		TSharedRef<FAmbivalentDirectionDragConnection> Operation = MakeShareable(new FAmbivalentDirectionDragConnection(InKnot, InGraphPanel, InStartingPins));
		Operation->Construct();

		return Operation;
	}

protected:
	FAmbivalentDirectionDragConnection(UEdGraphNode* InKnot, const TSharedRef<SGraphPanel>& InGraphPanel, const FDraggedPinTable& InStartingPins)
		: FDragConnection(InGraphPanel, InStartingPins)
		, KnotPtr(InKnot)
		, StartScreenPos(FVector2f::ZeroVector)
		, MostRecentScreenPos(FVector2f::ZeroVector)
		, bLatchedStartScreenPos(false)
	{
	}

protected:
	TWeakObjectPtr<UEdGraphNode> KnotPtr;

	FVector2f StartScreenPos;
	FVector2f MostRecentScreenPos;
	bool bLatchedStartScreenPos;
};

UEdGraphPin* FAmbivalentDirectionDragConnection::GetBestPin() const
{
	if (bLatchedStartScreenPos)
	{
		int32 InputPinIndex = -1;
		int32 OutputPinIndex = -1;

		if (KnotPtr.Get() && KnotPtr->ShouldDrawNodeAsControlPointOnly(InputPinIndex, OutputPinIndex))
		{
			const bool bIsRight = MostRecentScreenPos.X >= StartScreenPos.X;
			return bIsRight ? KnotPtr->GetPinAt(OutputPinIndex) : KnotPtr->GetPinAt(InputPinIndex);
		}
	}

	return nullptr;
}

void FAmbivalentDirectionDragConnection::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	if (bLatchedStartScreenPos)
	{
		const FVector2f LastScreenPos = MostRecentScreenPos;
		MostRecentScreenPos = DragDropEvent.GetScreenSpacePosition();

		// Switch directions on the preview connector as we cross from left to right of the starting drag point (or vis versa)
		const bool bWasRight = LastScreenPos.X >= StartScreenPos.X;
		const bool bIsRight = MostRecentScreenPos.X >= StartScreenPos.X;

		if (bWasRight ^ bIsRight)
		{
			GraphPanel->OnStopMakingConnection(/*bForceStop=*/ true);
			GraphPanel->OnBeginMakingConnection(GetBestPin());
		}
	}
	else
	{
		StartScreenPos = DragDropEvent.GetScreenSpacePosition();
		MostRecentScreenPos = StartScreenPos;
		bLatchedStartScreenPos = true;
	}

	FDragConnection::OnDragged(DragDropEvent);
}

void FAmbivalentDirectionDragConnection::ValidateGraphPinList(TArray<UEdGraphPin*>& OutValidPins)
{
	OutValidPins.Empty(DraggingPins.Num());

   if (KnotPtr.Get() != nullptr)
   {
		bool bUseOutput = true;

		// Pick output or input based on if the drag op is currently to the left or to the right of the starting drag point
		if (bLatchedStartScreenPos)
		{
			bUseOutput = (StartScreenPos.X < MostRecentScreenPos.X);
		}

		if (UEdGraphPin* TargetPinObj = GetHoveredPin())
		{
			// Dragging to another pin, pick the opposite direction as a source to maximize connection chances
			if (TargetPinObj->Direction == EGPD_Input)
			{
				bUseOutput = true;
			}
			else
			{
				bUseOutput = false;
			}
		}

		int32 InputPinIndex = -1;
		int32 OutputPinIndex = -1;

		if (KnotPtr->ShouldDrawNodeAsControlPointOnly(InputPinIndex, OutputPinIndex))
		{
			// Switch the effective valid pin so it makes sense for the current drag context
			if (bUseOutput)
			{
				OutValidPins.Add(KnotPtr->GetPinAt(OutputPinIndex));
			}
			else
			{
				OutValidPins.Add(KnotPtr->GetPinAt(InputPinIndex));
			}
		}
	}
	else
	{
		// Fall back to the default behavior
		FDragConnection::ValidateGraphPinList(OutValidPins);
	}
}

/////////////////////////////////////////////////////
// SGraphPinKnot

void SGraphPinKnot::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPin::Construct(SGraphPin::FArguments().SideToSideMargin(0.0f), InPin);
}

void SGraphPinKnot::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid() && Operation->IsOfType<FDragConnection>())
	{
		TSharedPtr<FDragConnection> DragConnectionOp = StaticCastSharedPtr<FDragConnection>(Operation);

		TArray<UEdGraphPin*> ValidPins;
		DragConnectionOp->ValidateGraphPinList(/*out*/ ValidPins);
		
		if (ValidPins.Num() > 0)
		{
			UEdGraphPin* PinToHoverOver = nullptr;
			UEdGraphNode* Knot = GraphPinObj->GetOwningNode();
			int32 InputPinIndex = -1;
			int32 OutputPinIndex = -1;

			if (Knot != nullptr && Knot->ShouldDrawNodeAsControlPointOnly(InputPinIndex, OutputPinIndex))
			{
				// Dragging to another pin, pick the opposite direction as a source to maximize connection chances
				PinToHoverOver = (ValidPins[0]->Direction == EGPD_Input) ? Knot->GetPinAt(OutputPinIndex) : Knot->GetPinAt(InputPinIndex);
				check(PinToHoverOver);
			}

			if (PinToHoverOver != nullptr)
			{
				DragConnectionOp->SetHoveredPin(PinToHoverOver);

				// Pins treat being dragged over the same as being hovered outside of drag and drop if they know how to respond to the drag action.
				SBorder::OnMouseEnter(MyGeometry, DragDropEvent);

				return;
			}
		}
	}

	SGraphPin::OnDragEnter(MyGeometry, DragDropEvent);
}

FSlateColor SGraphPinKnot::GetPinColor() const
{
	// Make ourselves transparent if we're the input, since we are underneath the output pin and would double-blend looking ugly
	if (UEdGraphPin* PinObj = GetPinObj())
	{
		if (PinObj->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			return FLinearColor::Transparent;
		} 
	}
	return SGraphPin::GetPinColor();
}

TSharedRef<SWidget>	SGraphPinKnot::GetDefaultValueWidget()
{
	return SNullWidget::NullWidget;
}

TSharedRef<FDragDropOperation> SGraphPinKnot::SpawnPinDragEvent(const TSharedRef<SGraphPanel>& InGraphPanel, const TArray< TSharedRef<SGraphPin> >& InStartingPins)
{
	FAmbivalentDirectionDragConnection::FDraggedPinTable PinHandles;
	PinHandles.Reserve(InStartingPins.Num());
	// since the graph can be refreshed and pins can be reconstructed/replaced 
	// behind the scenes, the DragDropOperation holds onto FGraphPinHandles 
	// instead of direct widgets/graph-pins
	for (const TSharedRef<SGraphPin>& PinWidget : InStartingPins)
	{
		PinHandles.Add(PinWidget->GetPinObj());
	}

	TSharedRef<FAmbivalentDirectionDragConnection> Operation = FAmbivalentDirectionDragConnection::New(GetPinObj()->GetOwningNode(), InGraphPanel, PinHandles);
	return Operation;
}

FReply SGraphPinKnot::OnPinMouseDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (!GraphPinObj->bNotConnectable && IsEditingEnabled())
		{
			if (MouseEvent.IsAltDown())
			{
				// Normally break connections, but overloaded here to delete the node entirely
				const FScopedTransaction Transaction(FGenericCommands::Get().Delete->GetDescription());

				UEdGraphNode* NodeToDelete = GetPinObj()->GetOwningNode();
				if (NodeToDelete != nullptr)
				{
					UEdGraph* Graph = NodeToDelete->GetGraph();
					if (Graph != nullptr)
					{
						const UEdGraphSchema* Schema = Graph->GetSchema();
						if (Schema != nullptr && Schema->SafeDeleteNodeFromGraph(Graph, NodeToDelete))
						{
							return FReply::Handled();
						}
					}
				}

				return FReply::Unhandled();
			}
			else if (MouseEvent.IsControlDown())
			{
				// Normally moves the connections from one pin to another, but moves the node instead since it's really representing a set of connections
				// Returning unhandled will cause the node behind us to catch it and move us
				return FReply::Unhandled();
			}
		}
	}

	return SGraphPin::OnPinMouseDown(SenderGeometry, MouseEvent);
}

//////////////////////////////////////////////////////////////////////////
// SGraphNodeKnot

void SGraphNodeKnot::Construct(const FArguments& InArgs, UEdGraphNode* InKnot)
{
	int32 InputPinIndex = -1;
	int32 OutputPinIndex = -1;
	verify(InKnot->ShouldDrawNodeAsControlPointOnly(InputPinIndex, OutputPinIndex) == true &&
		InputPinIndex >= 0 && OutputPinIndex >= 0);
	SGraphNodeDefault::Construct(SGraphNodeDefault::FArguments().GraphNodeObj(InKnot));
}


void SGraphNodeKnot::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	//@TODO: Keyboard focus on edit doesn't work unless the node is visible, but the text is just the comment and it's already shown in a bubble, so Transparent black it is...
	InlineEditableText = SNew(SInlineEditableTextBlock)
		.ColorAndOpacity(FLinearColor::Transparent)
		.Style(FAppStyle::Get(), "Graph.Node.NodeTitleInlineEditableText")
		.Text(this, &SGraphNodeKnot::GetEditableNodeTitleAsText)
		.OnVerifyTextChanged(this, &SGraphNodeKnot::OnVerifyNameTextChanged)
		.OnTextCommitted(this, &SGraphNodeKnot::OnNameTextCommited)
		.IsReadOnly(this, &SGraphNodeKnot::IsNameReadOnly)
		.IsSelected(this, &SGraphNodeKnot::IsSelectedExclusively);

	SetupErrorReporting();

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				// Grab handle to be able to move the node
				SNew(SSpacer)
				.Size(SKnotNodeDefinitions::NodeSpacerSize)
				.Visibility(EVisibility::Visible)
				.Cursor(EMouseCursor::CardinalCross)
			]

			+SOverlay::Slot()
// 			.VAlign(VAlign_Center)
// 			.HAlign(HAlign_Center)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Center)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						[
							// LEFT
							SAssignNew(LeftNodeBox, SVerticalBox)
						]
						+SOverlay::Slot()
						[
							// RIGHT
							SAssignNew(RightNodeBox, SVerticalBox)
						]
					]
				]
				+SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Center)
				.AutoHeight()
				[
					ErrorReporting->AsWidget()
				]
			]
		];
	// Create comment bubble
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew(CommentBubble, SCommentBubble)
	.GraphNode(GraphNode)
	.Text(this, &SGraphNode::GetNodeComment)
	.OnTextCommitted(this, &SGraphNode::OnCommentTextCommitted)
	.EnableTitleBarBubble(true)
	.EnableBubbleCtrls(true)
	.AllowPinning(true)
	.ColorAndOpacity(CommentColor)
	.GraphLOD(this, &SGraphNode::GetCurrentLOD)
	.IsGraphNodeHovered(this, &SGraphNode::IsHovered)
	.OnToggled(this, &SGraphNode::OnCommentBubbleToggled);

	GetOrAddSlot(ENodeZone::TopCenter)
	.SlotOffset2f(TAttribute<FVector2f>(this, &SGraphNodeKnot::GetCommentOffset))
	.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f))
	.AllowScaling( TAttribute<bool>(CommentBubble.Get(), &SCommentBubble::IsScalingAllowed))
	.VAlign(VAlign_Top)
	[
		CommentBubble.ToSharedRef()
	];
	CreatePinWidgets();
}

const FSlateBrush* SGraphNodeKnot::GetShadowBrush(bool bSelected) const
{
	return bSelected ? FAppStyle::GetBrush(TEXT("Graph.Node.ShadowSelected")) : FAppStyle::GetNoBrush();
}

TSharedPtr<SGraphPin> SGraphNodeKnot::CreatePinWidget(UEdGraphPin* Pin) const
{
	return SNew(SGraphPinKnot, Pin);
}

void SGraphNodeKnot::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	PinToAdd->SetShowLabel(false);

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			//.Padding(10, 4)
			[
				PinToAdd
			];
		InputPins.Add(PinToAdd);
	}
	else
	{
		RightNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			//.Padding(10, 4)
			[
				PinToAdd
			];
		OutputPins.Add(PinToAdd);
	}
}

FVector2f SGraphNodeKnot::GetCommentOffset() const
{
	const bool bBubbleVisible = GraphNode->bCommentBubbleVisible || bAlwaysShowCommentBubble;
	const float ZoomAmount = GraphNode->bCommentBubblePinned && OwnerGraphPanelPtr.IsValid() ? OwnerGraphPanelPtr.Pin()->GetZoomAmount() : 1.f;
	const float NodeWidthOffset = bBubbleVisible ?	SKnotNodeDefinitions::KnotCenterBubbleAdjust * ZoomAmount :
													SKnotNodeDefinitions::KnotCenterButtonAdjust * ZoomAmount;

	return FVector2f(NodeWidthOffset - CommentBubble->GetArrowCenterOffset(), -CommentBubble->GetDesiredSize().Y);
}

void SGraphNodeKnot::OnCommentBubbleToggled(bool bInCommentBubbleVisible)
{
	SGraphNode::OnCommentBubbleToggled(bInCommentBubbleVisible);
	bAlwaysShowCommentBubble = bInCommentBubbleVisible;
}

void SGraphNodeKnot::OnCommentTextCommitted(const FText& NewComment, ETextCommit::Type CommitInfo)
{
	SGraphNode::OnCommentTextCommitted(NewComment, CommitInfo);
	if (!bAlwaysShowCommentBubble && !CommentBubble->TextBlockHasKeyboardFocus() && !CommentBubble->IsHovered())
	{
		// Hide the comment bubble if visibility hasn't changed
		CommentBubble->SetCommentBubbleVisibility(/*bVisible =*/false);
	}
}
