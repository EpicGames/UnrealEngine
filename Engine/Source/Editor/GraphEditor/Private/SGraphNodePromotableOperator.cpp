// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphNodePromotableOperator.h"

#include "Containers/Set.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditorSettings.h"
#include "K2Node_PromotableOperator.h"
#include "KismetNodes/SGraphNodeK2Sequence.h"
#include "SGraphPin.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Images/SLayeredImage.h"

class SWidget;


#define LOCTEXT_NAMESPACE "SGraphNodePromotableOperator"

//////////////////////////////////////////////////////////////////////////
// SGraphNodePromotableOperator

void SGraphNodePromotableOperator::Construct(const FArguments& InArgs, UK2Node_PromotableOperator* InNode)
{
	SGraphNodeK2Sequence::Construct(SGraphNodeK2Sequence::FArguments(), InNode);

	LoadCachedIcons();
}

void SGraphNodePromotableOperator::CreatePinWidgets()
{
	SGraphNodeK2Sequence::CreatePinWidgets();

	TSet<TSharedRef<SWidget>> AllPins;
	GetPins(AllPins);

	LoadCachedIcons();

	for(TSharedRef<SWidget>& Widget : AllPins)
	{
		
		if(TSharedPtr<SGraphPin> Pin = StaticCastSharedRef<SGraphPin>(Widget))
		{
			UEdGraphPin* SourcePin = Pin->GetPinObj();
			
			// Split pins should be drawn as normal pins, the inner properties are not promotable
			if(!SourcePin || SourcePin->ParentPin)
			{
				continue;
			}
			
			if(TSharedPtr<SLayeredImage> PinImage = StaticCastSharedPtr<SLayeredImage>(Pin->GetPinImageWidget()))
			{
				// Set the image to use the outer icon, which will be the connect pin type color
				PinImage->SetLayerBrush(0, CachedOuterIcon);

				// Set the inner image to be wildcard color, which is grey by default
				PinImage->AddLayer(CachedInnerIcon, GetDefault<UGraphEditorSettings>()->WildcardPinTypeColor);
			}
		}
	}
}

void SGraphNodePromotableOperator::LoadCachedIcons()
{
	static const FName PromotableTypeOuterName("Kismet.VariableList.PromotableTypeOuterIcon");
	static const FName PromotableTypeInnerName("Kismet.VariableList.PromotableTypeInnerIcon");

	// Outer ring icons
	if(!CachedOuterIcon)
	{
		CachedOuterIcon = FAppStyle::GetBrush(PromotableTypeOuterName);
	}

	if(!CachedInnerIcon)
	{
		CachedInnerIcon = FAppStyle::GetBrush(PromotableTypeInnerName);
	}
}

#undef LOCTEXT_NAMESPACE
