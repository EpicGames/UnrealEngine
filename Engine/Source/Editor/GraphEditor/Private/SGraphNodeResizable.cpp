// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphNodeResizable.h"

#include "EdGraph/EdGraphNode.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/ICursor.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Geometry.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Attribute.h"
#include "SNodePanel.h"
#include "ScopedTransaction.h"
#include "SGraphPanel.h"
#include "Widgets/SWindow.h"

namespace GraphNodeResizableDefs
{
	/** Size of the hit result border for the window borders */
	static const FSlateRect HitResultBorderSize( 10, 10, 10, 10 );

	/** Default Title Bar Size */
	static const float DefaultTitleBarHeight = 12.f;

	/** Minimum size for node */
	static const FDeprecateSlateVector2D MinNodeSize( 30.0f, 30.0f );

	/** Maximum size for node */
	static const FDeprecateSlateVector2D MaxNodeSize( 400.0f, 400.0f );
}

bool SGraphNodeResizable::InSelectionArea(EResizableWindowZone InMouseZone) const
{
	return (	(InMouseZone == CRWZ_RightBorder) || (InMouseZone == CRWZ_BottomBorder) || (InMouseZone == CRWZ_BottomRightBorder) || 
				(InMouseZone == CRWZ_LeftBorder) || (InMouseZone == CRWZ_TopBorder) || (InMouseZone == CRWZ_TopLeftBorder) ||
				(InMouseZone == CRWZ_TopRightBorder) || (InMouseZone == CRWZ_BottomLeftBorder) );
}

void SGraphNodeResizable::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{	
	// Determine the zone the mouse is in
	if( !bUserIsDragging )
	{
		FVector2f LocalMouseCoordinates = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
		MouseZone = FindMouseZone(LocalMouseCoordinates);
		SNodePanel::SNode::OnMouseEnter( MyGeometry, MouseEvent );
	}
}

void SGraphNodeResizable::OnMouseLeave( const FPointerEvent& MouseEvent ) 
{
	if( !bUserIsDragging )
	{
		// Reset our mouse zone
		MouseZone = CRWZ_NotInWindow;
		SNodePanel::SNode::OnMouseLeave( MouseEvent );
	}
}

FCursorReply SGraphNodeResizable::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if (MouseZone == CRWZ_RightBorder || MouseZone == CRWZ_LeftBorder)
	{
		// right/left of node
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}
	else if (MouseZone == CRWZ_BottomRightBorder || MouseZone == CRWZ_TopLeftBorder)
	{
		// bottom right / top left hand corner
		return FCursorReply::Cursor( EMouseCursor::ResizeSouthEast );
	}
	else if (MouseZone == CRWZ_BottomBorder || MouseZone == CRWZ_TopBorder)
	{
		// bottom / top of node
		return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
	}
	else if (MouseZone == CRWZ_BottomLeftBorder || MouseZone == CRWZ_TopRightBorder)
	{
		// bottom left / top right hand corner
		return FCursorReply::Cursor( EMouseCursor::ResizeSouthWest );
	}
	else if (MouseZone == CRWZ_TitleBar) 
	{
		return FCursorReply::Cursor(EMouseCursor::CardinalCross);
	}

	return FCursorReply::Unhandled();
}

FReply SGraphNodeResizable::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( InSelectionArea() && (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && IsEditable.Get() ) 
	{
		bUserIsDragging = true;
		StoredUserSize = UserSize;
		DragSize = UserSize;
		//Find node anchor point
		InitNodeAnchorPoint();

		return FReply::Handled().CaptureMouse( SharedThis(this) );
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SGraphNodeResizable::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && bUserIsDragging )
	{
		bUserIsDragging = false;

		// Resize the node	
		UserSize.X = FMath::RoundToFloat(UserSize.X);
		UserSize.Y = FMath::RoundToFloat(UserSize.Y);

		GetNodeObj()->ResizeNode(UserSize);
		Invalidate(EInvalidateWidgetReason::Prepass);

		// End resize transaction
		ResizeTransactionPtr.Reset();

		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}


FReply SGraphNodeResizable::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bUserIsDragging)
	{
		FVector2f GraphSpaceCoordinates = NodeCoordToGraphCoord( MouseEvent.GetScreenSpacePosition() );
		FVector2f OldGraphSpaceCoordinates = NodeCoordToGraphCoord( MouseEvent.GetLastScreenSpacePosition() );
		TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		FVector2f Delta = (GraphSpaceCoordinates - OldGraphSpaceCoordinates) / (OwnerWindow.IsValid() ? OwnerWindow->GetDPIScaleFactor() : 1.0f);
		

		//Clamp delta value based on resizing direction
		if( MouseZone == CRWZ_LeftBorder || MouseZone == CRWZ_RightBorder )
		{
			Delta.Y = 0.0f;
		}
		else if( MouseZone == CRWZ_TopBorder || MouseZone == CRWZ_BottomBorder )
		{
			Delta.X = 0.0f;
		}

		//Resize node delta value
		FVector2f DeltaNodeSize = Delta;

		//Modify node size delta value based on resizing direction
		if(	(MouseZone == CRWZ_LeftBorder) || (MouseZone == CRWZ_TopBorder) || (MouseZone == CRWZ_TopLeftBorder) )
		{
			DeltaNodeSize = -DeltaNodeSize;
		}
		else if( MouseZone == CRWZ_TopRightBorder )
		{
			DeltaNodeSize.Y = -DeltaNodeSize.Y;
		}
		else if( MouseZone == CRWZ_BottomLeftBorder )
		{
			DeltaNodeSize.X = -DeltaNodeSize.X;
		}
		// Apply delta unfiltered to DragSize
		DragSize.X += DeltaNodeSize.X;
		DragSize.Y += DeltaNodeSize.Y;
		// apply snap
		const float SnapSize = static_cast<float>(SNodePanel::GetSnapGridSize());
		FVector2f SnappedSize;
		SnappedSize.X = SnapSize * FMath::RoundToFloat( DragSize.X/SnapSize );
		SnappedSize.Y = SnapSize * FMath::RoundToFloat( DragSize.Y/SnapSize );

		// Enforce min/max sizing
		const FVector2f MinSize = GetNodeMinimumSize2f();
		SnappedSize.X = FMath::Max( SnappedSize.X, MinSize.X );
		SnappedSize.Y = FMath::Max( SnappedSize.Y, MinSize.Y );

		const FVector2f MaxSize = GetNodeMaximumSize2f();
		SnappedSize.X = FMath::Min( SnappedSize.X, MaxSize.X );
		SnappedSize.Y = FMath::Min( SnappedSize.Y, MaxSize.Y );

		FVector2f DeltaNodePos(0.0f,0.0f);

		if( UserSize != SnappedSize )
		{
			//Modify node position (resizing top and left sides)
			if( MouseZone != CRWZ_BottomBorder && MouseZone != CRWZ_RightBorder && MouseZone != CRWZ_BottomRightBorder )
			{
				//Delta value to move graph node position
				DeltaNodePos = UserSize - SnappedSize;

				//Clamp position delta based on resizing direction
				if( MouseZone == CRWZ_BottomLeftBorder )
				{
					DeltaNodePos.Y = 0.0f;
				}
				else if( MouseZone == CRWZ_TopRightBorder )
				{
					DeltaNodePos.X = 0.0f;
				}
			}
			UserSize = SnappedSize;
			GraphNode->ResizeNode( UserSize );
			DeltaNodePos = GetCorrectedNodePosition() - GetPosition2f();
			Invalidate(EInvalidateWidgetReason::Prepass);
		}

		if (!ResizeTransactionPtr.IsValid() && UserSize != StoredUserSize)
		{
			// Start resize transaction.  The transaction is started here so all MoveTo actions are captured while empty
			//	transactions are not created
			ResizeTransactionPtr = MakeShareable(new FScopedTransaction(NSLOCTEXT("GraphEditor", "ResizeNodeAction", "Resize Node")));
		}

		SGraphNode::FNodeSet NodeFilter;
		SGraphNode::MoveTo( GetPosition2f() + DeltaNodePos, NodeFilter );
	}
	else
	{
		const FVector2f LocalMouseCoordinates = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
		MouseZone = FindMouseZone(LocalMouseCoordinates);
	}
	return SGraphNode::OnMouseMove( MyGeometry, MouseEvent );
}

void SGraphNodeResizable::InitNodeAnchorPoint()
{
	NodeAnchorPoint = GetPosition2f();
	
	if( (MouseZone == CRWZ_LeftBorder) || (MouseZone == CRWZ_TopBorder) || (MouseZone == CRWZ_TopLeftBorder) )
	{
		NodeAnchorPoint += UserSize;
	}
	else if( MouseZone == CRWZ_BottomLeftBorder )
	{
		NodeAnchorPoint.X += UserSize.X;
	}
	else if( MouseZone == CRWZ_TopRightBorder )
	{
		NodeAnchorPoint.Y += UserSize.Y;
	}
}


UE::Slate::FDeprecateVector2DResult SGraphNodeResizable::GetCorrectedNodePosition() const
{
	FVector2f CorrectedPos = NodeAnchorPoint;
	
	if( (MouseZone == CRWZ_LeftBorder) || (MouseZone == CRWZ_TopBorder) || (MouseZone == CRWZ_TopLeftBorder) )
	{
		CorrectedPos -= UserSize;
	}
	else if( MouseZone == CRWZ_BottomLeftBorder )
	{
		CorrectedPos.X -= UserSize.X;
	}
	else if( MouseZone == CRWZ_TopRightBorder )
	{
		CorrectedPos.Y -= UserSize.Y;
	}

	return CorrectedPos;
}
SGraphNodeResizable::EResizableWindowZone SGraphNodeResizable::FindMouseZone(const FVector2f& LocalMouseCoordinates) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FindMouseZone(FVector2D(LocalMouseCoordinates));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

SGraphNodeResizable::EResizableWindowZone SGraphNodeResizable::FindMouseZone(const FVector2D& LocalMouseCoordinates) const
{
	EResizableWindowZone OutMouseZone = CRWZ_NotInWindow;
	FSlateRect HitResultBorderSize = GetHitTestingBorder();
	const FVector2f NodeSize = GetDesiredSize();
	const FVector2f& LocalMouseCoords = UE::Slate::CastToVector2f(LocalMouseCoordinates);
	const float TitleBarHeight = GetTitleBarHeight();

	const float Zoom = GetOwnerPanel()->GetZoomAmount();

	if (Zoom > 0.0f && Zoom < 1.0f)
	{
		// Scale up the hit zones when zoomed out so that they remain accessible.
		const float Scale = 1.0f / Zoom;
		HitResultBorderSize = FSlateRect(HitResultBorderSize.Left * Scale, HitResultBorderSize.Top * Scale, HitResultBorderSize.Right * Scale, HitResultBorderSize.Bottom * Scale);
	}
	
	// Test for hit in location of 'grab' zone
	if (LocalMouseCoords.Y > (NodeSize.Y - HitResultBorderSize.Bottom))
	{
		OutMouseZone = CRWZ_BottomBorder;
	}
	else if (TitleBarHeight <= HitResultBorderSize.Top * 1.5f)
	{
		// Use the largest of the two hit areas.
		const float LargestZone = FMath::Max(TitleBarHeight, HitResultBorderSize.Top);
		
		// Allow half of the title bar to be treated as the top edge
		// so that the title bar can still be grabbed
		if (LocalMouseCoords.Y < LargestZone * 0.4)
		{
			OutMouseZone = CRWZ_TopBorder;
		}
		else if (LocalMouseCoords.Y < LargestZone)
		{
			OutMouseZone = CRWZ_TitleBar;
		}
	}
	else if (LocalMouseCoords.Y <= (HitResultBorderSize.Top))
	{
		OutMouseZone = CRWZ_TopBorder;
	}
	else if (LocalMouseCoords.Y <= TitleBarHeight)
	{
		OutMouseZone = CRWZ_TitleBar;
	}

	if (LocalMouseCoords.X > (NodeSize.X - HitResultBorderSize.Right))
	{
		if (OutMouseZone == CRWZ_BottomBorder)
		{
			OutMouseZone = CRWZ_BottomRightBorder;
		}
		else if (OutMouseZone == CRWZ_TopBorder)
		{
			OutMouseZone = CRWZ_TopRightBorder;
		}
		else
		{
			OutMouseZone = CRWZ_RightBorder;
		}
	}
	else if (LocalMouseCoords.X <= HitResultBorderSize.Left)
	{
		if (OutMouseZone == CRWZ_TopBorder)
		{
			OutMouseZone = CRWZ_TopLeftBorder;
		}
		else if (OutMouseZone == CRWZ_BottomBorder)
		{
			OutMouseZone = CRWZ_BottomLeftBorder;
		}
		else
		{
			OutMouseZone = CRWZ_LeftBorder;
		}
	}

	// Test for hit on rest of frame
	if (OutMouseZone == CRWZ_NotInWindow)
	{
		if (LocalMouseCoords.Y > HitResultBorderSize.Top)
		{
			OutMouseZone = CRWZ_InWindow;
		}
		else if (LocalMouseCoords.X > HitResultBorderSize.Left)
		{
			OutMouseZone = CRWZ_InWindow;
		}
	}
	return OutMouseZone;
}

float SGraphNodeResizable::GetTitleBarHeight() const
{
	// this can probably just be SGraphNode::GetTitleRect().Height()
	return GraphNodeResizableDefs::DefaultTitleBarHeight;
}

FVector2D SGraphNodeResizable::GetNodeMinimumSize() const
{
	return FVector2D(GraphNodeResizableDefs::MinNodeSize);
}

FVector2f SGraphNodeResizable::GetNodeMinimumSize2f() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return UE::Slate::CastToVector2f(GetNodeMinimumSize());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FVector2D SGraphNodeResizable::GetNodeMaximumSize() const
{
	return GraphNodeResizableDefs::MaxNodeSize;
}

FVector2f SGraphNodeResizable::GetNodeMaximumSize2f() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return UE::Slate::CastToVector2f(GetNodeMaximumSize());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FSlateRect SGraphNodeResizable::GetHitTestingBorder() const
{
	return GraphNodeResizableDefs::HitResultBorderSize;
}
