// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphPalette.h"

#include "AssetDiscoveryIndicator.h"
//#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphSchema.h"
#include "EditorWidgetsModule.h"
#include "GraphEditorDragDropAction.h"
#include "HAL/PlatformMath.h"
#include "IDocumentation.h"
#include "Input/DragAndDrop.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "SPinTypeSelector.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

class SWidget;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

void SGraphPaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
{
	check(InCreateData->Action.IsValid());

	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	ActionPtr = InCreateData->Action;

	// Find icons
	const FSlateBrush* IconBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor IconColor = FSlateColor::UseForeground();
	FText IconToolTip = GraphAction->GetTooltipDescription();
	bool bIsReadOnly = false;

	TSharedRef<SWidget> IconWidget = CreateIconWidget( IconToolTip, IconBrush, IconColor );
	TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget(InCreateData, bIsReadOnly );

	// Create the actual widget
	this->ChildSlot
	[
		SNew(SHorizontalBox)
		// Icon slot
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			IconWidget
		]
		// Name slot
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		.Padding(3,0)
		[
			NameSlotWidget
		]
	];
}

FReply SGraphPaletteItem::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
	if (GraphDropOp.IsValid())
	{
		GraphDropOp->DroppedOnAction(ActionPtr.Pin().ToSharedRef());
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SGraphPaletteItem::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
	if (GraphDropOp.IsValid())
	{
		GraphDropOp->SetHoveredAction( ActionPtr.Pin() );
	}
}

void SGraphPaletteItem::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
	if (GraphDropOp.IsValid())
	{
		GraphDropOp->SetHoveredAction( NULL );
	}
}

FReply SGraphPaletteItem::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if( MouseButtonDownDelegate.IsBound() )
	{
		if( MouseButtonDownDelegate.Execute( ActionPtr ) == true )
		{
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

TSharedRef<SWidget> SGraphPaletteItem::CreateIconWidget(const FText& IconToolTip, const FSlateBrush* IconBrush, const FSlateColor& IconColor)
{
	return SNew(SImage)
		.ToolTipText(IconToolTip)
		.Image(IconBrush)
		.ColorAndOpacity(IconColor);
}

TSharedRef<SWidget> SGraphPaletteItem::CreateIconWidget(const FText& IconToolTip, const FSlateBrush* IconBrush, const FSlateColor& IconColor, const FString& DocLink, const FString& DocExcerpt, const FSlateBrush* SecondaryIconBrush, const FSlateColor& SecondaryColor)
{
	return SPinTypeSelector::ConstructPinTypeImage(
		IconBrush, 
		IconColor, 
		SecondaryIconBrush, 
		SecondaryColor, 
		IDocumentation::Get()->CreateToolTip(IconToolTip, NULL, DocLink, DocExcerpt));
}

TSharedRef<SWidget> SGraphPaletteItem::CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnly )
{
	TSharedPtr< SWidget > DisplayWidget;

	// Copy the mouse delegate binding if we want it
	if( InCreateData->bHandleMouseButtonDown )
	{
		MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;
	}

	// If the creation data says read only, then it must be read only
	if(InCreateData->bIsReadOnly)
	{
		bIsReadOnly = true;
	}

	InlineRenameWidget =
		SAssignNew(DisplayWidget, SInlineEditableTextBlock)
		.Text(this, &SGraphPaletteItem::GetDisplayText)
		.HighlightText(InCreateData->HighlightText)
		.ToolTipText(this, &SGraphPaletteItem::GetItemTooltip)
		.OnTextCommitted(this, &SGraphPaletteItem::OnNameTextCommitted)
		.OnVerifyTextChanged(this, &SGraphPaletteItem::OnNameTextVerifyChanged)
		.IsSelected( InCreateData->IsRowSelectedDelegate )
		.IsReadOnly( bIsReadOnly );

	InCreateData->OnRenameRequest->BindSP( InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode );

	return DisplayWidget.ToSharedRef();
}

bool SGraphPaletteItem::OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage)
{
	return true;
}

void SGraphPaletteItem::OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
}

FText SGraphPaletteItem::GetDisplayText() const
{
	return ActionPtr.Pin()->GetMenuDescription();
}

FText SGraphPaletteItem::GetItemTooltip() const
{
	return ActionPtr.Pin()->GetTooltipDescription();
}


//////////////////////////////////////////////////////////////////////////


void SGraphPalette::Construct(const FArguments& InArgs)
{
	// Create the asset discovery indicator
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	TSharedRef<SWidget> AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_Vertical);

	this->ChildSlot
		[
			SNew(SBorder)
			.Padding(2.0f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)

				// Content list
				+SVerticalBox::Slot()
					[
						SNew(SOverlay)

						+SOverlay::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							SAssignNew(GraphActionMenu, SGraphActionMenu)
							.OnActionDragged(this, &SGraphPalette::OnActionDragged)
							.OnCreateWidgetForAction(this, &SGraphPalette::OnCreateWidgetForAction)
							.OnCollectAllActions(this, &SGraphPalette::CollectAllActions)
							.AutoExpandActionMenu(InArgs._AutoExpandActionMenu)
						]

						+SOverlay::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Bottom)
							.Padding(FMargin(24, 0, 24, 0))
							[
								// Asset discovery indicator
								AssetDiscoveryIndicator
							]
					]

			]
		];

	// Register with the Asset Registry to be informed when it is done loading up files.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().AddSP(this, &SGraphPalette::RefreshActionsList, true);
}

TSharedRef<SWidget> SGraphPalette::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return	SNew(SGraphPaletteItem, InCreateData);
}

FReply SGraphPalette::OnActionDragged(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, const FPointerEvent& MouseEvent)
{
	if( InActions.Num() > 0 && InActions[0].IsValid() )
	{
		TSharedPtr<FEdGraphSchemaAction> InAction = InActions[0];

		return FReply::Handled().BeginDragDrop(FGraphSchemaActionDragDropAction::New(InAction));
	}

	return FReply::Unhandled();
}

void SGraphPalette::RefreshActionsList(bool bPreserveExpansion)
{
	GraphActionMenu->RefreshAllActions(bPreserveExpansion);
}
