// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphPanel.h"

#include "AssetRegistry/AssetData.h"
#include "AssetSelection.h"
#include "ConnectionDrawingPolicy.h"
#include "Containers/EnumAsByte.h"
#include "Containers/SparseArray.h"
#include "DiffResults.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "DragAndDrop/GraphNodeDragDropOp.h"
#include "DragAndDrop/LevelDragDropOp.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphNode_Comment.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/World.h"
#include "Framework/Application/IMenu.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GraphEditAction.h"
#include "GraphEditorActions.h"
#include "GraphEditorDragDropAction.h"
#include "GraphEditorSettings.h"
#include "GraphEditorSnapping/NodeSnappingManager.h"
#include "HAL/PlatformCrt.h"
#include "Input/DragAndDrop.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "KismetNodes/KismetNodeInfoContext.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Children.h"
#include "Layout/PaintGeometry.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Layout/WidgetPath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "MarqueeOperation.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "NodeFactory.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateLayoutTransform.h"
#include "SGraphNode.h"
#include "ScopedTransaction.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/EditorStyleSettings.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/WidgetStyle.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "Types/PaintArgs.h"
#include "Types/SlateAttributeMetaData.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/InvalidateWidgetReason.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

DEFINE_LOG_CATEGORY_STATIC(LogGraphPanel, Log, All);

//////////////////////////////////////////////////////////////////////////
// SGraphPanel

SLATE_IMPLEMENT_WIDGET(SGraphPanel)
void SGraphPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ShowGraphStateOverlayAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ShouldDrawBackgroundAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ShouldDrawSurroundingShadowAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, DisplayAsReadOnlyAttribute, EInvalidateWidgetReason::Paint);
}


SGraphPanel::SGraphPanel()
	: ShowGraphStateOverlayAttribute(*this, true)
	, ShouldDrawBackgroundAttribute(*this, true)
	, ShouldDrawSurroundingShadowAttribute(*this, true)
	, DisplayAsReadOnlyAttribute(*this, false)
{
}

void SGraphPanel::Construct( const SGraphPanel::FArguments& InArgs )
{
	SNodePanel::Construct();

	this->OnGetContextMenuFor = InArgs._OnGetContextMenuFor;
	this->GraphObj = InArgs._GraphObj;
	this->DiffResults = InArgs._DiffResults;
	this->FocusedDiffResult = InArgs._FocusedDiffResult;
	this->OnSelectionChangedDelegate = InArgs._OnSelectionChanged;
	SetIsEditable(InArgs._IsEditable);
	SetDisplayAsReadOnly(InArgs._DisplayAsReadOnly);
	this->OnNodeDoubleClicked = InArgs._OnNodeDoubleClicked;
	this->OnDropActors = InArgs._OnDropActors;
	this->OnDropStreamingLevels = InArgs._OnDropStreamingLevels;
	this->OnVerifyTextCommit = InArgs._OnVerifyTextCommit;
	this->OnTextCommitted = InArgs._OnTextCommitted;
	this->OnSpawnNodeByShortcutAtLocation = InArgs._OnSpawnNodeByShortcutAtLocation;
	this->OnUpdateGraphPanel = InArgs._OnUpdateGraphPanel;
	this->OnDisallowedPinConnection = InArgs._OnDisallowedPinConnection;
	this->OnDoubleClicked = InArgs._OnDoubleClicked;
	this->OnClicked = InArgs._OnMouseButtonDown;
	this->OnNodeSingleClicked = InArgs._OnNodeSingleClicked;
	this->bAllowZoom = InArgs._AllowZoom;
	this->bAllowPanning = InArgs._AllowPanning;

	this->bPreservePinPreviewConnection = false;
	this->PinVisibility = SGraphEditor::Pin_Show;

	CachedAllottedGeometryScaledSize = FVector2f(160, 120);
	if (InArgs._InitialZoomToFit)
	{
		ZoomToFit(/*bOnlySelection=*/ false);
		bTeleportInsteadOfScrollingWhenZoomingToFit = true;
	}

	BounceCurve.AddCurve(0.0f, 1.0f);

	FEditorDelegates::BeginPIE.AddRaw( this, &SGraphPanel::OnBeginPIE );
	FEditorDelegates::EndPIE.AddRaw( this, &SGraphPanel::OnEndPIE );

	// Register for notifications
	MyRegisteredGraphChangedDelegate = FOnGraphChanged::FDelegate::CreateSP(this, &SGraphPanel::OnGraphChanged);
	MyRegisteredGraphChangedDelegateHandle = this->GraphObj->AddOnGraphChangedHandler(MyRegisteredGraphChangedDelegate);
	
	SetShowGraphStateOverlay(InArgs._ShowGraphStateOverlay);
	AllowConnectionSlicing = InArgs._AllowConnectionSlicing;
	SetShouldDrawBackground(InArgs._ShouldDrawBackground);
	SetShouldDrawSurroundingShadow(InArgs._ShouldDrawSurroundingShadow);

	SavedMousePosForOnPaintEventLocalSpace = FVector2f::ZeroVector;
	PreviousFrameSavedMousePosForSplineOverlap = FVector2f::ZeroVector;
	PreviewConnectorEndpoint = FVector2f::ZeroVector;

	TimeLeftToInvalidatePerTick = 0.0f;

	bHasCustomPrepass = true;
	bCheckNodeGraphObjValidity = false;

	if(FNodeSnappingManager::IsNodeSnappingEnabled())
	{
		NodeSnappingManager = MakePimpl<FNodeSnappingManager>(StaticCastSharedRef<SGraphPanel>(AsShared()));
	}
}

SGraphPanel::~SGraphPanel()
{
	FEditorDelegates::BeginPIE.RemoveAll( this );
	FEditorDelegates::EndPIE.RemoveAll( this );

	this->GraphObj->RemoveOnGraphChangedHandler(MyRegisteredGraphChangedDelegateHandle);
}

//////////////////////////////////////////////////////////////////////////

int32 SGraphPanel::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	CachedAllottedGeometryScaledSize = AllottedGeometry.GetLocalSize() * AllottedGeometry.Scale;

	//Style used for objects that are the same between revisions
	FWidgetStyle FadedStyle = InWidgetStyle;
	FadedStyle.BlendColorAndOpacityTint(FLinearColor(0.45f,0.45f,0.45f,0.30f));

	// First paint the background
	const UEditorExperimentalSettings& Options = *GetDefault<UEditorExperimentalSettings>();

	if (GetShouldDrawBackground())
	{
		const FSlateBrush* DefaultBackground = FAppStyle::GetBrush(TEXT("Graph.Panel.SolidBackground"));
		const FSlateBrush* CustomBackground = &GetDefault<UEditorStyleSettings>()->GraphBackgroundBrush;
		const FSlateBrush* BackgroundImage = CustomBackground->HasUObject() ? CustomBackground : DefaultBackground;
		PaintBackgroundAsLines(BackgroundImage, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	}

	const float ZoomFactor = AllottedGeometry.Scale * GetZoomAmount();

	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildNodes(AllottedGeometry, ArrangedChildren);

	// Determine some 'global' settings based on current LOD
	const bool bDrawShadowsThisFrame = GetCurrentLOD() > EGraphRenderingLOD::LowestDetail;

	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.

	// Save LayerId for comment boxes to ensure they always appear below nodes & wires
	const int32 CommentNodeShadowLayerId = LayerId++;
	const int32 CommentNodeLayerId = LayerId++;

	const int32 NodeDiffHighlightLayerID = LayerId++;

	// Save a LayerId for wires, which appear below nodes but above comments
	// We will draw them later, along with the arrows which appear above nodes.
	const int32 WireLayerId = LayerId++;

	const int32 NodeShadowsLayerId = LayerId;
	const int32 NodeLayerId = NodeShadowsLayerId + 1;
	int32 MaxLayerId = NodeLayerId;

	const FPaintArgs NewArgs = Args.WithNewParent(this);

	const FVector2f NodeShadowSize = UE::Slate::CastToVector2f(GetDefault<UGraphEditorSettings>()->GetShadowDeltaSize());
	const UEdGraphSchema* Schema = GraphObj->GetSchema();

	
	// If we were provided diff results, organize those by owner
	TMap<UEdGraphNode*, FDiffSingleResult> NodeDiffResults;
	TMap<UEdGraphPin*, FDiffSingleResult> PinDiffResults;
	if (DiffResults.IsValid())
	{
		// diffs with Node1/Pin1 get precedence so set those first
		for (const FDiffSingleResult& Result : *DiffResults)
		{
			if (Result.Pin1)
			{
				PinDiffResults.FindOrAdd(Result.Pin1, Result);

				// when zoomed out, make it easier to see diffed pins by also highlighting the node
				if(ZoomLevel <= 6)
				{
					NodeDiffResults.FindOrAdd(Result.Pin1->GetOwningNode(), Result);
				}
			}
			else if (Result.Node1)
			{
				NodeDiffResults.FindOrAdd(Result.Node1, Result);
			}
		}

		// only diffs with Node2/Pin2 if those nodes don't already have a diff result
		for (const FDiffSingleResult& Result : *DiffResults)
		{
			if (Result.Pin2)
			{
				PinDiffResults.FindOrAdd(Result.Pin2, Result);

				// when zoomed out, make it easier to see diffed pins by also highlighting the node
				if(ZoomLevel <= 6)
				{
					NodeDiffResults.FindOrAdd(Result.Pin2->GetOwningNode(), Result);
				}
			}
			else if (!Result.Pin1 && Result.Node2)
			{
				NodeDiffResults.FindOrAdd(Result.Node2, Result);
			}
		}
	}

	// Draw the child nodes
	{
		// When drawing a marquee, need a preview of what the selection will be.
		const FGraphPanelSelectionSet* SelectionToVisualize = &ObjectPtrDecay(SelectionManager.SelectedNodes);
		decltype(SelectionManager.SelectedNodes) SelectionPreview;
		if ( Marquee.IsValid() )
		{			
			ApplyMarqueeSelection(Marquee, ObjectPtrDecay(SelectionManager.SelectedNodes), SelectionPreview);
			SelectionToVisualize = &ObjectPtrDecay(SelectionPreview);
		}

		// Context for rendering node infos
		FKismetNodeInfoContext Context(GraphObj);

		// Build paint order: non-selected first, selected/moving last.
		TArray<int32> PaintOrder;
		TArray<int32> SelectedIndices;
		PaintOrder.Reserve(ArrangedChildren.Num());
		SelectedIndices.Reserve(ArrangedChildren.Num());
		for (int32 i = 0; i < ArrangedChildren.Num(); ++i)
		{
			if (SelectionToVisualize->Contains(StaticCastSharedRef<SNodePanel::SNode>(ArrangedChildren[i].Widget)->GetObjectBeingDisplayed()))
			{
				SelectedIndices.Add(i);
			}
			else
			{
				PaintOrder.Add(i);
			}
		}
		const int32 FirstSelectedIndex = PaintOrder.Num();
		PaintOrder.Append(SelectedIndices);

		// Non-selected nodes first, selected/moving nodes last.
		// This ensures that any node being dragged or selected always renders on top of all others.
		for (int32 OrderIndex = 0; OrderIndex < PaintOrder.Num(); ++OrderIndex)
		{
			int32 ChildIndex = PaintOrder[OrderIndex];
			const bool bSelected = (OrderIndex >= FirstSelectedIndex);
			FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
			TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(CurWidget.Widget);

			// Examine node to see what layers we should be drawing in
			int32 ShadowLayerId = bSelected ? MaxLayerId : NodeShadowsLayerId;
			int32 ChildLayerId = MaxLayerId;

			// If a comment node, draw in the dedicated comment slots
			{
				UObject* NodeObj = ChildNode->GetObjectBeingDisplayed();
				if (NodeObj && NodeObj->IsA(UEdGraphNode_Comment::StaticClass()))
				{
					ShadowLayerId = CommentNodeShadowLayerId;
					ChildLayerId = CommentNodeLayerId;
				}
			}


			UEdGraphNode* NodeObj = Cast<UEdGraphNode>(ChildNode->GetObjectBeingDisplayed());
			float Alpha = 1.0f;

			// Handle Node renaming once the node is visible
			if( bSelected && ChildNode->IsRenamePending() )
			{
				// Only open a rename when the window has focus
				TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
				if (!OwnerWindow.IsValid() || FSlateApplication::Get().HasFocusedDescendants(OwnerWindow.ToSharedRef()))
				{
					ChildNode->ApplyRename();
				}
			}

			/** if this graph is being diffed, highlight the changes in the graph */
			if(DiffResults.IsValid())
			{
				/** When diffing nodes, color code shadow based on diff result */
				if (NodeDiffResults.Contains(NodeObj))
				{
					const FDiffSingleResult& DiffResult = NodeDiffResults[NodeObj];
					for (const SNode::DiffHighlightInfo& Highlight : ChildNode->GetDiffHighlights(DiffResult))
					{
						FSlateDrawElement::MakeBox(
							OutDrawElements,
							NodeDiffHighlightLayerID,
							CurWidget.Geometry.ToInflatedPaintGeometry(NodeShadowSize),
							Highlight.Brush,
							ESlateDrawEffect::None,
							Highlight.Tint
							);
					}
				}
			}

			/** When diffing, set the backround of the differing pins to their diff colors */
			for (UEdGraphPin* Pin : NodeObj->Pins)
			{
				if (TSharedPtr<SGraphPin> PinWidget = ChildNode->FindWidgetForPin(Pin))
				{
					if (FDiffSingleResult* DiffResult = PinDiffResults.Find(Pin))
					{
						// if the diff result associated with this pin is focused, highlight the pin
						if (DiffResults.IsValid() && FocusedDiffResult.IsSet())
						{
							const int32 Index = FocusedDiffResult.Get();
							if (DiffResults->IsValidIndex(Index))
							{
								const FDiffSingleResult& Focused = (*DiffResults)[Index];
								PinWidget->SetDiffHighlighted(*DiffResult == Focused);
							}
						}

						FLinearColor PinDiffColor = DiffResult->GetDisplayColor();
						PinDiffColor.A = 0.7f;
						PinWidget->SetPinDiffColor(PinDiffColor);
						PinWidget->SetFadeConnections(false);
					}
					else
					{
						PinWidget->SetDiffHighlighted(false);
						PinWidget->SetPinDiffColor(TOptional<FLinearColor>());

						// when zoomed out, fade out pin connections that aren't involved in a diff
						PinWidget->SetFadeConnections(ZoomLevel <= 6 && (!NodeDiffResults.Contains(NodeObj) || NodeDiffResults[NodeObj].Pin1));
					}
				}
			}

			// Draw the node's shadow.
			if (bDrawShadowsThisFrame || bSelected)
			{
				// draw drop shadow on connected pins:
				ChildNode->DrawChildrenNodeShadow(
					*SelectionToVisualize,
					/*FSlateWindowElementList& */OutDrawElements,
					ShadowLayerId,
					CurWidget.Geometry,
					FLinearColor(1.0f, 1.0f, 1.0f, Alpha)
				);
			}

			// Draw the comments and information popups for this node, if it has any.
			{
				const SNodePanel::SNode::FNodeSlot* CommentSlot = ChildNode->GetSlot( ENodeZone::TopCenter );
				float CommentBubbleY = CommentSlot ? -CommentSlot->GetSlotOffset().Y : 0.f;
				Context.bSelected = bSelected;
				TArray<FGraphInformationPopupInfo> Popups;

				{
					ChildNode->GetNodeInfoPopups(&Context, /*out*/ Popups);
				}

				for (int32 PopupIndex = 0; PopupIndex < Popups.Num(); ++PopupIndex)
				{
					FGraphInformationPopupInfo& Popup = Popups[PopupIndex];
					PaintComment(Popup.Message, CurWidget.Geometry, MyCullingRect, OutDrawElements, ChildLayerId, Popup.BackgroundColor, /*inout*/ CommentBubbleY, InWidgetStyle);
				}
			}

			int32 CurWidgetsMaxLayerId;
			{
				/* When dragging off a pin, we want to duck the alpha of some nodes */
				TSharedPtr< SGraphPin > OnlyStartPin = (1 == PreviewConnectorFromPins.Num()) ? PreviewConnectorFromPins[0].FindInGraphPanel(*this) : TSharedPtr< SGraphPin >();
				const bool bNodeIsNotUsableInCurrentContext = Schema->FadeNodeWhenDraggingOffPin(NodeObj, OnlyStartPin.IsValid() ? OnlyStartPin.Get()->GetPinObj() : nullptr);

				const bool bCleanDiff = DiffResults.IsValid() && !NodeDiffResults.Contains(NodeObj);

				FWidgetStyle NodeStyleToUse = InWidgetStyle;
				if (bNodeIsNotUsableInCurrentContext)
				{
					NodeStyleToUse = FadedStyle;
				}
				else if (ZoomLevel <= 6 && bCleanDiff)
				{
					NodeStyleToUse = FadedStyle;
				}
				NodeStyleToUse.BlendColorAndOpacityTint(FLinearColor(1.0f, 1.0f, 1.0f, Alpha));

				// Draw the node.
				CurWidgetsMaxLayerId = CurWidget.Widget->Paint(NewArgs, CurWidget.Geometry, MyCullingRect, OutDrawElements, ChildLayerId, NodeStyleToUse, !DisplayAsReadOnlyAttribute.Get() && ShouldBeEnabled( bParentEnabled ) );
			}

			// Draw the node's overlay, if it has one.
			{
				// Get its size
				const FVector2f WidgetSize = CurWidget.Geometry.Size;

				{
					TArray<FOverlayBrushInfo> OverlayBrushes;
					ChildNode->GetOverlayBrushes(bSelected, WidgetSize, /*out*/ OverlayBrushes);

					for (int32 BrushIndex = 0; BrushIndex < OverlayBrushes.Num(); ++BrushIndex)
					{
						FOverlayBrushInfo& OverlayInfo = OverlayBrushes[BrushIndex];
						const FSlateBrush* OverlayBrush = OverlayInfo.Brush;
						if (OverlayBrush != nullptr)
						{
							FPaintGeometry BouncedGeometry = CurWidget.Geometry.ToPaintGeometry(OverlayBrush->ImageSize, FSlateLayoutTransform(OverlayInfo.OverlayOffset));

							// Handle bouncing during PIE
							const float BounceValue = FMath::Sin(2.0f * PI * BounceCurve.GetLerp());
							BouncedGeometry.DrawPosition += FVector2f(OverlayInfo.AnimationEnvelope * BounceValue * ZoomFactor);

							FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint()* OverlayBrush->GetTint(InWidgetStyle));
							//FinalColorAndOpacity.A = Alpha;

							CurWidgetsMaxLayerId++;
							FSlateDrawElement::MakeBox(
								OutDrawElements,
								CurWidgetsMaxLayerId,
								BouncedGeometry,
								OverlayBrush,
								ESlateDrawEffect::None,
								FinalColorAndOpacity
								);
						}

					}
				}

				{
					TArray<FOverlayWidgetInfo> OverlayWidgets = ChildNode->GetOverlayWidgets(bSelected, WidgetSize);

					for (int32 WidgetIndex = 0; WidgetIndex < OverlayWidgets.Num(); ++WidgetIndex)
					{
						FOverlayWidgetInfo& OverlayInfo = OverlayWidgets[WidgetIndex];
						if (SWidget* Widget = OverlayInfo.Widget.Get())
						{
							FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(*Widget, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidationIfConstructed);
							if (Widget->GetVisibility() == EVisibility::Visible)
							{
								// call SlatePrepass as these widgets are not in the 'normal' child hierarchy
								Widget->SlatePrepass(AllottedGeometry.GetAccumulatedLayoutTransform().GetScale());

								const FGeometry WidgetGeometry = CurWidget.Geometry.MakeChild(Widget->GetDesiredSize(), FSlateLayoutTransform(OverlayInfo.OverlayOffset));

								Widget->Paint(NewArgs, WidgetGeometry, MyCullingRect, OutDrawElements, CurWidgetsMaxLayerId, InWidgetStyle, bParentEnabled);
							}
						}
					}
				}
			}

			MaxLayerId = FMath::Max( MaxLayerId, CurWidgetsMaxLayerId + 1 );
		}
	}

	MaxLayerId += 1;

	// Draw connections between pins 
	if (Children.Num() > 0 )
	{
		ConnectionDrawingPolicy = TUniquePtr<FConnectionDrawingPolicy>{ NodeFactory.IsValid() ?
			NodeFactory->CreateConnectionPolicy(Schema, WireLayerId, MaxLayerId, ZoomFactor, MyCullingRect, OutDrawElements, GraphObj) :
			FNodeFactory::CreateConnectionPolicy(Schema, WireLayerId, MaxLayerId, ZoomFactor, MyCullingRect, OutDrawElements, GraphObj)
		};

		const bool bUseDrawStateCaching = ConnectionDrawingPolicy->UseDrawStateCaching();

		// Even if we're now in 'slice mode', you need to keep holding the modifiers to actually commit the slice,
		// to basically allow letting go of the modifiers to 'abort' your slice early instead of needing to undo it
		const bool bShouldPreviewSliceLine = CurrentSlice.IsSet() && FSlateApplication::Get().GetModifierKeys().IsAltDown();

		// Tell the connection policy about the slice line before we draw any connections so it will do intersection tests
		if (bShouldPreviewSliceLine)
		{
			FMarqueeRect AbsoluteLine = FMarqueeRect(AllottedGeometry.LocalToAbsolute(GraphCoordToPanelCoord(CurrentSlice->Line.StartPoint)));
            AbsoluteLine.UpdateEndPoint(AllottedGeometry.LocalToAbsolute(GraphCoordToPanelCoord(CurrentSlice->Line.EndPoint)));
            ConnectionDrawingPolicy->SetSliceLine(AbsoluteLine);
		}
		else
		{
			TArray<TSharedPtr<SGraphPin>> OverridePins;
			for (const FGraphPinHandle& Handle : PreviewConnectorFromPins)
			{
				TSharedPtr<SGraphPin> Pin = Handle.FindInGraphPanel(*this);
				if (Pin.IsValid() && Pin->GetPinObj())
				{
					OverridePins.Add(Pin);
				}
			}

			ConnectionDrawingPolicy->SetHoveredPins(CurrentHoveredPins, OverridePins, TimeWhenMouseEnteredPin);
			ConnectionDrawingPolicy->SetMarkedPin(MarkedPin);
			
			ConnectionDrawingPolicy->SetAbsoluteMousePosition(AllottedGeometry.LocalToAbsolute(SavedMousePosForOnPaintEventLocalSpace));

			if (IsRelinkingConnection())
			{
				ConnectionDrawingPolicy->SetRelinkConnections(RelinkConnections);
				ConnectionDrawingPolicy->SetSelectedNodes(GetSelectedGraphNodes());
			}
		}

		// Get the set of pins for all children and synthesize geometry for culled out pins so lines can be drawn to them.
		TMap<TSharedRef<SWidget>, FArrangedWidget> PinGeometries;
		TSet< TSharedRef<SWidget> > VisiblePins;
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(Children[ChildIndex]);

			// If this is a culled node, approximate the pin geometry to the corner of the node it is within
			if (IsNodeCulled(ChildNode, AllottedGeometry) || ChildNode->IsHidingPinWidgets())
			{
				TArray< TSharedRef<SWidget> > NodePins;
				ChildNode->GetPins(NodePins);

				const FVector2f NodeLoc = ChildNode->GetPosition2f();
				const FGeometry SynthesizedNodeGeometry(GraphCoordToPanelCoord(NodeLoc) * AllottedGeometry.Scale, AllottedGeometry.AbsolutePosition, FVector2f::ZeroVector, 1.f);

				for (TArray< TSharedRef<SWidget> >::TConstIterator NodePinIterator(NodePins); NodePinIterator; ++NodePinIterator)
				{
					const SGraphPin& PinWidget = static_cast<const SGraphPin&>((*NodePinIterator).Get());
					if (PinWidget.GetPinObj())
					{
						FVector2f PinLoc = NodeLoc + UE::Slate::CastToVector2f(PinWidget.GetNodeOffset());

						const FGeometry SynthesizedPinGeometry(GraphCoordToPanelCoord(PinLoc) * AllottedGeometry.Scale, AllottedGeometry.AbsolutePosition, FVector2f::ZeroVector, 1.f);
						PinGeometries.Add(*NodePinIterator, FArrangedWidget(*NodePinIterator, SynthesizedPinGeometry));
					}
				}

				// Also add synthesized geometries for culled nodes
				ArrangedChildren.AddWidget( FArrangedWidget(ChildNode, SynthesizedNodeGeometry) );
			}
			else
			{
				// if node snapping is enabled only unspapped pins will have their connections 
				// drawn, else this will be all visible pins:
				ChildNode->GetPinsToDraw(VisiblePins);
			}
		}

		// Now get the pin geometry for all visible children and append it to the PinGeometries map
		TMap<TSharedRef<SWidget>, FArrangedWidget> VisiblePinGeometries;
		{
			this->FindChildGeometries(AllottedGeometry, VisiblePins, VisiblePinGeometries);
			PinGeometries.Append(VisiblePinGeometries);
		}

		// Draw preview connections (only connected on one end)
		if (PreviewConnectorFromPins.Num() > 0)
		{
			for (const FGraphPinHandle& Handle : PreviewConnectorFromPins)
			{
				TSharedPtr< SGraphPin > CurrentStartPin = Handle.FindInGraphPanel(*this);
				if (!CurrentStartPin.IsValid() || !CurrentStartPin->GetPinObj())
				{
					continue;
				}
				const FArrangedWidget* PinGeometry = PinGeometries.Find( CurrentStartPin.ToSharedRef() );

				if (PinGeometry != nullptr)
				{
					FVector2f StartPoint;
					FVector2f EndPoint;

					if (CurrentStartPin->GetDirection() == EGPD_Input)
					{
						StartPoint = AllottedGeometry.LocalToAbsolute(PreviewConnectorEndpoint);
						EndPoint = FGeometryHelper::VerticalMiddleLeftOf( PinGeometry->Geometry ) - FVector2f(ConnectionDrawingPolicy->ArrowRadius.X, 0);
					}
					else
					{
						StartPoint = FGeometryHelper::VerticalMiddleRightOf( PinGeometry->Geometry );
						EndPoint = AllottedGeometry.LocalToAbsolute(PreviewConnectorEndpoint);
					}

					ConnectionDrawingPolicy->DrawPreviewConnector(PinGeometry->Geometry, StartPoint, EndPoint, CurrentStartPin.Get()->GetPinObj());
				}

				if (!bUseDrawStateCaching || !bIsDrawStateCached)
				{
					//@TODO: Re-evaluate this incompatible mojo; it's mutating every pin state every frame to accomplish a visual effect
					ConnectionDrawingPolicy->SetIncompatiblePinDrawState(CurrentStartPin, VisiblePins);
				}
			}
			bIsDrawStateCached = true;
		}
		else
		{
			//@TODO: Re-evaluate this incompatible mojo; it's mutating every pin state every frame to accomplish a visual effect
			ConnectionDrawingPolicy->ResetIncompatiblePinDrawState(VisiblePins);
		}

		// Draw all regular connections
		ConnectionDrawingPolicy->Draw(PinGeometries, ArrangedChildren);

		// Pull back data from the drawing policy
		{
			FGraphSplineOverlapResult OverlapData = ConnectionDrawingPolicy->SplineOverlapResult;

			if (OverlapData.IsValid())
			{
				OverlapData.ComputeBestPin();

				// Only allow spline overlaps when there is no node under the cursor (unless it is a comment box)
				const FVector2f PaintAbsoluteSpaceMousePos = AllottedGeometry.LocalToAbsolute(SavedMousePosForOnPaintEventLocalSpace);
				const int32 HoveredNodeIndex = SWidget::FindChildUnderPosition(ArrangedChildren, PaintAbsoluteSpaceMousePos);
				if (HoveredNodeIndex != INDEX_NONE)
				{
					TSharedRef<SGraphNode> HoveredNode = StaticCastSharedRef<SGraphNode>(ArrangedChildren[HoveredNodeIndex].Widget);
					UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(HoveredNode->GetObjectBeingDisplayed());
					if (CommentNode == nullptr)
					{
						// Wasn't a comment node, disallow the spline interaction
						OverlapData = FGraphSplineOverlapResult(OverlapData.GetCloseToSpline());
					}
				}
			}

			// Update the spline hover state
			if (const_cast<SGraphPanel*>(this)->OnSplineHoverStateChanged(OverlapData))
			{
				// if hover state changed, we update the tooltip text based on the connection drawing policy
				const_cast<SGraphPanel*>(this)->SetToolTip(ConnectionDrawingPolicy->GetConnectionToolTip(*this, OverlapData));
			}

			// After drawing all connections, we can draw the slice line on top of them
			if (bShouldPreviewSliceLine && ConnectionDrawingPolicy->DrawSliceLine())
			{
				// Take ownership of the intersected connections to save a copy, since we're about to delete the drawing policy
				const_cast<SGraphPanel*>(this)->CurrentSlice->ConnectionsIntersectingSliceLine = MoveTemp(ConnectionDrawingPolicy->ConnectionsIntersectingSliceLine);
			}
		}
	}
	else
	{
		ConnectionDrawingPolicy.Reset();
	}

	if (GetShouldDrawSurroundingShadow())
	{
		// Draw a shadow overlay around the edges of the graph
		++MaxLayerId;
		PaintSurroundSunkenShadow(FAppStyle::GetBrush(TEXT("Graph.Shadow")), AllottedGeometry, MyCullingRect, OutDrawElements, MaxLayerId);
	}

	if (GetShowGraphStateOverlay())
	{
		const FSlateBrush* BorderBrush = nullptr;
		if ((GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != nullptr))
		{
			// Draw a surrounding indicator when PIE is active, to make it clear that the graph is read-only, etc...
			BorderBrush = FAppStyle::GetBrush(TEXT("Graph.PlayInEditor"));
		}
		else if (!GetIsEditable())
		{
			// Draw a different border when we're not simulating but the graph is read-only
			BorderBrush = FAppStyle::GetBrush(TEXT("Graph.ReadOnlyBorder"));
		}

		if (BorderBrush != nullptr)
		{
			// Actually draw the border
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				MaxLayerId,
				AllottedGeometry.ToPaintGeometry(),
				BorderBrush
				);
		}
	}

	// Draw the marquee selection rectangle
	PaintMarquee(AllottedGeometry, MyCullingRect, OutDrawElements, MaxLayerId);

	// Draw the software cursor
	++MaxLayerId;
	PaintSoftwareCursor(AllottedGeometry, MyCullingRect, OutDrawElements, MaxLayerId);

	return MaxLayerId;
}

bool SGraphPanel::OnSplineHoverStateChanged(const FGraphSplineOverlapResult& NewSplineHoverState)
{
	TSharedPtr<SGraphPin> OldPin1Widget;
	TSharedPtr<SGraphPin> OldPin2Widget;
	PreviousFrameSplineOverlap.GetPinWidgets(*this, OldPin1Widget, OldPin2Widget);

	PreviousFrameSplineOverlap = NewSplineHoverState;

	TSharedPtr<SGraphPin> NewPin1Widget;
	TSharedPtr<SGraphPin> NewPin2Widget;
	PreviousFrameSplineOverlap.GetPinWidgets(*this, NewPin1Widget, NewPin2Widget);

	PreviousFrameSavedMousePosForSplineOverlap = SavedMousePosForOnPaintEventLocalSpace;

	// Handle exiting hovering on the pins
	if (OldPin1Widget.IsValid() && OldPin1Widget != NewPin1Widget && OldPin1Widget != NewPin2Widget)
	{
		OldPin1Widget->OnMouseLeave(LastPointerEvent);
		// reset connection tooltip if hover outside spline
		SetToolTipText(FText());
	}

	if (OldPin2Widget.IsValid() && OldPin2Widget != NewPin1Widget && OldPin2Widget != NewPin2Widget)
	{
		OldPin2Widget->OnMouseLeave(LastPointerEvent);
		// reset connection tooltip if hover outside spline
		SetToolTipText(FText());
	}

	// Handle enter hovering on the pins
	bool bChangedHover = false;
	if (NewPin1Widget.IsValid() && NewPin1Widget != OldPin1Widget && NewPin1Widget != OldPin2Widget)
	{
		NewPin1Widget->OnMouseEnter(LastPointerGeometry, LastPointerEvent);
		bChangedHover = true;
	}

	if (NewPin2Widget.IsValid() && NewPin2Widget != OldPin1Widget && NewPin2Widget != OldPin2Widget)
	{
		NewPin2Widget->OnMouseEnter(LastPointerGeometry, LastPointerEvent);
		bChangedHover = true;
	}

	if (bChangedHover)
	{
		// Get the pin/wire glowing quicker, since it's a direct selection (this time was already set to 'now' as part of entering the pin)
		//@TODO: Source this parameter from the graph rendering settings once it is there (see code in ApplyHoverDeemphasis)
		TimeWhenMouseEnteredPin -= 0.75f;
	}
	return bChangedHover;
}

bool SGraphPanel::SupportsKeyboardFocus() const
{
	return true;
}

void SGraphPanel::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	SNodePanel::OnArrangeChildren(AllottedGeometry, ArrangedChildren);

	FArrangedChildren MyArrangedChildren(ArrangedChildren.GetFilter());
	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
		TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(CurWidget.Widget);

		TArray<FOverlayWidgetInfo> OverlayWidgets = ChildNode->GetOverlayWidgets(false, CurWidget.Geometry.Size);

		for (int32 WidgetIndex = 0; WidgetIndex < OverlayWidgets.Num(); ++WidgetIndex)
		{
			FOverlayWidgetInfo& OverlayInfo = OverlayWidgets[WidgetIndex];

			MyArrangedChildren.AddWidget(AllottedGeometry.MakeChild( OverlayInfo.Widget.ToSharedRef(), CurWidget.Geometry.Position + OverlayInfo.OverlayOffset, OverlayInfo.Widget->GetDesiredSize(), GetZoomAmount() ));
		}
	}

	ArrangedChildren.Append(MyArrangedChildren);
}

bool SGraphPanel::CustomPrepass(float LayoutScaleMultiplier)
{
	if(bCheckNodeGraphObjValidity)
	{
		bCheckNodeGraphObjValidity = false;
		RemoveAllNodesWithInvalidPointers();
	}
	return true; // still run prepass on everything
}

void SGraphPanel::UpdateSelectedNodesPositions(const FVector2f& PositionIncrement)
{
	FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "NudgeNodeAction", "Nudge Node"));
	for (auto& NodeIt : SelectionManager.SelectedNodes)
	{
		TSharedRef<SNode>* pWidget = NodeToWidgetLookup.Find(NodeIt);
		if (pWidget != nullptr)
		{
			SNode& Widget = pWidget->Get();
			SNode::FNodeSet NodeFilter;
			Widget.MoveTo(Widget.GetPosition2f() + PositionIncrement, NodeFilter);
		}
	}
}

FReply SGraphPanel::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if( GetIsEditable() )
	{
		const bool bIsModifierActive = InKeyEvent.IsCommandDown() || InKeyEvent.IsAltDown() || InKeyEvent.IsShiftDown() || InKeyEvent.IsControlDown();
		if (!bIsModifierActive)
		{
			const float SnapGridSize = static_cast<float>(GetSnapGridSize());
			if( InKeyEvent.GetKey() == EKeys::Up || InKeyEvent.GetKey() == EKeys::NumPadEight )
			{
				UpdateSelectedNodesPositions(FVector2f(0.0f,-1.0f * SnapGridSize));
				return FReply::Handled();
			}
			if( InKeyEvent.GetKey() == EKeys::Down || InKeyEvent.GetKey() == EKeys::NumPadTwo )
			{
				UpdateSelectedNodesPositions(FVector2f(0.0f, SnapGridSize));
				return FReply::Handled();
			}
			if( InKeyEvent.GetKey() == EKeys::Right || InKeyEvent.GetKey() == EKeys::NumPadSix )
			{
				UpdateSelectedNodesPositions(FVector2f(SnapGridSize,0.0f));
				return FReply::Handled();
			}
			if( InKeyEvent.GetKey() == EKeys::Left || InKeyEvent.GetKey() == EKeys::NumPadFour )
			{
				UpdateSelectedNodesPositions(FVector2f(-1.0f * SnapGridSize,0.0f));
				return FReply::Handled();
			}
		}
		bool bZoomOutKeyEvent = false;
		bool bZoomInKeyEvent = false;
		// Iterate through all key mappings to generate key event flags
		for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
		{
			EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
			const FInputChord& ZoomOutChord = *FGraphEditorCommands::Get().ZoomOut->GetActiveChord(ChordIndex);
			const FInputChord& ZoomInChord = *FGraphEditorCommands::Get().ZoomIn->GetActiveChord(ChordIndex);
			bZoomOutKeyEvent |= ZoomOutChord.IsValidChord() && InKeyEvent.GetKey() == ZoomOutChord.Key;
			bZoomInKeyEvent |= ZoomInChord.IsValidChord() && InKeyEvent.GetKey() == ZoomInChord.Key;
		}

		if(bZoomOutKeyEvent)
		{
			ChangeZoomLevel(-1, CachedAllottedGeometryScaledSize / 2.f, InKeyEvent.IsControlDown());
			return FReply::Handled();
		}
		if( bZoomInKeyEvent)
		{
			ChangeZoomLevel(+1, CachedAllottedGeometryScaledSize / 2.f, InKeyEvent.IsControlDown());
			return FReply::Handled();
		}

		// If we're only dragging out from a single pin then we can process the node spawn keyboard shortcuts
		// If you're dragging multiple connections then it's probably a Ctrl + Drag to move wires around, which
		// wouldn't make as much sense to do node creation during
		UEdGraphPin* PreviewConnectionPin = PreviewConnectorFromPins.Num() == 1 ? PreviewConnectorFromPins[0].GetPinObj(*this) : nullptr;
		if (OnSpawnNodeByShortcutAtLocation.IsBound() && PreviewConnectionPin)
		{
			// Note: We can't use SavedMousePosForOnPaintEventLocalSpace since it isn't updated while dragging,
			// and so would just be the mouse position of the connection origin. So instead we'll just use the current cursor pos
			FVector2f NewNodePosition = PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(FSlateApplication::Get().GetCursorPos()));
			FInputChord KeyChord = FInputChord(InKeyEvent.GetKey(), EModifierKey::FromBools(InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsShiftDown(), InKeyEvent.IsCommandDown()));

			int32 NodeCountBefore = GraphObj->Nodes.Num();
			FReply SpawnNodeReply = OnSpawnNodeByShortcutAtLocation.Execute(KeyChord, NewNodePosition);
			int32 NodeCountAfter = GraphObj->Nodes.Num();

			// If we spawned a node then we won't call down into super and instead do some extra handling
			int32 NumSpawnedNodes = NodeCountAfter - NodeCountBefore;
			if (NumSpawnedNodes > 0)
			{
				TArrayView<UEdGraphNode* const> SpawnedNodes = MakeArrayView(&GraphObj->Nodes[NodeCountBefore], NumSpawnedNodes);

				// Try to auto-wire the newly spawned node
				// Note: Usually the auto-wiring is handled by a schema action or something like FBlueprintMenuActionItemImpl::AutowireSpawnedNodes,
				// but since we're not going through the regular action menu codepath we'll try to just do it here ourselves
				// with slightly fewer heuristics. Could be good to expose that more publicly/centrally though
				if (NumSpawnedNodes == 1)
				{
					bool bWasAutoWired = false;
					for (const UEdGraphPin* Pin : SpawnedNodes[0]->Pins)
					{
						if (Pin->LinkedTo.Num() > 0)
						{
							bWasAutoWired = true;
							break;
						}
					}

					if (!bWasAutoWired)
					{
						SpawnedNodes[0]->AutowireNewNode(PreviewConnectionPin);
					}
				}

				// The parent SNodePanel won't get a chance to set its LastKeyChordDetected, so we'll
				// clear it out here so this key press is sort of "consumed"
				LastKeyChordDetected = FInputChord();

				// We spawned a new node through hotkey instead of letting go of the mouse,
				// so we should cancel the drag to avoid the mouse up summoning the add node context menu
				FSlateApplication::Get().CancelDragDrop();

				OnStopMakingConnection(/*bForceStop=*/ true);

				// Try to make the newly spawned node's connected pin end up underneath the mouse
				TArrayView<UEdGraphPin*> DraggedFromPins = MakeArrayView(&PreviewConnectionPin, 1);
				AdjustNewlySpawnedNodePositions(SpawnedNodes, DraggedFromPins, NewNodePosition);

				UEdGraphPin* ResumeDraggingFromPin = nullptr;

				// For now we don't let the spawn node shortcut provide an explicit pin that should be
				// used when resuming dragging, but instead we'll just see if we spawned a 'control point only' (reroute) node,
				// and if so then automatically use its appropriate in/out pin to continue the drag connection from
				if (SpawnedNodes.Num() == 1)
				{
					UEdGraphNode* SpawnedNode = SpawnedNodes[0];
					int32 OutPinIndex, InPinIndex;
					if (SpawnedNode && SpawnedNode->ShouldDrawNodeAsControlPointOnly(OutPinIndex, InPinIndex))
					{
						ResumeDraggingFromPin = PreviewConnectionPin->Direction == EGPD_Input ? SpawnedNode->Pins[OutPinIndex] : SpawnedNode->Pins[InPinIndex];
					}
				}

				// If we found a pin then need to start a new drag operation from it
				if (ResumeDraggingFromPin)
				{
					// We need to do this one frame later since node widgets aren't created synchronously.
					// Luckily the 'create widget' timer is scheduled synchronously within the actual spawning above,
					// so this should always run after the new widget exists, though before it's been painted
					static auto ResumeDragDelegate = [](double, float, TSharedRef<SGraphPanel> Panel, FGraphPinHandle DragFromPinHandle) -> EActiveTimerReturnType
					{
						TSharedPtr<SGraphPin> DragFromPinWidget = DragFromPinHandle.FindInGraphPanel(*Panel);
						if (DragFromPinWidget.IsValid())
						{
							FPointerEvent MouseEvent = FPointerEvent(
								Panel->LastPointerEvent.GetUserIndex(),
								Panel->LastPointerEvent.GetPointerIndex(),
								FSlateApplication::Get().GetCursorPos(),
								FSlateApplication::Get().GetLastCursorPos(),
								FSlateApplication::Get().GetPressedMouseButtons(),
								EKeys::LeftMouseButton, /* EffectingButton */
								0.f,                    /* WheelDelta */
								FModifierKeysState()    /* InModifierKeys */
							);

							// This will technically be a frame behind but it shouldn't matter too much for this case
							FGeometry PinGeometry = DragFromPinWidget->GetTickSpaceGeometry();

							// This is far from ideal, but SGraphPin doesn't expose its SpawnPinDragEvent method, and even if we made a public equivalent
							// there's still some extra validation and bookkeeping that we'd probably want to ensure gets run,
							// so emulating a mousedown to keep things to a single code-path might actually be an okay option for now
							// If the pin is editable and able to be dragged from, then it should return a reply that wants to begin an FDragConnection
							FReply ResumeDragReply = DragFromPinWidget->OnPinMouseDown(PinGeometry, MouseEvent);
							if (ResumeDragReply.GetDragDropContent().IsValid())
							{
								// Then to start a drag event outside of a Slate event reply, we'll sneakily pretend an external drag started
								TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(DragFromPinWidget.ToSharedRef());
								if (WidgetWindow)
								{
									FDragDropEvent DragDropEvent(MouseEvent, ResumeDragReply.GetDragDropContent());
									FSlateApplication::Get().ProcessDragEnterEvent(WidgetWindow.ToSharedRef(), DragDropEvent);
								}
							}
						}

						return EActiveTimerReturnType::Stop;
					};

					RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateStatic(ResumeDragDelegate, SharedThis(this), FGraphPinHandle(ResumeDraggingFromPin)));
				}

				return FReply::Handled();
			}
		}
	}

	return SNodePanel::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SGraphPanel::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsControlDown = MouseEvent.IsControlDown();
	const bool bIsAltDown = MouseEvent.IsAltDown();
	
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && (bIsAltDown || bIsControlDown))
	{
		if (bIsAltDown)
		{
			// If we allow slicing, we want to defer break links to OnMouseButtonUp so we can tell if the mouse was dragged to slice or if we want a normal
			if (AllowConnectionSlicing.Get())
			{
				LastClickSplineOverlap = PreviousFrameSplineOverlap;
				// Don't allow starting a slice while hovering a node, or for read-only graphs
				if (!NodeUnderMousePtr.IsValid() && IsGraphEditable() && !DisplayAsReadOnlyAttribute.Get())
				{
					const FVector2f GraphMousePos = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );
					CurrentSlice = FConnectionSlice(GraphMousePos);
					Invalidate(EInvalidateWidgetReason::Paint);
					return FReply::Handled().CaptureMouse(SharedThis(this));
				}
			}
			// If we don't allow slicing, run default pin break behavior (executes on mouse down instead of mouse up)
			else
			{
				UEdGraphPin* Pin1;
				UEdGraphPin* Pin2;
				if (PreviousFrameSplineOverlap.GetPins(*this, Pin1, Pin2))
				{
					const UEdGraphSchema* Schema = GraphObj->GetSchema();
					Schema->BreakSinglePinLink(Pin1, Pin2);
					Invalidate(EInvalidateWidgetReason::Paint);
					return FReply::Handled();
				}
			}
		}
		else if (SGraphPin* BestPinFromHoveredSpline = GetBestPinFromHoveredSpline())
		{
			if(bIsControlDown)
			{
				// try to disconnect just the wire (spline) that is hovered):
				TSharedPtr<SGraphPin> BestPin = PreviousFrameSplineOverlap.GetPin1Handle().FindInGraphPanel(*this);
				TSharedPtr<SGraphPin> OtherPin = PreviousFrameSplineOverlap.GetPin2Handle().FindInGraphPanel(*this);
				if(BestPin && OtherPin)
				{
					if (BestPin.Get() != BestPinFromHoveredSpline)
					{
						std::swap(BestPin, OtherPin);
					}
					
					return OnMouseDownDisconnectClosestPinFromWire(
						BestPin.ToSharedRef(), 
						OtherPin.ToSharedRef());
				}
			}

			// fall back legacy behavior, just allow the spline to handle mouse down:
			Invalidate(EInvalidateWidgetReason::Paint);
			return BestPinFromHoveredSpline->OnPinMouseDown(MyGeometry, MouseEvent);
		}
	}

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FGraphPinHandle Pin1Handle = PreviousFrameSplineOverlap.GetPin1Handle();
		if (Pin1Handle.IsValid())
		{
			TSharedPtr<class SGraphPin> SourcePin = Pin1Handle.FindInGraphPanel(*this);
			if (SourcePin.IsValid())
			{
				const UEdGraphSchema* Schema = GraphObj->GetSchema();
				if (Schema->IsConnectionRelinkingAllowed(SourcePin->GetPinObj()))
				{
					Invalidate(EInvalidateWidgetReason::Paint);
					return SourcePin->OnPinMouseDown(MyGeometry, MouseEvent);
				}
			}
		}
	}

	if (OnClicked.IsBound())
	{
		if (const FReply Reply = OnClicked.Execute(MyGeometry, MouseEvent); Reply.IsEventHandled())
		{
			Invalidate(EInvalidateWidgetReason::Paint);
			return FReply::Handled();
		}
	}
	
	return SNodePanel::OnMouseButtonDown(MyGeometry, MouseEvent);
}

TArray<UEdGraphNode*> SGraphPanel::GetSelectedGraphNodes() const
{
	TArray<UEdGraphNode*> SelectedGraphNodes;
	SelectedGraphNodes.Reserve(SelectionManager.SelectedNodes.Num());

	for (auto NodeIt = SelectionManager.SelectedNodes.CreateConstIterator(); NodeIt; ++NodeIt)
	{
		const TSharedRef<SNode>* SelectedNode = NodeToWidgetLookup.Find(*NodeIt);
		if (SelectedNode)
		{
			UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode->Get().GetObjectBeingDisplayed());
			if (SelectedGraphNode)
			{
				SelectedGraphNodes.Add(SelectedGraphNode);
			}
		}
	}

	return SelectedGraphNodes;
}

FReply SGraphPanel::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsAltDown = MouseEvent.IsAltDown();
	if (CurrentSlice && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bIsAltDown)
	{
		if (bInSlicingMode)
		{
			ensure(HasMouseCapture());
			const FScopedTransaction SliceTransaction(NSLOCTEXT("GraphEditor", "SliceConnectionsAction", "Slice Connections"));
		
			const UEdGraphSchema* Schema = GraphObj->GetSchema();
		
			for (const TPair<FEdGraphPinReference, FEdGraphPinReference>& ConnectionPair : CurrentSlice->ConnectionsIntersectingSliceLine)
			{
				UEdGraphPin* Pin1 = ConnectionPair.Key.Get();
				UEdGraphPin* Pin2 = ConnectionPair.Value.Get();
			
				if (Pin1 && Pin2)
				{
					Schema->BreakSinglePinLink(Pin1, Pin2);
				}
			}

			CurrentSlice.Reset();
            bInSlicingMode = false;
			return FReply::Handled().ReleaseMouseCapture();
		}

		// Intercept alt-left clicking on the hovered spline for targeted break link
		UEdGraphPin* Pin1;
		UEdGraphPin* Pin2;
		if (LastClickSplineOverlap.GetPins(*this, Pin1, Pin2))
		{
			const UEdGraphSchema* Schema = GraphObj->GetSchema();
			Schema->BreakSinglePinLink(Pin1, Pin2);
			CurrentSlice.Reset();
			return FReply::Handled().ReleaseMouseCapture();
		}
	}
	
	if (!NodeUnderMousePtr.IsValid() && !Marquee.IsValid() && (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && (MouseEvent.IsShiftDown()))
	{
		if (SGraphPin* BestPinFromHoveredSpline = GetBestPinFromHoveredSpline())
		{
			Invalidate(EInvalidateWidgetReason::Paint);
			return BestPinFromHoveredSpline->OnMouseButtonUp(MyGeometry, MouseEvent);
		}
	}

	return SNodePanel::OnMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SGraphPanel::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	UEdGraphPin* Pin1;
	UEdGraphPin* Pin2;
	if (PreviousFrameSplineOverlap.GetPins(*this, /*out*/ Pin1, /*out*/ Pin2))
	{
		// Give the schema a chance to do something interesting with a double click on a proper spline (both ends are attached to a pin, i.e., not a preview/drag one)
		const FVector2f DoubleClickPositionInGraphSpace = PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));

		const UEdGraphSchema* Schema = GraphObj->GetSchema();
		Schema->OnPinConnectionDoubleCicked(Pin1, Pin2, DoubleClickPositionInGraphSpace);
	}
	else if (!PreviousFrameSplineOverlap.GetCloseToSpline())
	{
		OnDoubleClicked.ExecuteIfBound();
	}

	return SNodePanel::OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
}

class SGraphPin* SGraphPanel::GetBestPinFromHoveredSpline() const
{
	TSharedPtr<SGraphPin> BestPinWidget = PreviousFrameSplineOverlap.GetBestPinWidget(*this);
	return BestPinWidget.Get();
}

void SGraphPanel::GetAllPins(TSet< TSharedRef<SWidget> >& AllPins)
{
	// Get the set of pins for all children
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(Children[ChildIndex]);
		ChildNode->GetPinsIncludingSnappedChildren(AllPins);
	}
}

void SGraphPanel::AddPinToHoverSet(UEdGraphPin* HoveredPin)
{
	CurrentHoveredPins.Add(HoveredPin);
	TimeWhenMouseEnteredPin = FSlateApplication::Get().GetCurrentTime();

	// About covers the fade in time when highlighting pins or splines.
	TimeLeftToInvalidatePerTick = 1.5f;

	// This handle should always be for this function
	if (!ActiveTimerHandleInvalidatePerTick.IsValid())
	{
		ActiveTimerHandleInvalidatePerTick = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SGraphPanel::InvalidatePerTick));
	}
}

void SGraphPanel::RemovePinFromHoverSet(UEdGraphPin* UnhoveredPin)
{
	CurrentHoveredPins.Remove(UnhoveredPin);
	TimeWhenMouseLeftPin = FSlateApplication::Get().GetCurrentTime();
}

void SGraphPanel::ArrangeChildrenForContextMenuSummon(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	// First pass nodes
	for (int32 ChildIndex = 0; ChildIndex < VisibleChildren.Num(); ++ChildIndex)
	{
		const TSharedRef<SNode>& SomeChild = VisibleChildren[ChildIndex];
		if (!SomeChild->RequiresSecondPassLayout())
		{
			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(SomeChild, SomeChild->GetPosition2f() - GetViewOffset(), SomeChild->GetDesiredSizeForMarquee2f(), GetZoomAmount()));
		}
	}

	// Second pass nodes
	for (int32 ChildIndex = 0; ChildIndex < VisibleChildren.Num(); ++ChildIndex)
	{
		const TSharedRef<SNode>& SomeChild = VisibleChildren[ChildIndex];
		if (SomeChild->RequiresSecondPassLayout())
		{
			SomeChild->PerformSecondPassLayout(NodeToWidgetLookup);
			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(SomeChild, SomeChild->GetPosition2f() - GetViewOffset(), SomeChild->GetDesiredSizeForMarquee2f(), GetZoomAmount()));
		}
	}
}

TSharedPtr<SWidget> SGraphPanel::OnSummonContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<SGraphNode> NodeUnderMouse = GetGraphNodeUnderMouse(MyGeometry, MouseEvent);
	UEdGraphPin* PinUnderCursor = GetPinUnderMouse(MyGeometry, MouseEvent, NodeUnderMouse);
	UEdGraphNode* EdNodeUnderMouse = NodeUnderMouse.IsValid() ? NodeUnderMouse->GetNodeObj() : nullptr;
	TArray<UEdGraphPin*> SourcePins;

	if(!EdNodeUnderMouse && !PinUnderCursor)
	{
		// if there's a wire under the mouse, use its begin and end pins as the source pins:
		UEdGraphPin* Pin1;
		UEdGraphPin* Pin2;
		if (PreviousFrameSplineOverlap.GetPins(*this, Pin1, Pin2))
		{
			SourcePins.Add(Pin1);
			SourcePins.Add(Pin2);
		}
	}

	const FVector2f NodeAddPosition = PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));
	return SummonContextMenu(MouseEvent.GetScreenSpacePosition(), NodeAddPosition, EdNodeUnderMouse, PinUnderCursor, SourcePins);
}


bool SGraphPanel::OnHandleLeftMouseRelease(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<SGraphPin> PreviewConnectionPin = PreviewConnectorFromPins.Num() > 0 ? PreviewConnectorFromPins[0].FindInGraphPanel(*this) : nullptr;
	if (PreviewConnectionPin.IsValid() && GetIsEditable())
	{
		TSet< TSharedRef<SWidget> > AllConnectors;
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			//@FINDME:
			TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(Children[ChildIndex]);
			ChildNode->GetPins(AllConnectors);
		}

		TMap<TSharedRef<SWidget>, FArrangedWidget> PinGeometries;
		this->FindChildGeometries(MyGeometry, AllConnectors, PinGeometries);

		bool bHandledDrop = false;
		TSet<UEdGraphNode*> NodeList;
		for ( TMap<TSharedRef<SWidget>, FArrangedWidget>::TIterator SomePinIt(PinGeometries); !bHandledDrop && SomePinIt; ++SomePinIt )
		{
			FArrangedWidget& PinWidgetGeometry = SomePinIt.Value();
			if( PinWidgetGeometry.Geometry.IsUnderLocation( MouseEvent.GetScreenSpacePosition() ) )
			{
				SGraphPin& TargetPin = static_cast<SGraphPin&>( PinWidgetGeometry.Widget.Get() );
				
				if (PreviewConnectionPin->TryHandlePinConnection(TargetPin))
				{
					// We have to do a second check on PinObjs here since TryHandlePinConnection, may invalidate them.
					UEdGraphPin* PreviewConnectionPinObj = PreviewConnectionPin->GetPinObj();
					UEdGraphPin* TargetPinObj = TargetPin.GetPinObj();
					if (TargetPinObj && PreviewConnectionPinObj)
					{
						NodeList.Add(TargetPinObj->GetOwningNode());
						NodeList.Add(PreviewConnectionPinObj->GetOwningNode());
					}
				}
				bHandledDrop = true;
			}
		}

		// No longer make a connection for a pin; we just connected or failed to connect.
		OnStopMakingConnection(/*bForceStop=*/ true);

		return true;
	}
	else
	{
		return false;
	}
}

FReply SGraphPanel::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	LastPointerEvent = MouseEvent;
	LastPointerGeometry = MyGeometry;

	// Save the mouse position to use in OnPaint for spline hit detection
	SavedMousePosForOnPaintEventLocalSpace = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (Children.Num() > 0 && ConnectionDrawingPolicy)
	{
		UE::Slate::FDeprecateVector2DResult AbsolutePos = GetPersistentState().AllottedGeometry.LocalToAbsolute(SavedMousePosForOnPaintEventLocalSpace);
		if (ConnectionDrawingPolicy->HaveConnectionsGraphicallyChanged(*this, AbsolutePos))
		{
			Invalidate(EInvalidateWidgetReason::Paint);
		}
	}
	// Invalidate the spline results if we moved very far
	const FVector2f MouseDelta = SavedMousePosForOnPaintEventLocalSpace - PreviousFrameSavedMousePosForSplineOverlap;
	const float MouseDeltaLengthSquared = MouseDelta.SizeSquared();
	const bool bCursorInDeadZone = MouseDeltaLengthSquared <= FMath::Square(FSlateApplication::Get().GetDragTriggerDistance());

	if (!bCursorInDeadZone)
	{
		//@TODO: Should we do this or just rely on the next OnPaint?
		// Our frame-latent approximation is going to be totally junk if the mouse is moving quickly
		OnSplineHoverStateChanged(FGraphSplineOverlapResult());
	}

	if(NodeSnappingManager && NodeSnappingManager->IsDragging())
	{
		NodeSnappingManager->UpdateDrag(MyGeometry, MouseEvent);
		Invalidate(EInvalidateWidgetReason::Paint);
		return FReply::Handled();
	}

	// Keep updating the slice line as long as the mouse button is held, regardless of whether still holding the modifier keys
	// Also consume the mouse move since we don't really want other graph behaviour happening while slicing
	if (CurrentSlice)
	{
		Invalidate(EInvalidateWidgetReason::Paint);

		if (this->HasMouseCapture())
		{
			const FVector2f GraphMousePos = PanelCoordToGraphCoord(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));
			CurrentSlice->Line.UpdateEndPoint(GraphMousePos);
			if (!CurrentSlice->Line.GetSize().IsNearlyZero())
			{
				bInSlicingMode = true;
			}
			return FReply::Handled();
		}

		// If we somehow lost mouse capture (eg. window was deactivated) then we should cancel the slice
		CurrentSlice.Reset();
	}

	return SNodePanel::OnMouseMove(MyGeometry, MouseEvent);
}

void SGraphPanel::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FGraphEditorDragDropAction> DragConnectionOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
	if (DragConnectionOp.IsValid())
	{
		DragConnectionOp->SetHoveredGraph( SharedThis(this) );
		PreviewConnectorEndpoint = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
	}
}

void SGraphPanel::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FGraphEditorDragDropAction> Operation = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
	if( Operation.IsValid() )
	{
		Operation->SetHoveredGraph(TSharedPtr<SGraphPanel>(nullptr));
	}
	else
	{
		TSharedPtr<FDecoratedDragDropOp> AssetOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
		if( AssetOp.IsValid()  )
		{
			AssetOp->ResetToDefaultToolTip();
		}
	}
}

FReply SGraphPanel::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return FReply::Unhandled();
	}

	// Handle Read only graphs
	if (!GetIsEditable())
	{
		TSharedPtr<FGraphEditorDragDropAction> GraphDragDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();

		if (GraphDragDropOp.IsValid())
		{
			GraphDragDropOp->SetDropTargetValid(false);
		}
		else
		{
			TSharedPtr<FDecoratedDragDropOp> AssetOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
			if (AssetOp.IsValid())
			{
				FText Tooltip = AssetOp->GetHoverText();
				if (Tooltip.IsEmpty())
				{
					Tooltip = NSLOCTEXT( "GraphPanel", "DragDropOperation", "Graph is Read-Only" );
				}
				AssetOp->SetToolTip(Tooltip, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
			}
		}
		return FReply::Handled();
	}

	if (Operation->IsOfType<FGraphEditorDragDropAction>())
	{
		PreviewConnectorEndpoint = MyGeometry.AbsoluteToLocal( DragDropEvent.GetScreenSpacePosition() );
		return FReply::Handled();
	}
	else if (Operation->IsOfType<FExternalDragOperation>())
	{
		return AssetUtil::CanHandleAssetDrag(DragDropEvent);
	}
	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		if ((GraphObj != nullptr) && (GraphObj->GetSchema() != nullptr))
		{
			TSharedPtr<FAssetDragDropOp> AssetOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
			bool bOkIcon = false;
			FText TooltipText;
			if (AssetOp->HasAssets())
			{
				const TArray<FAssetData>& HoveredAssetData = AssetOp->GetAssets();
				FString TooltipTextString;
				GraphObj->GetSchema()->GetAssetsGraphHoverMessage(HoveredAssetData, GraphObj, /*out*/ TooltipTextString, /*out*/ bOkIcon);
				TooltipText = FText::FromString(TooltipTextString);
			}
			const FSlateBrush* TooltipIcon = bOkIcon ? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")) : FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			AssetOp->SetToolTip(TooltipText, TooltipIcon);
		}
		return FReply::Handled();
	} 
	else
	{
		return FReply::Unhandled();
	}
}

FReply SGraphPanel::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	const FVector2f NodeAddPosition = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( DragDropEvent.GetScreenSpacePosition() ) );

	FSlateApplication::Get().SetKeyboardFocus(AsShared(), EFocusCause::SetDirectly);

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid() || !GetIsEditable())
	{
		return FReply::Unhandled();
	}

	if (Operation->IsOfType<FGraphEditorDragDropAction>())
	{
		check(GraphObj);
		TSharedPtr<FGraphEditorDragDropAction> DragConn = StaticCastSharedPtr<FGraphEditorDragDropAction>(Operation);
		if (DragConn.IsValid() && DragConn->IsSupportedBySchema(GraphObj->GetSchema()))
		{
			return DragConn->DroppedOnPanel(SharedThis(this), DragDropEvent.GetScreenSpacePosition(), NodeAddPosition, *GraphObj);
		}

		return FReply::Unhandled();
	}
	else if (Operation->IsOfType<FActorDragDropGraphEdOp>())
	{
		TSharedPtr<FActorDragDropGraphEdOp> ActorOp = StaticCastSharedPtr<FActorDragDropGraphEdOp>(Operation);
		OnDropActors.ExecuteIfBound(ActorOp->Actors, GraphObj, NodeAddPosition);
		return FReply::Handled();
	}

	else if (Operation->IsOfType<FLevelDragDropOp>())
	{
		TSharedPtr<FLevelDragDropOp> LevelOp = StaticCastSharedPtr<FLevelDragDropOp>(Operation);
		OnDropStreamingLevels.ExecuteIfBound(LevelOp->StreamingLevelsToDrop, GraphObj, NodeAddPosition);
		return FReply::Handled();
	}

	else if (Operation->IsOfType<FGraphNodeDragDropOp>())
	{
		TSharedPtr<FGraphNodeDragDropOp> NodeDropOp = StaticCastSharedPtr<FGraphNodeDragDropOp>(Operation);
		NodeDropOp->OnPerformDropToGraphAtLocation.ExecuteIfBound(NodeDropOp, GraphObj, NodeAddPosition, DragDropEvent.GetScreenSpacePosition());
		return FReply::Handled();
	}
	else
	{
		if ((GraphObj != nullptr) && (GraphObj->GetSchema() != nullptr))
		{
			TArray< FAssetData > DroppedAssetData = AssetUtil::ExtractAssetDataFromDrag( DragDropEvent );

			if ( DroppedAssetData.Num() > 0 )
			{
				GraphObj->GetSchema()->DroppedAssetsOnGraph(DroppedAssetData, NodeAddPosition, GraphObj);
				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	}
}

TSharedPtr<SGraphNode> SGraphPanel::GetGraphNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FArrangedChildren ArrangedNodes(EVisibility::Visible);
	this->ArrangeChildrenForContextMenuSummon(MyGeometry, ArrangedNodes);
	FArrangedGraphNode ArrangedWidget = {};
	if (FNodeSnappingManager* SnapManager = GetSnappingManager())
	{
		ArrangedWidget = SnapManager->GetGraphNodeUnderMouse(MyGeometry, MouseEvent);
	}
	else
	{
		const int32 HoveredNodeIndex = SWidget::FindChildUnderMouse(ArrangedNodes, MouseEvent);
		if (HoveredNodeIndex != INDEX_NONE)
		{
			const FArrangedWidget& HoveredNode = ArrangedNodes[HoveredNodeIndex];
			ArrangedWidget.Geometry = HoveredNode.Geometry;
			ArrangedWidget.Widget = StaticCastSharedRef<SGraphNode>(HoveredNode.Widget);
		}
	}
	
	if (ArrangedWidget.Widget)
	{
		// Selection should switch to this code if it isn't already selected.
		// When multiple nodes are selected, we do nothing, provided that the
		// node for which the context menu is being created is in the selection set.
		TSharedPtr<SGraphNode> GraphSubNode = ArrangedWidget.Widget->GetNodeUnderMouse(ArrangedWidget.Geometry, MouseEvent);
		ArrangedWidget.Widget = GraphSubNode.IsValid() ? GraphSubNode.ToSharedRef() : ArrangedWidget.Widget;
		if (!SelectionManager.IsNodeSelected(ArrangedWidget.Widget->GetObjectBeingDisplayed()))
		{
			SelectionManager.SelectSingleNode(ArrangedWidget.Widget->GetObjectBeingDisplayed());
		}
	}

	return ArrangedWidget.Widget;
}

UEdGraphPin* SGraphPanel::GetPinUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TSharedPtr<SGraphNode> GraphNode) const
{
	UEdGraphPin* PinUnderCursor = nullptr;
	if (GraphNode.IsValid())
	{
		const TSharedPtr<SGraphPin> HoveredPin = GraphNode->GetHoveredPin(GraphNode->GetCachedGeometry(), MouseEvent);
		if (HoveredPin.IsValid())
		{
			PinUnderCursor = HoveredPin->GetPinObj();
		}
	}

	return PinUnderCursor;
}

void SGraphPanel::AdjustNewlySpawnedNodePositions(TArrayView<UEdGraphNode* const> SpawnedNodes, TArrayView<UEdGraphPin*> DraggedFromPins, FVector2f AnchorPosition)
{
	static auto FindFirstLinkedAutoWiredPin = [](TArrayView<UEdGraphNode* const> SpawnedNodes, TArrayView<UEdGraphPin*> DraggedFromPins) -> UEdGraphPin*
	{
		for (UEdGraphPin* DraggedPin : DraggedFromPins)
		{
			for (UEdGraphPin* LinkedToPin : DraggedPin->LinkedTo)
			{
				if (SpawnedNodes.Contains(LinkedToPin->GetOwningNode()))
				{
					return LinkedToPin;
				}
			}
		}

		return nullptr;
	};

	if (UEdGraphPin* DraggedConnectionWasAutoWiredToNewPin = FindFirstLinkedAutoWiredPin(SpawnedNodes, DraggedFromPins))
	{
		MoveNodesToAnchorPinAtGraphPosition(SpawnedNodes, FGraphPinHandle(DraggedConnectionWasAutoWiredToNewPin), AnchorPosition);
	}
}

void SGraphPanel::MoveNodesToAnchorPinAtGraphPosition(TArrayView<UEdGraphNode* const> NodesToMove, FGraphPinHandle PinToAnchor, FVector2f DesiredPinGraphPosition)
{
	struct FAnchorUtils
	{
		static int32 RoundToGrid(int32 Value, int32 GridSize)
		{
			return FMath::RoundToInt(static_cast<float>(Value) / static_cast<float>(GridSize)) * GridSize;
		}

		// The standard SnapToGrid() will floor values, but this will round them instead
		static void SnapToGridRounded(UEdGraphNode* Node, uint32 GridSnapSize)
		{
			Node->NodePosX = RoundToGrid(Node->NodePosX, GridSnapSize);
			Node->NodePosY = RoundToGrid(Node->NodePosY, GridSnapSize);
		}

		static EActiveTimerReturnType AlignPinToPositionDelayed(double, float, TSharedRef<SGraphPanel> Panel, FGraphPinHandle DragFromPinHandle, FVector2f DesiredPinImageCenterGraph, TArray<UEdGraphNode*> SpawnedNodes)
		{
			AlignPinToPosition(Panel, DragFromPinHandle, DesiredPinImageCenterGraph, SpawnedNodes);
			return EActiveTimerReturnType::Stop;
		}

		static void AlignPinToPosition(TSharedRef<SGraphPanel> Panel, FGraphPinHandle DragFromPinHandle, FVector2f DesiredPinImageCenterGraph, TArrayView<UEdGraphNode* const> SpawnedNodes)
		{
			TSharedPtr<SGraphPin> DragFromPinWidget = DragFromPinHandle.FindInGraphPanel(*Panel);
			if (!DragFromPinWidget.IsValid())
			{
				return;
			}

			// Normally, the new node's widgets haven't been painted yet, so can't use GetTickSpaceGeometry(),
			// but we want to avoid them painting in the wrong position for a frame anyway, so will just
			// arrange all nodes synchronously as part of FindChildGeometry so we know where they'd be without actually
			// painting to screen. We could theoretically set the node's visibility to hidden until we've done this,
			// but then would also need the connections and overlays etc. to also be hidden
			// TODO: Theoretically we only need to arrange the single node and its descendants, not the whole graph
			// (like ArrangeChildrenForContextMenuSummon but even more reduced), but if it requires SecondPassLayout
			// then would need to do the whole graph anyway, so might be better to just make both types have the same cost
			// for now now to avoid the extra complexity, given certain schemas purge all nodes on any graph change anyway

			// Note: If this schema return true for `ShouldAlwaysPurgeOnModification` then new nodes might have been created,
			// but this timer will be executed before the SNodePanel's Paint, and thus Tick have been called, which is where
			// it populates the VisibleChildren list. And since ArrangeChildren uses this list to know which nodes to arrange,
			// if we don't force it to be populated here then we won't be able to arrange and find the new node's pin
			if (Panel->VisibleChildren.Num() == 0)
			{
				Panel->PopulateVisibleChildren(Panel->GetTickSpaceGeometry());
			}

			UEdGraphPin* PinObj = DragFromPinWidget->GetPinObj();
			UEdGraphNode* OwningNode = PinObj ? PinObj->GetOwningNode() : nullptr;
			TSharedPtr<SGraphNode> OwningNodeWidget = OwningNode ? Panel->GetNodeWidgetFromGuid(OwningNode->NodeGuid) : nullptr;
			if (OwningNodeWidget)
			{
				bool bNodeNeedsPrepass = OwningNodeWidget->NeedsPrepass();
				bool bNoDesiredSize = OwningNodeWidget->GetDesiredSize().GetMax() <= 0.001f;
				if (bNodeNeedsPrepass || bNoDesiredSize)
				{
					const int32 ChildIndex = Panel->Children.Find(OwningNodeWidget.ToSharedRef());
					const float SelfLayoutScaleMultiplier = Panel->GetPrepassLayoutScaleMultiplier();
					const float ChildLayoutScaleMultiplier = Panel->bHasRelativeLayoutScale
								? SelfLayoutScaleMultiplier * Panel->GetRelativeLayoutScale(ChildIndex, SelfLayoutScaleMultiplier)
								: SelfLayoutScaleMultiplier;

					OwningNodeWidget->MarkPrepassAsDirty();
					OwningNodeWidget->SlatePrepass(ChildLayoutScaleMultiplier);
				}
			}

			TSharedPtr<SWidget> PinImageWidget = DragFromPinWidget->GetPinImageWidget();
			TSet<TSharedRef<SWidget>> WidgetsToFind = { DragFromPinWidget.ToSharedRef() };

			// If this pin had an image, then may as well look for that too since it can be
			// more accurate than the center of the overall pin if it has any text wrapping etc.
			if (PinImageWidget.IsValid())
			{
				WidgetsToFind.Add(PinImageWidget.ToSharedRef());
			}

			// Purposefully not using FindChildGeometry() since that's actually checked,
			// and we don't want to panic if the node doesn't happen to exist yet. This util will both
			// force layout to be computed, and if we had an image widget, give us that widget's geometry too
			TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
			Panel->FindChildGeometries(Panel->GetTickSpaceGeometry(), WidgetsToFind, Result);

			// Check if we found anything
			FArrangedWidget* ArrangedDragFromPinWidget = Result.Find(DragFromPinWidget.ToSharedRef());
			FArrangedWidget* ArrangedPinImageWidget = PinImageWidget.IsValid() ? Result.Find(PinImageWidget.ToSharedRef()) : nullptr;

			// If we couldn't even find the pin widget we dragged from then we can't do much else
			if (!ArrangedDragFromPinWidget)
			{
				return;
			}

			// Default to a similar calculation to Paint(), though we don't have access to the drawing policy so can't add the arrow offset
			FVector2f PinImageCenterAbsolute = DragFromPinWidget->GetDirection() == EGPD_Input ? FGeometryHelper::VerticalMiddleLeftOf(ArrangedDragFromPinWidget->Geometry) : FGeometryHelper::VerticalMiddleRightOf(ArrangedDragFromPinWidget->Geometry);

			// Though if we found the actual pin image widget, then we can use its exact center instead, which will probably be more accurate
			// Either should be close enough given we'll be snapping to the grid afterwards anyway
			if (ArrangedPinImageWidget)
			{
				PinImageCenterAbsolute = ArrangedPinImageWidget->Geometry.GetAbsolutePositionAtCoordinates(FVector2f(0.5f, 0.5f));
			}

			FVector2f PinImageCenterPanel = Panel->GetTickSpaceGeometry().AbsoluteToLocal(PinImageCenterAbsolute);
			FVector2f PinImageCenterGraph = Panel->PanelCoordToGraphCoord(PinImageCenterPanel);
			FVector2f Delta = DesiredPinImageCenterGraph - PinImageCenterGraph;

			// Offset all nodes that were spawned by this same delta so that their relative
			// positioning is maintained (re-snapping them each individually though)
			for (UEdGraphNode* SpawnedNode : SpawnedNodes)
			{
				// Extra safety in case this was called from within a timer
				if (IsValid(SpawnedNode))
				{
					SpawnedNode->SetPosition(SpawnedNode->GetPosition() + Delta);

					// Note: Not using the standard SnapToGrid() on purpose since we actually want to be as close
					// to the user's dragged location as possible, and flooring ends up with noticeably more error
					// when you eg. drop just a few px above the previous node
					SnapToGridRounded(SpawnedNode, Panel->GetSnapGridSize());
				}
			}
		}
	};

	if (NodesToMove.Num() == 0)
	{
		return;
	}

	// If the widget already exists then we can just align it synchronously,
	// but if it doesn't (as this is usually used for newly spawned nodes before
	// their deferred construction has run) then we'll try again next frame
	TSharedPtr<SGraphPin> DragFromPinWidget = PinToAnchor.FindInGraphPanel(*this);
	if (DragFromPinWidget.IsValid())
	{
		FAnchorUtils::AlignPinToPosition(SharedThis(this), PinToAnchor, DesiredPinGraphPosition, NodesToMove);
	}
	else
	{
		TArray<UEdGraphNode*> NodesToMoveCopy;
		NodesToMoveCopy.Append(NodesToMove);
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateStatic(FAnchorUtils::AlignPinToPositionDelayed, SharedThis(this), PinToAnchor, DesiredPinGraphPosition, NodesToMoveCopy));
	}
}

FReply SGraphPanel::OnMouseDownDisconnectClosestPinFromWire(TSharedRef<SGraphPin> ClosestPin, TSharedRef<SGraphPin> ConnectedPin)
{
	TArray<TSharedRef<SGraphPin>> PinArray;
	PinArray.Add(ConnectedPin);
	TSharedPtr<FDragDropOperation> DragEvent = ClosestPin->SpawnPinDragEvent(SharedThis(this), PinArray);
	const UEdGraphSchema* Schema = GraphObj->GetSchema();
	Schema->BreakSinglePinLink(ClosestPin->GetPinObj(), ConnectedPin->GetPinObj());
	ClosestPin->bIsMovingLinks = true;
	Invalidate(EInvalidateWidgetReason::Paint);
	return FReply::Handled().BeginDragDrop(DragEvent.ToSharedRef());
}

void SGraphPanel::OnBeginMakingConnection(UEdGraphPin* InOriginatingPin)
{
	OnBeginMakingConnection(FGraphPinHandle(InOriginatingPin));
}

void SGraphPanel::OnBeginMakingConnection(FGraphPinHandle PinHandle)
{
	if (PinHandle.IsValid())
	{
		DismissContextMenu();

		PreviewConnectorFromPins.Add(PinHandle);
		PreviewConnectorEndpoint = SavedMousePosForOnPaintEventLocalSpace;
		bIsDrawStateCached = false;
	}
}

void SGraphPanel::OnStopMakingConnection(bool bForceStop)
{
	if (bForceStop || !bPreservePinPreviewConnection)
	{
		PreviewConnectorFromPins.Reset();
		bPreservePinPreviewConnection = false;
		bIsDrawStateCached = false;
	}
}

void SGraphPanel::OnBeginRelinkConnection(const FGraphPinHandle& InSourcePinHandle, const FGraphPinHandle& InTargetPinHandle)
{
	RelinkConnections.Add({ InSourcePinHandle.GetPinObj(*this), InTargetPinHandle.GetPinObj(*this) });
	OnBeginMakingConnection(InSourcePinHandle);
}

void SGraphPanel::OnEndRelinkConnection(bool bForceStop)
{
	OnStopMakingConnection(bForceStop);
	RelinkConnections.Empty();
}

bool SGraphPanel::IsRelinkingConnection() const
{
	return (RelinkConnections.IsEmpty() == false);
}

void SGraphPanel::PreservePinPreviewUntilForced()
{
	bPreservePinPreviewConnection = true;
}

/** Add a slot to the CanvasPanel dynamically */
void SGraphPanel::AddGraphNode( const TSharedRef<SNodePanel::SNode>& NodeToAdd )
{
	TSharedRef<SGraphNode> GraphNode = StaticCastSharedRef<SGraphNode>(NodeToAdd);
	GraphNode->SetOwner( SharedThis(this) );

	const UEdGraphNode* Node = GraphNode->GetNodeObj();
	if (Node)
	{
		NodeGuidMap.Add(Node->NodeGuid, GraphNode);
	}

	SNodePanel::AddGraphNode(NodeToAdd);
}

void SGraphPanel::RemoveAllNodes()
{
	NodeGuidMap.Empty();
	CurrentHoveredPins.Empty();
	ConnectionDrawingPolicy.Reset();

	for (int32 Iter = 0; Iter != Children.Num(); ++Iter)
	{
		GetChild(Iter)->InvalidateGraphData();
	}

	SNodePanel::RemoveAllNodes();
}

TSharedPtr<SWidget> SGraphPanel::SummonContextMenu(const UE::Slate::FDeprecateVector2DParameter& WhereToSummon, const UE::Slate::FDeprecateVector2DParameter& WhereToAddNode, UEdGraphNode* ForNode, UEdGraphPin* ForPin, const TArray<UEdGraphPin*>& DragFromPins)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SGraphPanel::SummonContextMenu);

	if (OnGetContextMenuFor.IsBound())
	{
		FGraphContextMenuArguments SpawnInfo;
		SpawnInfo.NodeAddPosition = WhereToAddNode;
		SpawnInfo.GraphNode = ForNode;
		SpawnInfo.GraphPin = ForPin;
		SpawnInfo.DragFromPins = DragFromPins;

		FActionMenuContent FocusedContent = OnGetContextMenuFor.Execute(SpawnInfo);

		TSharedRef<SWidget> MenuContent = FocusedContent.Content;
		
		TSharedPtr<IMenu> Menu = FSlateApplication::Get().PushMenu(
			AsShared(),
			FWidgetPath(),
			MenuContent,
			WhereToSummon,
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
			);

		if (Menu.IsValid() && Menu->GetOwnedWindow().IsValid() && FocusedContent.WidgetToFocus.IsValid())
		{
			Menu->GetOwnedWindow()->SetWidgetToFocusOnActivate(FocusedContent.WidgetToFocus);
		}

		if (Menu.IsValid())
		{
			Menu->GetOnMenuDismissed().AddLambda([DelegateList=FocusedContent.OnMenuDismissed](TSharedRef<IMenu>) { DelegateList.Broadcast(); });
		}
		else
		{
			FocusedContent.OnMenuDismissed.Broadcast();
		}

		ContextMenu = Menu;

		return FocusedContent.WidgetToFocus;
	}

	return TSharedPtr<SWidget>();
}

void SGraphPanel::SummonCreateNodeMenuFromUICommand(uint32 NumNodesAdded)
{
	FVector2f WhereToSummonMenu = LastPointerEvent.GetScreenSpacePosition();
	const float AdditionalOffset = 1.0f + (static_cast<float>(NumNodesAdded) * GetDefault<UGraphEditorSettings>()->PaddingAutoCollateIncrement);
	FVector2f WhereToAddNode = PastePosition + AdditionalOffset;

	TSharedPtr<SGraphNode> NodeUnderMouse = GetGraphNodeUnderMouse(LastPointerGeometry, LastPointerEvent);

	// Do not open the context menu on top of non-empty graph area
	if (NodeUnderMouse.IsValid())
	{
		return;
	}
	
	TArray<UEdGraphPin*> DragFromPins;
	TSharedPtr<SWidget> CreateNodeMenuWidget = SummonContextMenu(WhereToSummonMenu, WhereToAddNode, nullptr, nullptr, DragFromPins);

	if (CreateNodeMenuWidget.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(CreateNodeMenuWidget);
		return;
	}
}

void SGraphPanel::DismissContextMenu()
{
	if (TSharedPtr<IMenu> ContextMenuPinned = ContextMenu.Pin())
	{
		ContextMenuPinned->Dismiss();
	}

	ContextMenu.Reset();
}

void SGraphPanel::AttachGraphEvents(TSharedPtr<SGraphNode> CreatedSubNode)
{
	check(CreatedSubNode.IsValid());
	CreatedSubNode->SetIsEditable(GetIsEditableAttribute().ToAttribute());
	CreatedSubNode->SetDoubleClickEvent(OnNodeDoubleClicked);
	CreatedSubNode->SetVerifyTextCommitEvent(OnVerifyTextCommit);
	CreatedSubNode->SetTextCommittedEvent(OnTextCommitted);
}

bool SGraphPanel::GetBoundsForNode(const UObject* InNode, FVector2D& MinCorner, FVector2D& MaxCorner, float Padding) const
{
	FVector2f TempMinCorner, TempMaxCorner;
	bool bRetVal = SNodePanel::GetBoundsForNode(InNode, TempMinCorner, TempMaxCorner, Padding);
	MinCorner = FVector2D(TempMinCorner);
	MaxCorner = FVector2D(TempMaxCorner);

	return bRetVal;
}
bool SGraphPanel::GetBoundsForNode(const UObject* InNode, FVector2f& MinCorner, FVector2f& MaxCorner, float Padding) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVector2D TempMinCorner, TempMaxCorner;
	bool bRetVal = GetBoundsForNode(InNode, TempMinCorner, TempMaxCorner, Padding);
	MinCorner = UE::Slate::CastToVector2f(TempMinCorner);
	MaxCorner = UE::Slate::CastToVector2f(TempMaxCorner);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return bRetVal;
}

class FConnectionAligner
{
public:
	void DefineConnection(UEdGraphNode* SourceNode, const TSharedPtr<SGraphPin>& SourcePin, UEdGraphNode* DestinationNode, const TSharedPtr<SGraphPin>& DestinationPin)
	{
		auto& Dependencies = Connections.FindOrAdd(SourceNode);
		if (SourcePin->GetPinObj()->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			Dependencies.Outputs.FindOrAdd(DestinationNode).Add(FPinPair{ SourcePin, DestinationPin });
		}
		else
		{
			Dependencies.Inputs.FindOrAdd(DestinationNode).Add(FPinPair{ SourcePin, DestinationPin });
		}
	}

	/** Align all the connections */
	void Process()
	{
		struct FRankedNode
		{
			UEdGraphNode* Node;
			uint32 Rank;
		};

		TArray<FRankedNode> RankedNodes;
		RankedNodes.Reserve(Connections.Num());

		TMap<UEdGraphNode*, uint32> LongestChainCache;
		LongestChainCache.Reserve(Connections.Num());

		for (auto& Pair : Connections)
		{
			RankedNodes.Add(FRankedNode{ Pair.Key, CalculateNodeRank(Pair.Key, LongestChainCache) });
		}

		// Sort the nodes based on dependencies - highest is processed first
		RankedNodes.Sort([](const FRankedNode& A, const FRankedNode& B){
			return A.Rank > B.Rank;
		});

		TSet<UEdGraphNode*> VisitedNodes;
		for (FRankedNode& RankedNode : RankedNodes)
		{
			StraightenConnectionsForNode(RankedNode.Node, VisitedNodes, EEdGraphPinDirection::EGPD_Output);
			if (VisitedNodes.Num() == RankedNodes.Num())
			{
				return;
			}

			StraightenConnectionsForNode(RankedNode.Node, VisitedNodes, EEdGraphPinDirection::EGPD_Input);
			if (VisitedNodes.Num() == RankedNodes.Num())
			{
				return;
			}
		}
	}

private:

	void StraightenConnectionsForNode(UEdGraphNode* Node, TSet<UEdGraphNode*>& VisitedNodes, EEdGraphPinDirection Direction)
	{
		FDependencyInfo* Info = Connections.Find(Node);
		if (!Info)
		{
			return;
		}

		for (auto& NodeToPins : Info->GetDirection(Direction))
		{
			if (NodeToPins.Value.Num() == 0 || VisitedNodes.Contains(NodeToPins.Key))
			{
				continue;
			}

			// Align the averages of all the pins
			float AlignmentDelta = 0.f;
			for (const FPinPair& Pins : NodeToPins.Value)
			{
				AlignmentDelta += (Node->GetNodePosY() + Pins.SrcPin->GetNodeOffset().Y) - (NodeToPins.Key->GetNodePosY() + Pins.DstPin->GetNodeOffset().Y);
			}

			UEdGraph* GraphObj = NodeToPins.Key->GetGraph();

			check(GraphObj);

			const UEdGraphSchema* Schema = GraphObj->GetSchema();

			float NewNodePosY = NodeToPins.Key->GetNodePosY() + (AlignmentDelta / static_cast<float>(NodeToPins.Value.Num()));
			Schema->SetNodePosition(NodeToPins.Key, FVector2f(NodeToPins.Key->GetNodePosX(), NewNodePosY));
	
			VisitedNodes.Add(Node);
			VisitedNodes.Add(NodeToPins.Key);
			
			StraightenConnectionsForNode(NodeToPins.Key, VisitedNodes, Direction);
		}
	}

	/** Find the longest chain of single-connection nodes connected to the specified node */
	uint32 FindLongestUniqueChain(UEdGraphNode* Node, TMap<UEdGraphNode*, uint32>& LongestChainCache, EEdGraphPinDirection Direction)
	{
		if (uint32* Length = LongestChainCache.Find(Node))
		{
			// Already set, or circular dependency - ignore
			return *Length;
		}

		// Prevent reentrancy
		LongestChainCache.Add(Node, 0);

		uint32 ThisLength = 0;
		
		if (FDependencyInfo* Dependencies = Connections.Find(Node))
		{
			auto& ConnectedNodes = Dependencies->GetDirection(Direction);

			// We only follow unique (1-1) connections
			if (ConnectedNodes.Num() == 1)
			{
				for (auto& NodeToPins : ConnectedNodes)
				{
					ThisLength = FindLongestUniqueChain(NodeToPins.Key, LongestChainCache, Direction) + 1;
				}
			}
		}

		LongestChainCache[Node] = ThisLength;
		return ThisLength;
	};

	/** Calculate the depth of dependencies for the specified node */
	uint32 CalculateNodeRank(UEdGraphNode* Node, TMap<UEdGraphNode*, uint32>& LongestChainCache)
	{
		uint32 Rank = 0;
		if (FDependencyInfo* PinMap = Connections.Find(Node))
		{
			for (auto& NodeToPins : PinMap->Outputs)
			{
				Rank += FindLongestUniqueChain(NodeToPins.Key, LongestChainCache, EEdGraphPinDirection::EGPD_Output) + 1;
			}
			for (auto& NodeToPins : PinMap->Inputs)
			{
				Rank += FindLongestUniqueChain(NodeToPins.Key, LongestChainCache, EEdGraphPinDirection::EGPD_Input) + 1;
			}
		}
		return Rank;
	}

private:

	/** A pair of pins */
	struct FPinPair
	{
		TSharedPtr<SGraphPin> SrcPin, DstPin;
	};

	/** Map of nodes and pins that are connected to the owning pin */
	struct FDependencyInfo
	{
		TMap<UEdGraphNode*, TArray<FPinPair>> Outputs;
		TMap<UEdGraphNode*, TArray<FPinPair>> Inputs;
		uint32 Rank;

		TMap<UEdGraphNode*, TArray<FPinPair>>& GetDirection(EEdGraphPinDirection Direction)
		{
			return Direction == EEdGraphPinDirection::EGPD_Output ? Outputs : Inputs;
		}
	};
	typedef TMap<UEdGraphNode*, FDependencyInfo> FConnections;

	FConnections Connections;
};

void SGraphPanel::StraightenConnections()
{
	bool bHasAlignedNodes = false;
	
	FConnectionAligner Aligner;
	for (auto& It : SelectionManager.SelectedNodes)
	{
		UEdGraphNode* SourceNode = Cast<UEdGraphNode>(It);
		if (!SourceNode)
		{
			continue;
		}

		TSharedRef<SNode>* ThisNodePtr = NodeToWidgetLookup.Find(SourceNode);
		if (!ThisNodePtr)
		{
			continue;
		}

		for (UEdGraphPin* SourcePin : SourceNode->Pins)
		{
			for (UEdGraphPin* LinkedTo : SourcePin->LinkedTo)
			{
				UEdGraphNode* DestNode = LinkedTo ? LinkedTo->GetOwningNode() : nullptr;
				if (DestNode && SelectionManager.SelectedNodes.Contains(DestNode))
				{
					TSharedRef<SNode>* DestGraphNodePtr = NodeToWidgetLookup.Find(DestNode);
					if (!DestGraphNodePtr)
					{
						continue;
					}

					TSharedPtr<SGraphPin> PinWidget = StaticCastSharedRef<SGraphNode>(*ThisNodePtr)->FindWidgetForPin(SourcePin);
					TSharedPtr<SGraphPin> LinkedPinWidget = StaticCastSharedRef<SGraphNode>(*DestGraphNodePtr)->FindWidgetForPin(LinkedTo);
					
					if (PinWidget.IsValid() && LinkedPinWidget.IsValid())
					{
						bHasAlignedNodes = true;
						Aligner.DefineConnection(SourceNode, PinWidget, DestNode, LinkedPinWidget);
					}
				}
			}
		}
	}
	
	// If we aren't aligning selected nodes, try to align a hovered Single Pin (non-knot) connected nodes.
	if (!bHasAlignedNodes && CurrentHoveredPins.Num() > 0)
	{
		UEdGraphPin* SourcePin = nullptr;
		for (const FEdGraphPinReference& CurrentHoverPin : CurrentHoveredPins)
		{
			int32 InputPinIndex = INDEX_NONE;
			int32 OutputPinIndex = INDEX_NONE;
			UEdGraphNode* InKnot = CurrentHoverPin.Get()->GetOwningNodeUnchecked();
			bool bIsKnot = (InKnot != nullptr && InKnot->ShouldDrawNodeAsControlPointOnly(InputPinIndex, OutputPinIndex) == true &&
				InputPinIndex >= 0 && OutputPinIndex >= 0);
	
			//only use the actual node pins and not knot pins
			if (!bIsKnot)
			{
				SourcePin = CurrentHoverPin.Get();
			}
		}

		if (SourcePin)
		{
			UEdGraphNode* SourceNode = SourcePin->GetOwningNode();
			if (SourceNode)
			{
				UEdGraphPin* DestPin = (SourcePin->LinkedTo.Num() == 1) ? SourcePin->LinkedTo[0] : nullptr;
				UEdGraphNode* DestNode = DestPin ? DestPin->GetOwningNode() : nullptr;
				if (DestPin && DestNode)
				{
					TSharedRef<SNode>* SrcNodePtr = NodeToWidgetLookup.Find(SourceNode);
					TSharedRef<SNode>* DstNodePtr = NodeToWidgetLookup.Find(DestNode);

					if (SrcNodePtr && DstNodePtr)
					{
						TSharedPtr<SGraphPin> PinWidget = StaticCastSharedRef<SGraphNode>(*SrcNodePtr)->FindWidgetForPin(SourcePin);
						TSharedPtr<SGraphPin> LinkedPinWidget = StaticCastSharedRef<SGraphNode>(*DstNodePtr)->FindWidgetForPin(DestPin);
			
						if (PinWidget.IsValid() && LinkedPinWidget.IsValid())
						{
							Aligner.DefineConnection(SourceNode, PinWidget, DestNode, LinkedPinWidget);
						}
					}
				}
			}
		}
	}

	Aligner.Process();
}

void SGraphPanel::StraightenConnections(UEdGraphPin* SourcePin, UEdGraphPin* PinToAlign)
{
	UEdGraphNode* OwningNode = SourcePin->GetOwningNode();

	TSharedRef<SNode>* OwningNodeWidgetPtr = NodeToWidgetLookup.Find(OwningNode);
	if (!OwningNodeWidgetPtr)
	{
		return;
	}

	TSharedRef<SGraphNode> SourceGraphNode = StaticCastSharedRef<SGraphNode>(*OwningNodeWidgetPtr);

	FConnectionAligner Aligner;

	auto AddConnectedPin = [&](UEdGraphPin* ConnectedPin){
		UEdGraphNode* ConnectedNode = ConnectedPin ? ConnectedPin->GetOwningNode() : nullptr;
		if (!ConnectedNode)
		{
			return;
		}

		TSharedRef<SNode>* DestGraphNodePtr = NodeToWidgetLookup.Find(ConnectedNode);
		if (!DestGraphNodePtr)
		{
			return;
		}

		TSharedPtr<SGraphPin> PinWidget = SourceGraphNode->FindWidgetForPin(SourcePin);
		TSharedPtr<SGraphPin> LinkedPinWidget = StaticCastSharedRef<SGraphNode>(*DestGraphNodePtr)->FindWidgetForPin(ConnectedPin);
			
		if (PinWidget.IsValid() && LinkedPinWidget.IsValid())
		{
			Aligner.DefineConnection(OwningNode, PinWidget, ConnectedNode, LinkedPinWidget);
		}
	};

	if (PinToAlign)
	{
		// If we're only aligning a specific pin, do that
		AddConnectedPin(PinToAlign);
	}
	// Else add all the connected pins
	else for (UEdGraphPin* ConnectedPin : SourcePin->LinkedTo)
	{
		AddConnectedPin(ConnectedPin);
	}
	
	Aligner.Process();
}

void SGraphPanel::RefreshNode(UEdGraphNode& Node)
{
	TSharedPtr<SGraphNode> GraphNode = GetNodeWidgetFromGuid(Node.NodeGuid);
	if (GraphNode.IsValid())
	{
		GraphNode->UpdateGraphNode();
	}
}

const TSharedRef<SGraphNode> SGraphPanel::GetChild(int32 ChildIndex)
{
	return StaticCastSharedRef<SGraphNode>(Children[ChildIndex]);
}

void SGraphPanel::AddNode(UEdGraphNode* Node, AddNodeBehavior Behavior)
{
	TSharedPtr<SGraphNode> NewNode;
	if (NodeFactory.IsValid())
	{
		NewNode = NodeFactory->CreateNodeWidget(Node);
	}
	else
	{
		NewNode = FNodeFactory::CreateNodeWidget(Node);
	}

	check(NewNode.IsValid());

	const bool bWasUserAdded = 
		Behavior == WasUserAdded ? true : 
		Behavior == NotUserAdded ? false :
					(UserAddedNodes.Find(Node) != nullptr);

	NewNode->SetIsEditable(GetIsEditableAttribute().ToAttribute());
	NewNode->SetDoubleClickEvent(OnNodeDoubleClicked);
	NewNode->SetVerifyTextCommitEvent(OnVerifyTextCommit);
	NewNode->SetTextCommittedEvent(OnTextCommitted);
	NewNode->SetDisallowedPinConnectionEvent(OnDisallowedPinConnection);

	this->AddGraphNode
	(
		NewNode.ToSharedRef()
	);

	if (bWasUserAdded)
	{
		// Add the node to visible children, this allows focus to occur on sub-widgets for naming purposes.
		VisibleChildren.Add(NewNode.ToSharedRef());

		NewNode->PlaySpawnEffect();
		NewNode->RequestRenameOnSpawn();
	}

	// Note: We delay the creation of widgets for new nodes by a frame in `OnGraphChanged()`, using a Slate timer per node that later
	// calls into this method. Slate timers are executed from within the Paint event, but before the actual OnPaint is called. This means we've
	// just inserted the new node widget after this panel has already pre-passed the existing node widgets, and because it's now a child of the panel
	// it'll also be painted this frame, despite not having been pre-passed (meaning it'll be stuck with a desired size of zero).
	// Because the new node widget(s) get painted with zero size, pin connection wires are then be drawn based on the layout of these zero-sized node(s),
	// resulting in a pretty obvious one-frame flash whenever you insert new nodes. It's particularly visible when using undo/redo,
	// since a lot of nodes can be inserted at once. To avoid this flash of 'painting without pre-pass', we'll just manually pre-pass
	// the new widget here so that when we go to paint it after this function returns it'll at least have some sizing information when we arrange it in our OnPaint().
	// This is safe since graph widgets don't rely on any outer layout information for their metrics, and we don't size ourselves based on node widgets either.
	// We also need to take a bit of care to pass through the same layout scale multiplier as Prepass_ChildLoop() would have so that the zoom level
	// scale is used, otherwise you'd still get a single frame of jitter while the graph is zoomed out.
	const int32 ChildIndex = Children.Num() - 1;
	const float SelfLayoutScaleMultiplier = GetPrepassLayoutScaleMultiplier();
	const float ChildLayoutScaleMultiplier = bHasRelativeLayoutScale
		? SelfLayoutScaleMultiplier * GetRelativeLayoutScale(ChildIndex, SelfLayoutScaleMultiplier)
		: SelfLayoutScaleMultiplier;

	NewNode->MarkPrepassAsDirty();
	NewNode->SlatePrepass(ChildLayoutScaleMultiplier);
}

void SGraphPanel::RemoveNode(const UEdGraphNode* Node)
{
	if(NodeSnappingManager)
	{
		NodeSnappingManager->RemoveNode(Node);
	}

	for (int32 Iter = 0; Iter != Children.Num(); ++Iter)
	{
		TSharedRef<SGraphNode> Child = GetChild(Iter);
		if (Child->GetNodeObj() == Node)
		{
			Child->InvalidateGraphData();
			Children.RemoveAt(Iter);
			break;
		}
	}
	for (int32 Iter = 0; Iter != VisibleChildren.Num(); ++Iter)
	{
		TSharedRef<SGraphNode> Child = StaticCastSharedRef<SGraphNode>(VisibleChildren[Iter]);
		if (Child->GetNodeObj() == Node)
		{
			VisibleChildren.RemoveAt(Iter);
			break;
		}
	}

	NodeToWidgetLookup.Remove(Node);
}

void SGraphPanel::RemoveAllNodesWithInvalidPointers()
{
	if (GraphObj == nullptr)
	{
		return;
	}
	
	TArray<TSharedRef<SGraphNode>> NodesWithInvalidPointers;

	for (int32 Iter = 0; Iter != Children.Num(); ++Iter)
	{
		TSharedRef<SGraphNode> Child = GetChild(Iter);
		if (const UEdGraphNode* NodeObj = Child->GetNodeObj())
		{
			if (!GraphObj->Nodes.Contains(NodeObj))
			{
				NodesWithInvalidPointers.Add(Child);
			}
		}
	}

	for (const TSharedRef<SGraphNode>& NodeWithInvalidPoint : NodesWithInvalidPointers)
	{
		if (const UEdGraphNode* NodeObj = NodeWithInvalidPoint->GetNodeObj())
		{
			RemoveNode(NodeObj);
		}
	}
}

void SGraphPanel::OnSelectionChanged(const TSet<UObject*>& Objects)
{
	SNodePanel::OnSelectionChanged(Objects);
	OnSelectionChangedDelegate.ExecuteIfBound(Objects);
}

TSharedPtr<SGraphNode> SGraphPanel::GetNodeWidgetFromGuid(FGuid Guid) const
{
	return NodeGuidMap.FindRef(Guid).Pin();
}

void SGraphPanel::Update()
{
	static bool bIsUpdating = false;
	if (bIsUpdating)
	{
		return;
	}
	TGuardValue<bool> ReentrancyGuard(bIsUpdating, true);

	// Add widgets for all the nodes that don't have one.
	if (GraphObj != nullptr)
	{
		// Scan for all missing nodes
		for (int32 NodeIndex = 0; NodeIndex < GraphObj->Nodes.Num(); ++NodeIndex)
		{
			UEdGraphNode* Node = GraphObj->Nodes[NodeIndex];
			if (Node)
			{
				// Helps detect cases of UE-26998 without causing a crash. Prevents the node from being rendered altogether and provides info on the state of the graph vs the node.
				// Because the editor won't crash, a GLEO can be expected if the node's outer is in the transient package.
				if (ensureMsgf(Node->GetOuter() == GraphObj, TEXT("Found %s ('%s') that does not belong to %s. Node Outer: %s, Node Outer Type: %s, Graph Outer: %s, Graph Outer Type: %s"),
					*Node->GetName(), *Node->GetClass()->GetName(),
					*GraphObj->GetName(),
					*Node->GetOuter()->GetName(), *Node->GetOuter()->GetClass()->GetName(),
					*GraphObj->GetOuter()->GetName(), *GraphObj->GetOuter()->GetClass()->GetName()
					))
 				{
					AddNode(Node, CheckUserAddedNodesList);
				}
				else
				{
					UE_LOGF(LogGraphPanel, Error, "Found %ls ('%ls') that does not belong to %ls. Node Outer: %ls, Node Outer Type: %ls, Graph Outer: %ls, Graph Outer Type: %ls",
						*Node->GetName(), *Node->GetClass()->GetName(),
						*GraphObj->GetName(),
						*Node->GetOuter()->GetName(), *Node->GetOuter()->GetClass()->GetName(),
						*GraphObj->GetOuter()->GetName(), *GraphObj->GetOuter()->GetClass()->GetName()
					);
				}
			}
			else
			{
				UE_LOGF(LogGraphPanel, Warning, "Found NULL Node in GraphObj array of a graph in asset '%ls'. A node type has been deleted without creating an ActiveClassRedirector to K2Node_DeadClass.", *GraphObj->GetOutermost()->GetName());
			}
		}

		// check the last selection action, and execute it
		if (!UserSelectedNodes.IsEmpty())
		{
			DeferredSelectionTargetObjects.Empty();
			for (TWeakObjectPtr<UEdGraphNode>& NodePtr : UserSelectedNodes)
			{
				if (NodePtr.IsValid())
				{
					UEdGraphNode* Node = NodePtr.Get();
					DeferredSelectionTargetObjects.Add(Node);
				}
			}
		}
	}
	else
	{
		RemoveAllNodes();
	}

	if(NodeSnappingManager)
	{
		NodeSnappingManager->HandleGraphChanged();
	}

	// Clean out set of added nodes
	UserAddedNodes.Reset();
	UserSelectedNodes.Reset();

	// Invoke any delegate methods
	OnUpdateGraphPanel.ExecuteIfBound();

	// Clear the update pending flag to allow deferred zoom commands to run.
	bVisualUpdatePending = false;
}

// Purges the existing visual representation (typically followed by an Update call in the next tick)
void SGraphPanel::PurgeVisualRepresentation()
{
	// No need to call OnSplineHoverStateChanged since we're about to destroy all the nodes and pins
	PreviousFrameSplineOverlap = FGraphSplineOverlapResult();

	// Clear all of the nodes and pins
	RemoveAllNodes();

	// Set a flag to know that an update is pending to prevent running pending commands like zoom to fit until widgets are generated.
	bVisualUpdatePending = true;
}

bool SGraphPanel::IsNodeTitleVisible(const class UEdGraphNode* Node, bool bRequestRename)
{
	bool bTitleVisible = false;
	TSharedRef<SNode>* pWidget = NodeToWidgetLookup.Find(Node);

	if (pWidget != nullptr)
	{
		TWeakPtr<SGraphNode> GraphNode = StaticCastSharedRef<SGraphNode>(*pWidget);
		if(GraphNode.IsValid() && !HasMouseCapture())
		{
			FSlateRect TitleRect = GraphNode.Pin()->GetTitleRect();
			const FVector2f TopLeft = FVector2f( TitleRect.Left, TitleRect.Top );
			const FVector2f BottomRight = FVector2f( TitleRect.Right, TitleRect.Bottom );

			if( IsRectVisible( TopLeft, BottomRight ))
			{
				bTitleVisible = true;
			}
			else if( bRequestRename )
			{
				bTitleVisible = JumpToRect( TopLeft, BottomRight );
			}

			if( bTitleVisible && bRequestRename )
			{
				GraphNode.Pin()->RequestRename();
				SelectAndCenterObject(Node, false);
			}
		}
	}
	return bTitleVisible;
}

bool SGraphPanel::IsRectVisible(const UE::Slate::FDeprecateVector2DParameter &TopLeft, const UE::Slate::FDeprecateVector2DParameter &BottomRight)
{
	return TopLeft.ComponentwiseAllGreaterOrEqual( PanelCoordToGraphCoord( FVector2f::ZeroVector )) && 
		BottomRight.ComponentwiseAllLessOrEqual( PanelCoordToGraphCoord( CachedAllottedGeometryScaledSize ) );
}

bool SGraphPanel::JumpToRect(const UE::Slate::FDeprecateVector2DParameter &TopLeft, const UE::Slate::FDeprecateVector2DParameter &BottomRight)
{
	ZoomToTarget(TopLeft, BottomRight);

	return true;
}

void SGraphPanel::JumpToNode(const UEdGraphNode* JumpToMe, bool bRequestRename, bool bSelectNode)
{
	if (JumpToMe != nullptr)
	{
		if (bRequestRename)
		{
			TSharedRef<SNode>* pWidget = NodeToWidgetLookup.Find(JumpToMe);
			if (pWidget != nullptr)
			{
				TSharedRef<SGraphNode> GraphNode = StaticCastSharedRef<SGraphNode>(*pWidget);
				GraphNode->RequestRename();
			}
		}

		if (bSelectNode)
		{
			// Select this node, and request that we jump to it.
			SelectAndCenterObject(JumpToMe, true);
		}
		else
		{
			// Jump to the node
			CenterObject(JumpToMe);
		}
	}
}

void SGraphPanel::JumpToPin(const UEdGraphPin* JumpToMe)
{
	if (JumpToMe != nullptr)
	{
		JumpToNode(JumpToMe->GetOwningNode(), false, true);
	}
}

void SGraphPanel::OnBeginPIE( const bool bIsSimulating )
{
	// Play the bounce curve on a continuous loop during PIE
	BounceCurve.Play( this->AsShared(), true );
}

void SGraphPanel::OnEndPIE( const bool bIsSimulating )
{
	// Stop the bounce curve
	BounceCurve.JumpToEnd();
}

void SGraphPanel::OnGraphChanged(const FEdGraphEditAction& EditAction)
{
	const bool bShouldPurge = GraphObj->GetSchema()->ShouldAlwaysPurgeOnModification();
	if (bShouldPurge || EditAction.Action == GRAPHACTION_Default)
	{
		if (!bVisualUpdatePending)
		{
			PurgeVisualRepresentation();

			const auto RefreshPanelDelegateWrapper = [](double, float, TWeakPtr<SGraphPanel> WeakParent) -> EActiveTimerReturnType
			{
				TSharedPtr<SGraphPanel> Parent = WeakParent.Pin();
				if (Parent.IsValid())
				{
					Parent->Update();
				}
				return EActiveTimerReturnType::Stop;
			};

			// Trigger the refresh
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(RefreshPanelDelegateWrapper, StaticCastWeakPtr<SGraphPanel>(AsWeak())));
		}

		if ((EditAction.Graph == GraphObj) &&
			(EditAction.Nodes.Num() > 0) &&
			EditAction.bUserInvoked)
		{
			if (EditAction.Action & GRAPHACTION_AddNode)
			{
				UserAddedNodes.Append(EditAction.Nodes);
			}
			if (EditAction.Action & GRAPHACTION_SelectNode)
			{
				UserSelectedNodes.Reset();
				for (const UEdGraphNode* Node : EditAction.Nodes)
				{
					TWeakObjectPtr<UEdGraphNode> NodePtr = MakeWeakObjectPtr(const_cast<UEdGraphNode*>(Node));
					UserSelectedNodes.Add(NodePtr);
				}
			}
		}
	}
	else if ((EditAction.Graph == GraphObj) && (EditAction.Nodes.Num() > 0) )
	{
		// Ensure that any new non-default action(s) get handled here
		constexpr int32 HandledActionsMask = (GRAPHACTION_AddNode | GRAPHACTION_EditNode | GRAPHACTION_SelectNode | GRAPHACTION_RemoveNode);
		ensureMsgf((EditAction.Action & ~HandledActionsMask) == 0, TEXT("Unhandled actions: %08x"), EditAction.Action & ~HandledActionsMask);

		// Remove action handled immediately by SGraphPanel::OnGraphChanged
		const bool bWasAddAction = (EditAction.Action & GRAPHACTION_AddNode) != 0;
		const bool bWasEditAction = (EditAction.Action & GRAPHACTION_EditNode) != 0;
		const bool bWasSelectAction = (EditAction.Action & GRAPHACTION_SelectNode) != 0;
		const bool bWasRemoveAction = (EditAction.Action & GRAPHACTION_RemoveNode) != 0;

		// The *only* reason we defer these actions is because code higher up the call stack
		// assumes that the node is created later (for example, GenerateBlueprintAPIUtils::AddNodeToGraph
		// calls AddNode (which calls this function) before calling AllocateDefaultPins, so if we create 
		// the widget immediately it won't be able to create its pins). There are lots of other examples, 
		// and I can't be sure that I've found them all.... 
		
		// Minor note, the ugly little lambdas are just to deal with the time values and return values 
		// that the timer system requires (and we don't leverage):
		if (bWasRemoveAction)
		{
			const auto RemoveNodesDelegateWrapper = [](double, float, TWeakPtr<SGraphPanel> WeakParent, TSet< TWeakObjectPtr<UEdGraphNode> > NodePtrs) -> EActiveTimerReturnType
			{
				TSharedPtr<SGraphPanel> Parent = WeakParent.Pin();
				if (Parent.IsValid())
				{
					for (TWeakObjectPtr<UEdGraphNode>& NodePtr : NodePtrs)
					{
						if (NodePtr.IsValid())
						{
							UEdGraphNode* Node = NodePtr.Get();
							Parent->RemoveNode(Node);
						}
					}
				}
				return EActiveTimerReturnType::Stop;
			};

			TSet< TWeakObjectPtr<UEdGraphNode> > NodePtrSet;
			for (const UEdGraphNode* Node : EditAction.Nodes)
			{
				TWeakObjectPtr<UEdGraphNode> NodePtr = MakeWeakObjectPtr(const_cast<UEdGraphNode*>(Node));
				NodePtrSet.Add(NodePtr);
			}

			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(RemoveNodesDelegateWrapper, StaticCastWeakPtr<SGraphPanel>(AsWeak()), NodePtrSet));
			bCheckNodeGraphObjValidity = true;
		}
		if (bWasAddAction)
		{
			const auto AddNodesDelegateWrapper = [](double, float, TWeakPtr<SGraphPanel> WeakParent, TSet< TWeakObjectPtr<UEdGraphNode> > NodePtrs, bool bForceUserAdded) -> EActiveTimerReturnType
			{
				TSharedPtr<SGraphPanel> Parent = WeakParent.Pin();
				if (Parent.IsValid())
				{
					for (TWeakObjectPtr<UEdGraphNode>& NodePtr : NodePtrs)
					{
						if (NodePtr.IsValid())
						{
							UEdGraphNode* Node = NodePtr.Get();
							if (IsValid(Node))
							{
								if (Parent->bVisualUpdatePending)
								{
									if (bForceUserAdded)
									{
										Parent->UserAddedNodes.Add(Node);
									}
								}
								else
								{
									Parent->RemoveNode(Node);
									Parent->AddNode(Node, bForceUserAdded ? WasUserAdded : NotUserAdded);
								}
							}
						}
					}
				}
				return EActiveTimerReturnType::Stop;
			};

			TSet< TWeakObjectPtr<UEdGraphNode> > NodePtrSet;
			for (const UEdGraphNode* Node : EditAction.Nodes)
			{
				TWeakObjectPtr<UEdGraphNode> NodePtr = MakeWeakObjectPtr(const_cast<UEdGraphNode*>(Node));
				NodePtrSet.Add(NodePtr);
			}

			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(AddNodesDelegateWrapper, StaticCastWeakPtr<SGraphPanel>(AsWeak()), NodePtrSet, EditAction.bUserInvoked));
		}
		if (bWasSelectAction)
		{
			const auto SelectNodeDelegateWrapper = [](double, float, TWeakPtr<SGraphPanel> WeakParent, TSet< TWeakObjectPtr<UEdGraphNode> > NodePtrs, bool bForceUserAdded) -> EActiveTimerReturnType
			{
				TSharedPtr<SGraphPanel> Parent = WeakParent.Pin();
				if (Parent.IsValid())
				{
					if (Parent->bVisualUpdatePending)
					{
						if (bForceUserAdded)
						{
							Parent->UserSelectedNodes = NodePtrs;
						}
					}
					else
					{
						Parent->DeferredSelectionTargetObjects.Empty();
						for (TWeakObjectPtr<UEdGraphNode>& NodePtr : NodePtrs)
						{
							if (NodePtr.IsValid())
							{
								UEdGraphNode* Node = NodePtr.Get();
								Parent->DeferredSelectionTargetObjects.Add(Node);
							}
						}
					}
				}
				
				return EActiveTimerReturnType::Stop;
			};

			TSet< TWeakObjectPtr<UEdGraphNode> > NodePtrSet;
			for (const UEdGraphNode* Node : EditAction.Nodes)
			{
				TWeakObjectPtr<UEdGraphNode> NodePtr = MakeWeakObjectPtr(const_cast<UEdGraphNode*>(Node));
				NodePtrSet.Add(NodePtr);
			}

			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(SelectNodeDelegateWrapper, StaticCastWeakPtr<SGraphPanel>(AsWeak()), NodePtrSet, EditAction.bUserInvoked));
		}
		if (bWasEditAction)
		{
			for (const UEdGraphNode* Node : EditAction.Nodes)
			{
				RefreshNode(const_cast<UEdGraphNode&>(*Node));
			}
		}
	}
}

void SGraphPanel::NotifyGraphChanged(const FEdGraphEditAction& EditAction)
{
	// Forward call
	OnGraphChanged(EditAction);
}

void SGraphPanel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( GraphObj );
}

FString SGraphPanel::GetReferencerName() const
{
	return TEXT("SGraphPanel");
}

EActiveTimerReturnType SGraphPanel::InvalidatePerTick(double InCurrentTime, float InDeltaTime)
{
	// Invalidate the layout so it will redraw.
	Invalidate(EInvalidateWidget::Layout);

	TimeLeftToInvalidatePerTick -= InDeltaTime;

	// When the time is done, stop the invalidation per tick because the UI will be static once more.
	if (TimeLeftToInvalidatePerTick <= 0.0f)
	{
		TimeLeftToInvalidatePerTick = 0.0f;
		return EActiveTimerReturnType::Stop;
	}
	return EActiveTimerReturnType::Continue;
}

void SGraphPanel::SetNodeFactory(const TSharedRef<class FGraphNodeFactory>& NewNodeFactory)
{
	NodeFactory = NewNodeFactory;
}

bool SGraphPanel::IsSnappingEnabled()
{
	return FNodeSnappingManager::IsNodeSnappingEnabled();
}
