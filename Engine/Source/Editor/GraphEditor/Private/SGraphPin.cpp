// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphPin.h"

#include "Animation/AnimNodeBase.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragConnection.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "GraphEditorDragDropAction.h"
#include "GraphEditorSettings.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "SLevelOfDetailBranchNode.h"
#include "SNodePanel.h"
#include "SPinTypeSelector.h"
#include "SPinValueInspector.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

class IToolTip;
class UBlueprint;
struct FSlateBrush;

/////////////////////////////////////////////////////
// FGraphPinHandle

FGraphPinHandle::FGraphPinHandle(UEdGraphPin* InPin)
{
	if (InPin != nullptr)
	{
		if (UEdGraphNode* Node = InPin->GetOwningNodeUnchecked())
		{
			NodeGuid = Node->NodeGuid;
			PinId = InPin->PinId;
		}
	}
}

UEdGraphPin* FGraphPinHandle::GetPinObj(const SGraphPanel& Panel) const
{
	UEdGraphPin* AssociatedPin = nullptr;
	if (IsValid())
	{
		TSharedPtr<SGraphNode> NodeWidget = Panel.GetNodeWidgetFromGuid(NodeGuid);
		if (NodeWidget.IsValid())
		{
			UEdGraphNode* NodeObj = NodeWidget->GetNodeObj();
			AssociatedPin = NodeObj->FindPinById(PinId);
		}
	}
	return AssociatedPin;
}

TSharedPtr<SGraphPin> FGraphPinHandle::FindInGraphPanel(const SGraphPanel& InPanel) const
{
	if (UEdGraphPin* ReferencedPin = GetPinObj(InPanel))
	{
		TSharedPtr<SGraphNode> GraphNode = InPanel.GetNodeWidgetFromGuid(NodeGuid);
		return GraphNode->FindWidgetForPin(ReferencedPin);
	}
	return TSharedPtr<SGraphPin>();
}


/////////////////////////////////////////////////////
// SGraphPin


SGraphPin::SGraphPin()
	: PinImageHighDetailSlot(nullptr)
	, GraphPinObj(nullptr)
	, PinColorModifier(FLinearColor::White)
	, CachedNodeOffset(FVector2f::ZeroVector)
	, Custom_Brush_Connected(nullptr)
	, Custom_Brush_Disconnected(nullptr)
	, bGraphDataInvalid(false)
	, bShowLabel(true)
	, bOnlyShowDefaultValue(false)
	, bIsMovingLinks(false)
	, bUsePinColorForText(false)
	, bDragAndDropEnabled(true)
	, bIsDiffHighlighted(false)
	, bFadeConnections(false)
	, bIsSnappingEnabled(false)
{
	IsEditable = true;

	// Make these names const so they're not created for every pin

	/** Original Pin Styles */
	static const FName NAME_Pin_Connected("Graph.Pin.Connected");
	static const FName NAME_Pin_Disconnected("Graph.Pin.Disconnected");

	/** Variant A Pin Styles */
	static const FName NAME_Pin_Connected_VarA("Graph.Pin.Connected_VarA");
	static const FName NAME_Pin_Disconnected_VarA("Graph.Pin.Disconnected_VarA");

	static const FName NAME_ArrayPin_Connected("Graph.ArrayPin.Connected");
	static const FName NAME_ArrayPin_Disconnected("Graph.ArrayPin.Disconnected");

	static const FName NAME_RefPin_Connected("Graph.RefPin.Connected");
	static const FName NAME_RefPin_Disconnected("Graph.RefPin.Disconnected");

	static const FName NAME_DelegatePin_Connected("Graph.DelegatePin.Connected");
	static const FName NAME_DelegatePin_Disconnected("Graph.DelegatePin.Disconnected");

	static const FName NAME_SetPin("Kismet.VariableList.SetTypeIcon");
	static const FName NAME_MapPinKey("Kismet.VariableList.MapKeyTypeIcon");
	static const FName NAME_MapPinValue("Kismet.VariableList.MapValueTypeIcon");

	static const FName NAME_Pin_Background("Graph.Pin.Background");
	static const FName NAME_Pin_BackgroundHovered("Graph.Pin.BackgroundHovered");
	
	static const FName NAME_Pin_DiffOutline("Graph.Pin.DiffHighlight");

	static const FName NAME_PosePin_Connected("Graph.PosePin.Connected");
	static const FName NAME_PosePin_Disconnected("Graph.PosePin.Disconnected");

	const EBlueprintPinStyleType StyleType = GetDefault<UGraphEditorSettings>()->DataPinStyle;

	switch(StyleType)
	{
	case BPST_VariantA:
		CachedImg_Pin_Connected = FAppStyle::GetBrush( NAME_Pin_Connected_VarA );
		CachedImg_Pin_Disconnected = FAppStyle::GetBrush( NAME_Pin_Disconnected_VarA );
		break;
	case BPST_Original:
	default:
		CachedImg_Pin_Connected = FAppStyle::GetBrush( NAME_Pin_Connected );
		CachedImg_Pin_Disconnected = FAppStyle::GetBrush( NAME_Pin_Disconnected );
		break;
	}

	CachedImg_RefPin_Connected = FAppStyle::GetBrush( NAME_RefPin_Connected );
	CachedImg_RefPin_Disconnected = FAppStyle::GetBrush( NAME_RefPin_Disconnected );

	CachedImg_ArrayPin_Connected = FAppStyle::GetBrush( NAME_ArrayPin_Connected );
	CachedImg_ArrayPin_Disconnected = FAppStyle::GetBrush( NAME_ArrayPin_Disconnected );

	CachedImg_DelegatePin_Connected = FAppStyle::GetBrush( NAME_DelegatePin_Connected );
	CachedImg_DelegatePin_Disconnected = FAppStyle::GetBrush( NAME_DelegatePin_Disconnected );

	CachedImg_PosePin_Connected = FAppStyle::GetBrush(NAME_PosePin_Connected);
	CachedImg_PosePin_Disconnected = FAppStyle::GetBrush(NAME_PosePin_Disconnected);

	CachedImg_SetPin = FAppStyle::GetBrush(NAME_SetPin);
	CachedImg_MapPinKey = FAppStyle::GetBrush(NAME_MapPinKey);
	CachedImg_MapPinValue = FAppStyle::GetBrush(NAME_MapPinValue);

	CachedImg_Pin_Background = FAppStyle::GetBrush( NAME_Pin_Background );
	CachedImg_Pin_BackgroundHovered = FAppStyle::GetBrush( NAME_Pin_BackgroundHovered );

	CachedImg_Pin_DiffOutline = FAppStyle::GetBrush( NAME_Pin_DiffOutline );
}

SGraphPin::~SGraphPin()
{
	if (ValueInspectorTooltip.IsValid())
	{
		bool bForceDismiss = true;
		ValueInspectorTooltip.Pin()->TryDismissTooltip(bForceDismiss);
	}
}

void SGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	bUsePinColorForText = InArgs._UsePinColorForText;
	this->SetCursor(EMouseCursor::Default);

	SetVisibility(MakeAttributeSP(this, &SGraphPin::GetPinVisiblity));

	GraphPinObj = InPin;
	check(GraphPinObj != NULL);

	const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
	checkf(
		Schema, 
		TEXT("Missing schema for pin: %s with outer: %s of type %s"), 
		*(GraphPinObj->GetName()),
		GraphPinObj->GetOuter() ? *(GraphPinObj->GetOuter()->GetName()) : TEXT("NULL OUTER"), 
		GraphPinObj->GetOuter() ? *(GraphPinObj->GetOuter()->GetClass()->GetName()) : TEXT("NULL OUTER")
	);

	const bool bIsInput = (GetDirection() == EGPD_Input);

	// Create the pin icon widget
	TSharedRef<SWidget> PinWidgetRef = SPinTypeSelector::ConstructPinTypeImage(
		MakeAttributeSP(this, &SGraphPin::GetPinIcon ),
		MakeAttributeSP(this, &SGraphPin::GetPinColor),
		MakeAttributeSP(this, &SGraphPin::GetSecondaryPinIcon),
		MakeAttributeSP(this, &SGraphPin::GetSecondaryPinColor));
	PinImage = PinWidgetRef;

	PinWidgetRef->SetCursor( 
		TAttribute<TOptional<EMouseCursor::Type> >::Create (
			TAttribute<TOptional<EMouseCursor::Type> >::FGetter::CreateRaw( this, &SGraphPin::GetPinCursor )
		)
	);

	// Create the pin indicator widget (used for watched values)
	static const FName NAME_NoBorder("NoBorder");
	TSharedRef<SWidget> PinStatusIndicator =
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), NAME_NoBorder)
		.Visibility(this, &SGraphPin::GetPinStatusIconVisibility)
		.ContentPadding(0.0f)
		.OnClicked(this, &SGraphPin::ClickedOnPinStatusIcon)
		[
			SNew(SImage)
			.Image(this, &SGraphPin::GetPinStatusIcon)
		];

	TSharedRef<SWidget> LabelWidget = GetLabelWidget(InArgs._PinLabelStyle);

	// Create the widget used for the pin body (status indicator, label, and value)
	LabelAndValue =
		SNew(SWrapBox)
		.PreferredSize(150.f);

	TSharedPtr<SWidget> PinDeprecationIndicator;
	if (InPin->bDeprecated)
	{
		static const FName DeprecationWarning("Icons.WarningWithColor");
		PinDeprecationIndicator = SNew(SImage)
			.Image(FAppStyle::GetBrush(DeprecationWarning))
			.ToolTipText(FText::FromString(InPin->DeprecationMessage));
	}

	if (!bIsInput)
	{
		LabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				PinStatusIndicator
			];

		if (PinDeprecationIndicator)
		{
			LabelAndValue->AddSlot()
				.VAlign(VAlign_Center)
				[
					PinDeprecationIndicator.ToSharedRef()
				];
		}

		LabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				LabelWidget
			];
	}
	else
	{
		if (PinDeprecationIndicator)
		{
			LabelAndValue->AddSlot()
				.VAlign(VAlign_Center)
				[
					PinDeprecationIndicator.ToSharedRef()
				];
		}

		LabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				LabelWidget
			];

		ValueWidget = GetDefaultValueWidget();

		if (ValueWidget != SNullWidget::NullWidget)
		{
			TSharedPtr<SBox> ValueBox;
			LabelAndValue->AddSlot()
				.Padding(bIsInput ? FMargin(InArgs._SideToSideMargin, 0.0f, 0.0f, 0.0f) : FMargin(0.0f, 0.0f, InArgs._SideToSideMargin, 0.0f))
				.VAlign(VAlign_Center)
				[
					SAssignNew(ValueBox, SBox)
					.Padding(0.0f)
					[
						ValueWidget.ToSharedRef()
					]
				];

			if (!DoesWidgetHandleSettingEditingEnabled())
			{
				ValueBox->SetEnabled(TAttribute<bool>(this, &SGraphPin::IsEditingEnabled));
			}
		}

		LabelAndValue->AddSlot()
			.VAlign(VAlign_Center)
			[
				PinStatusIndicator
			];
	}

	TSharedPtr<SHorizontalBox> PinContent;
	if (bIsInput)
	{
		// Input pin
		FullPinHorizontalRowWidget = PinContent = 
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, InArgs._SideToSideMargin, 0.0f)
			.Expose(PinImageHighDetailSlot)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				LabelAndValue.ToSharedRef()
			];
	}
	else
	{
		// Output pin
		FullPinHorizontalRowWidget = PinContent = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				LabelAndValue.ToSharedRef()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(InArgs._SideToSideMargin, 0.0f, 0.0f, 0.0f)
			.Expose(PinImageHighDetailSlot);
	}

	// Set up a hover for pins that is tinted the color of the pin.
	
	SBorder::Construct(SBorder::FArguments()
		.BorderImage(this, &SGraphPin::GetPinBorder)
		.BorderBackgroundColor(this, &SGraphPin::GetHighlightColor)
		.OnMouseButtonDown(this, &SGraphPin::OnPinNameMouseDown)
		[
			SNew(SBorder)
			.BorderImage(CachedImg_Pin_DiffOutline)
			.BorderBackgroundColor(this, &SGraphPin::GetPinDiffColor)
			[
				SAssignNew(PinNameLODBranchNode, SLevelOfDetailBranchNode)
				.UseLowDetailSlot(this, &SGraphPin::UseLowDetailPinNames)
				.OnGetActiveDetailSlotContent(this, &SGraphPin::OnGetActiveDetailSlotContent)
				.HighDetail()
				[
					PinContent.ToSharedRef()
				]
			]
		]
	);

	TSharedPtr<IToolTip> TooltipWidget = SNew(SToolTip)
		.Text(this, &SGraphPin::GetTooltipText);

	SetToolTip(TooltipWidget);

	bIsSnappingEnabled = SGraphPanel::IsSnappingEnabled();
}

TSharedRef<SWidget>	SGraphPin::GetDefaultValueWidget()
{
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SGraphPin::GetLabelWidget(const FName& InLabelStyle)
{
	return SNew(STextBlock)
		.Text(this, &SGraphPin::GetPinLabel)
		.TextStyle(FAppStyle::Get(), InLabelStyle)
		.Visibility(this, &SGraphPin::GetPinLabelVisibility)
		.ColorAndOpacity(this, &SGraphPin::GetPinTextColor);
}

void SGraphPin::RefreshLOD()
{
	if (PinNameLODBranchNode.IsValid())
	{
		PinNameLODBranchNode->RefreshLODSlotContent();
	}
}

void SGraphPin::SetIsEditable(TAttribute<bool> InIsEditable)
{
	IsEditable = MoveTemp(InIsEditable);
}

FReply SGraphPin::OnPinMouseDown( const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent )
{
	bIsMovingLinks = false;

	if(IsSnapped())
	{
		return FReply::Unhandled();
	}

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && ensure(!bGraphDataInvalid))
	{
		if (IsEditingEnabled())
		{
			if (MouseEvent.IsAltDown())
			{
				// Alt-Left clicking will break all existing connections to a pin
				const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
				Schema->BreakPinLinks(*GraphPinObj, true);
				return FReply::Handled();
			}

			TSharedPtr<SGraphNode> OwnerNodePinned = OwnerNodePtr.Pin();
			if ((MouseEvent.IsControlDown() || MouseEvent.IsShiftDown()) && (GraphPinObj->LinkedTo.Num() > 0))
			{
				// Get a reference to the owning panel widget
				check(OwnerNodePinned.IsValid());
				TSharedPtr<SGraphPanel> OwnerPanelPtr = OwnerNodePinned->GetOwnerPanel();
				check(OwnerPanelPtr.IsValid());

				// Obtain the set of all pins within the panel
				TSet<TSharedRef<SWidget> > AllPins;
				OwnerPanelPtr->GetAllPins(AllPins);

				// Construct a UEdGraphPin->SGraphPin mapping for the full pin set
				TMap< FGraphPinHandle, TSharedRef<SGraphPin> > PinToPinWidgetMap;
				for (const TSharedRef<SWidget>& SomePinWidget : AllPins)
				{
					const SGraphPin& PinWidget = static_cast<const SGraphPin&>(SomePinWidget.Get());

					UEdGraphPin* GraphPin = PinWidget.GetPinObj();
					if (GraphPin->LinkedTo.Num() > 0)
					{
						PinToPinWidgetMap.Add(FGraphPinHandle(GraphPin), StaticCastSharedRef<SGraphPin>(SomePinWidget));
					}
				}

				// Define a local struct to temporarily store lookup information for pins that we are currently linked to
				struct FLinkedToPinInfo
				{
					// Pin name string
					FName PinName;

					// A weak reference to the node object that owns the pin
					TWeakObjectPtr<UEdGraphNode> OwnerNodePtr;

					// The direction of the pin
					EEdGraphPinDirection Direction;
				};

				// Build a lookup table containing information about the set of pins that we're currently linked to
				TArray<FLinkedToPinInfo> LinkedToPinInfoArray;
				for (UEdGraphPin* Pin : GetPinObj()->LinkedTo)
				{
					if (TSharedRef<SGraphPin>* PinWidget = PinToPinWidgetMap.Find(Pin))
					{
						check((*PinWidget)->OwnerNodePtr.IsValid());

						FLinkedToPinInfo PinInfo;
						PinInfo.PinName = (*PinWidget)->GetPinObj()->PinName;
						PinInfo.OwnerNodePtr = (*PinWidget)->OwnerNodePtr.Pin()->GetNodeObj();
						PinInfo.Direction = (*PinWidget)->GetPinObj()->Direction;
						LinkedToPinInfoArray.Add(MoveTemp(PinInfo));
					}
				}


				// Now iterate over our lookup table to find the instances of pin widgets that we had previously linked to
				TArray<TSharedRef<SGraphPin>> PinArray;
				for (FLinkedToPinInfo PinInfo : LinkedToPinInfoArray)
				{
					if (UEdGraphNode* OwnerNodeObj = PinInfo.OwnerNodePtr.Get())
					{
						for (UEdGraphPin* Pin : PinInfo.OwnerNodePtr.Get()->Pins)
						{
							if (Pin->PinName == PinInfo.PinName && Pin->Direction == PinInfo.Direction)
							{
								if (TSharedRef<SGraphPin>* pWidget = PinToPinWidgetMap.Find(FGraphPinHandle(Pin)))
								{
									PinArray.Add(*pWidget);
								}
							}
						}
					}
				}

				TSharedPtr<FDragDropOperation> DragEvent;
				if (PinArray.Num() > 0)
				{
					DragEvent = SpawnPinDragEvent(OwnerPanelPtr.ToSharedRef(), PinArray);
				}

				// Control-Left clicking will break all existing connections to a pin
				// Note: that for some nodes, this can cause reconstruction. In that case, pins we had previously linked to may now be destroyed. 
				//       So the break MUST come after the SpawnPinDragEvent(), since that acquires handles from PinArray (the pins need to be
				//       around for us to construct valid handles from).
				if (MouseEvent.IsControlDown())
				{
					const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
					Schema->BreakPinLinks(*GraphPinObj, true);
				}

				if (DragEvent.IsValid())
				{
					bIsMovingLinks = true;
					return FReply::Handled().BeginDragDrop(DragEvent.ToSharedRef());
				}
				else
				{
					// Shouldn't get here, but just in case we lose our previous links somehow after breaking them, we'll just skip the drag.
					return FReply::Handled();
				}
			}

			if (!MouseEvent.IsShiftDown() && !GraphPinObj->bNotConnectable && bDragAndDropEnabled)
			{
				// Start a drag-drop on the pin
				if (ensure(OwnerNodePinned.IsValid()))
				{
					TArray<TSharedRef<SGraphPin>> PinArray;
					PinArray.Add(SharedThis(this));

					if (TSharedPtr<SGraphPanel> OwnerGraphPanel = OwnerNodePinned->GetOwnerPanel())
					{
						return FReply::Handled().BeginDragDrop(SpawnPinDragEvent(OwnerGraphPanel.ToSharedRef(), PinArray));
					}
				}
				else
				{
					return FReply::Unhandled();
				}
			}
		}

		// It's not connectible, but we don't want anything above us to process this left click.
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SGraphPin::OnPinNameMouseDown( const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent )
{
	const float LocalX = SenderGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ).X;

	if ((GetDirection() == EGPD_Input) || FMath::Abs( SenderGeometry.GetLocalSize().X - LocalX ) < 60.f )
	{
		// Right half of the output pin or all of the input pin, treat it like a connection attempt
		return OnPinMouseDown(SenderGeometry, MouseEvent);
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SGraphPin::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		bIsMovingLinks = MouseEvent.IsControlDown() && (GraphPin->LinkedTo.Num() > 0);
	}

	return FReply::Unhandled();
}

TOptional<EMouseCursor::Type> SGraphPin::GetPinCursor() const
{
	check(PinImage.IsValid());

	if (PinImage->IsHovered())
	{
		if (bIsMovingLinks)
		{
			return EMouseCursor::GrabHandClosed;
		}
		else
		{
			return EMouseCursor::Crosshairs;
		}
	}
	else
	{
		return EMouseCursor::Default;
	}
}

TSharedRef<FDragDropOperation> SGraphPin::SpawnPinDragEvent(const TSharedRef<SGraphPanel>& InGraphPanel, const TArray< TSharedRef<SGraphPin> >& InStartingPins)
{
	FDragConnection::FDraggedPinTable PinHandles;
	PinHandles.Reserve(InStartingPins.Num());
	// since the graph can be refreshed and pins can be reconstructed/replaced 
	// behind the scenes, the DragDropOperation holds onto FGraphPinHandles 
	// instead of direct widgets/graph-pins
	for (const TSharedRef<SGraphPin>& PinWidget : InStartingPins)
	{
		PinHandles.Add(PinWidget->GetPinObj());
	}

	return FDragConnection::New(InGraphPanel, PinHandles);
}

/**
 * The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
 *
 * @param MyGeometry The Geometry of the widget receiving the event
 * @param MouseEvent Information about the input event
 *
 * @return Whether the event was handled along with possible requests for the system to take action.
 */
FReply SGraphPin::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.IsShiftDown())
	{
		// Either store the shift-clicked pin or attempt to connect it if already stored
		TSharedPtr<SGraphPanel> OwnerPanelPtr = OwnerNodePtr.Pin()->GetOwnerPanel();
		check(OwnerPanelPtr.IsValid());
		if (OwnerPanelPtr->MarkedPin.IsValid())
		{
			// avoid creating transaction if toggling the marked pin
			if (!OwnerPanelPtr->MarkedPin.HasSameObject(this))
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_CreateConnection", "Create Pin Link") );
				TryHandlePinConnection(*OwnerPanelPtr->MarkedPin.Pin());
			}
			OwnerPanelPtr->MarkedPin.Reset();
		}
		else
		{
			OwnerPanelPtr->MarkedPin = SharedThis(this);
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SGraphPin::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!IsHovered() && ensure(!bGraphDataInvalid) && GetIsConnectable())
	{
		UEdGraphPin* MyPin = GetPinObj();
		if (MyPin && !MyPin->IsPendingKill() && MyPin->GetOuter() && MyPin->GetOuter()->IsA(UEdGraphNode::StaticClass()))
		{
			struct FHoverPinHelper
			{
				FHoverPinHelper(TSharedPtr<SGraphPanel> Panel, TSet<FEdGraphPinReference>& PinSet)
					: PinSetOut(PinSet), TargetPanel(Panel)
				{}

				void SetHovered(UEdGraphPin* Pin)
				{
					bool bAlreadyAdded = false;
					PinSetOut.Add(Pin, &bAlreadyAdded);
					if (bAlreadyAdded)
					{
						return;
					}
					TargetPanel->AddPinToHoverSet(Pin);
	
					for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
					{
						int32 InputPinIndex = -1;
						int32 OutputPinIndex = -1;
						UEdGraphNode* InKnot = LinkedPin->GetOwningNodeUnchecked();
						if (InKnot != nullptr && InKnot->ShouldDrawNodeAsControlPointOnly(InputPinIndex, OutputPinIndex) == true &&
							InputPinIndex >= 0 && OutputPinIndex >= 0)
						{
							SetHovered(InKnot);
						}
					}
				}

			private:
				void SetHovered(UEdGraphNode* KnotNode)
				{
					bool bAlreadyTraversed = false;
					IntermediateNodes.Add(KnotNode, &bAlreadyTraversed);

					if (!bAlreadyTraversed)
					{
						for (UEdGraphPin* KnotPin : KnotNode->Pins)
						{
							SetHovered(KnotPin);
						}
					}
				}

			private:
				TSet<UEdGraphNode*> IntermediateNodes;
				TSet<FEdGraphPinReference>& PinSetOut;
				TSharedPtr<SGraphPanel>   TargetPanel;
			};


			TSharedPtr<SGraphPanel> Panel = OwnerNodePtr.Pin()->GetOwnerPanel();
			if (Panel.IsValid())
			{
				FHoverPinHelper(Panel, HoverPinSet).SetHovered(MyPin);
			}
		}
	}

	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
}

void SGraphPin::OnMouseLeave( const FPointerEvent& MouseEvent )
{	
	TSharedPtr<SGraphPanel> Panel = OwnerNodePtr.Pin()->GetOwnerPanel();

	for (const FEdGraphPinReference& WeakPin : HoverPinSet)
	{
		if (UEdGraphPin* PinInNet = WeakPin.Get())
		{
			Panel->RemovePinFromHoverSet(PinInNet);
		}
	}
	HoverPinSet.Empty();

	SCompoundWidget::OnMouseLeave(MouseEvent);
}

void SGraphPin::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return;
	}

	// Is someone dragging a connection?
	if (Operation->IsOfType<FGraphEditorDragDropAction>())
	{
		// Ensure that the pin is valid before using it - in the case of OnDragEnter a previous OnPinNameMouseDown handler
		// may have invalidated the graph data:
		if(!bGraphDataInvalid && GraphPinObj != NULL && !GraphPinObj->IsPendingKill() && GraphPinObj->GetOuter() != NULL && GraphPinObj->GetOuter()->IsA(UEdGraphNode::StaticClass()))
		{
			if (GetIsConnectable())
			{
				// Inform the Drag and Drop operation that we are hovering over this pin.
				TSharedPtr<FGraphEditorDragDropAction> DragConnectionOp = StaticCastSharedPtr<FGraphEditorDragDropAction>(Operation);
				DragConnectionOp->SetHoveredPin(GraphPinObj);

				// Pins treat being dragged over the same as being hovered outside of drag and drop if they know how to respond to the drag action.
				SBorder::OnMouseEnter(MyGeometry, DragDropEvent);
			}
		}	
	}
}

void SGraphPin::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return;
	}

	// Is someone dragging a connection?
	if (Operation->IsOfType<FGraphEditorDragDropAction>() && GetIsConnectable())
	{
		// Inform the Drag and Drop operation that we are not hovering any pins
		TSharedPtr<FGraphEditorDragDropAction> DragConnectionOp = StaticCastSharedPtr<FGraphEditorDragDropAction>(Operation);
		DragConnectionOp->SetHoveredPin(nullptr);

		SBorder::OnMouseLeave(DragDropEvent);
	}

	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> AssetOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
		AssetOp->ResetToDefaultToolTip();
	}
}

FReply SGraphPin::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return FReply::Unhandled();
	}

	if (Operation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<SGraphNode> NodeWidget = OwnerNodePtr.Pin();
		if (NodeWidget.IsValid())
		{
			UEdGraphNode* Node = NodeWidget->GetNodeObj();
			if(Node != NULL && Node->GetSchema() != NULL)
			{
				TSharedPtr<FAssetDragDropOp> AssetOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
				bool bOkIcon = false;
				FString TooltipText;
				if (AssetOp->HasAssets())
				{
					Node->GetSchema()->GetAssetsPinHoverMessage(AssetOp->GetAssets(), GraphPinObj, TooltipText, bOkIcon);
				}
				const FSlateBrush* TooltipIcon = bOkIcon ? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")) : FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));;
				AssetOp->SetToolTip(FText::FromString(TooltipText), TooltipIcon);
					
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

bool SGraphPin::TryHandlePinConnection(SGraphPin& OtherSPin)
{
	UEdGraphPin* PinA = GetPinObj();
	UEdGraphPin* PinB = OtherSPin.GetPinObj();
	if (PinA && PinB)
	{
		UEdGraph* MyGraphObj = PinA->GetOwningNode()->GetGraph();
		check(MyGraphObj);
		return MyGraphObj->GetSchema()->TryCreateConnection(PinA, PinB);
	}
	return false;
}

FReply SGraphPin::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<SGraphNode> NodeWidget = OwnerNodePtr.Pin();
	bool bReadOnly = NodeWidget.IsValid() ? !NodeWidget->IsNodeEditable() : false;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid() || bReadOnly)
	{
		return FReply::Unhandled();
	}

	// Is someone dropping a connection onto this pin?
	if (Operation->IsOfType<FGraphEditorDragDropAction>() && GetIsConnectable())
	{
		TSharedPtr<FGraphEditorDragDropAction> DragConnectionOp = StaticCastSharedPtr<FGraphEditorDragDropAction>(Operation);

		FVector2f NodeAddPosition = FVector2f::ZeroVector;
		TSharedPtr<SGraphNode> OwnerNode = OwnerNodePtr.Pin();
		if (OwnerNode.IsValid())
		{
			NodeAddPosition	= OwnerNode->GetPosition2f() + MyGeometry.Position;

			//Don't have access to bounding information for node, using fixed offset that should work for most cases.
			const float FixedOffset = 200.0f;

			//Line it up vertically with pin
			NodeAddPosition.Y += MyGeometry.Size.Y;

			// if the pin widget is nested into another compound
			if (MyGeometry.Position == FVector2f::ZeroVector)
			{
				FVector2f PinOffsetPosition = MyGeometry.AbsolutePosition - NodeWidget->GetTickSpaceGeometry().AbsolutePosition;
				NodeAddPosition = OwnerNode->GetPosition2f() + PinOffsetPosition;
			}

			if(GetDirection() == EEdGraphPinDirection::EGPD_Input)
			{
				//left side just offset by fixed amount
				//@TODO: knowing the width of the node we are about to create would allow us to line this up more precisely,
				//       but this information is not available currently
				NodeAddPosition.X -= FixedOffset;
			}
			else
			{
				//right side we need the width of the pin + fixed amount because our reference position is the upper left corner of pin(which is variable length)
				NodeAddPosition.X += MyGeometry.Size.X + FixedOffset;
			}
			
		}

		return DragConnectionOp->DroppedOnPin(DragDropEvent.GetScreenSpacePosition(), NodeAddPosition);
	}
	// handle dropping an asset on the pin
	else if (Operation->IsOfType<FAssetDragDropOp>() && NodeWidget.IsValid())
	{
		UEdGraphNode* Node = NodeWidget->GetNodeObj();
		if(Node != NULL && Node->GetSchema() != NULL)
		{
			TSharedPtr<FAssetDragDropOp> AssetOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
			if (AssetOp->HasAssets())
			{
				Node->GetSchema()->DroppedAssetsOnPin(AssetOp->GetAssets(), NodeWidget->GetPosition2f() + MyGeometry.Position, GraphPinObj);
			}
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SGraphPin::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	CachedNodeOffset = AllottedGeometry.AbsolutePosition / AllottedGeometry.Scale - UE::Slate::CastToVector2f(OwnerNodePtr.Pin()->GetUnscaledPosition());
	CachedNodeOffset.Y += AllottedGeometry.Size.Y * 0.5f;

	if (!ValueInspectorTooltip.IsValid() && IsHovered() && FKismetDebugUtilities::CanInspectPinValue(GetPinObj()))
	{
		ValueInspectorTooltip = FPinValueInspectorTooltip::SummonTooltip(GetPinObj());
		TSharedPtr<FPinValueInspectorTooltip> ValueTooltip = ValueInspectorTooltip.Pin();

		if (ValueTooltip.IsValid())
		{
			FVector2f TooltipLocation;
			GetInteractiveTooltipLocation(TooltipLocation);
			ValueTooltip->MoveTooltip(TooltipLocation);
		}
	}
	else if (ValueInspectorTooltip.IsValid() && ((!IsHovered()) || !FKismetDebugUtilities::CanInspectPinValue(GetPinObj())))
	{
		ValueInspectorTooltip.Pin()->TryDismissTooltip();
	}
}

UEdGraphPin* SGraphPin::GetPinObj() const
{
	ensureMsgf(!bGraphDataInvalid, TEXT("The Graph Pin Object has been invalidated. Someone is keeping a hard ref on the SGraphPin (%s). See InvalidateGraphData for more info"), *ToString());

	if (bGraphDataInvalid || (GraphPinObj && GraphPinObj->bWasTrashed))
	{
		return nullptr;
	}

	return GraphPinObj;
}

/** @param OwnerNode  The SGraphNode that this pin belongs to */
void SGraphPin::SetOwner( const TSharedRef<SGraphNode> OwnerNode )
{
	check( !OwnerNodePtr.IsValid() );
	OwnerNodePtr = OwnerNode;
}

void SGraphPin::SetPinObj(UEdGraphPin* PinObj)
{
	GraphPinObj = PinObj;
}

EVisibility SGraphPin::IsPinVisibleAsAdvanced() const
{
	bool bHideAdvancedPin = false;
	const TSharedPtr<SGraphNode> NodeWidget = OwnerNodePtr.Pin();
	if (NodeWidget.IsValid())
	{
		if(const UEdGraphNode* Node = NodeWidget->GetNodeObj())
		{
			bHideAdvancedPin = (ENodeAdvancedPins::Hidden == Node->AdvancedPinDisplay);
		}
	}

	UEdGraphPin* GraphPin = GetPinObj();
	const bool bIsAdvancedPin = GraphPin && !GraphPin->IsPendingKill() && GraphPin->bAdvancedView;
	const bool bCanBeHidden = !IsConnected();
	return (bIsAdvancedPin && bHideAdvancedPin && bCanBeHidden) ? EVisibility::Collapsed : EVisibility::Visible;
}

UE::Slate::FDeprecateVector2DResult SGraphPin::GetNodeOffset() const
{
	return CachedNodeOffset;
}

FText SGraphPin::GetPinLabel() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && !GraphPin->IsPendingKill())
	{
		if (UEdGraphNode* GraphNode = GraphPin->GetOwningNodeUnchecked())
		{
			return GraphNode->GetPinDisplayName(GraphPin);
		}
	}
	return FText::GetEmpty();
}

/** @return whether this pin is incoming or outgoing */
EEdGraphPinDirection SGraphPin::GetDirection() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	return GraphPin ? static_cast<EEdGraphPinDirection>(GraphPinObj->Direction) : EEdGraphPinDirection::EGPD_MAX;
}

bool SGraphPin::IsArray() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	return GraphPin ? GraphPin->PinType.IsArray() : false;
}

bool SGraphPin::IsSet() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	return GraphPin ? GraphPinObj->PinType.IsSet() : false;
}

bool SGraphPin::IsMap() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	return GraphPin ? GraphPin->PinType.IsMap() : false;
}

bool SGraphPin::IsByMutableRef() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	return GraphPin ? (GraphPin->PinType.bIsReference && !GraphPin->PinType.bIsConst) : false;
}

bool SGraphPin::IsDelegate() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	const UEdGraphSchema* Schema = GraphPin ? GraphPin->GetSchema() : nullptr;
	return Schema && Schema->IsDelegateCategory(GraphPin->PinType.PinCategory);
}

/** @return whether this pin is connected to another pin */
bool SGraphPin::IsConnected() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	return GraphPin? GraphPin->LinkedTo.Num() > 0 : false;
}

bool SGraphPin::AreConnectionsFaded() const
{
	return bFadeConnections;
}

/** @return The brush with which to pain this graph pin's incoming/outgoing bullet point */
const FSlateBrush* SGraphPin::GetPinIcon() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (!GraphPin || GraphPin->IsPendingKill() || !GraphPin->GetOwningNodeUnchecked())
	{
		return CachedImg_Pin_Disconnected;
	}

	if (Custom_Brush_Connected && Custom_Brush_Disconnected)
	{
		if (IsConnected())
		{
			return Custom_Brush_Connected;
		}
		else
		{
			return Custom_Brush_Disconnected;
		}
	}

	if (IsArray())
	{
		if (IsConnected())
		{
			return CachedImg_ArrayPin_Connected;
		}
		else
		{
			return CachedImg_ArrayPin_Disconnected;
		}
	}
	else if(IsDelegate())
	{
		if (IsConnected())
		{
			return CachedImg_DelegatePin_Connected;		
		}
		else
		{
			return CachedImg_DelegatePin_Disconnected;
		}
	}
	else if (GraphPin->bDisplayAsMutableRef || IsByMutableRef())
	{
		if (IsConnected())
		{
			return CachedImg_RefPin_Connected;		
		}
		else
		{
			return CachedImg_RefPin_Disconnected;
		}

	}
	else if (IsSet())
	{
		return CachedImg_SetPin;
	}
	else if (IsMap())
	{
		return CachedImg_MapPinKey;
	}
	else if (GraphPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && ((GraphPin->PinType.PinSubCategoryObject == FPoseLink::StaticStruct()) || (GraphPin->PinType.PinSubCategoryObject == FComponentSpacePoseLink::StaticStruct())))
	{
		if (IsConnected())
		{
			return CachedImg_PosePin_Connected;
		}
		else
		{
			return CachedImg_PosePin_Disconnected;
		}
	}
	else
	{
		if (IsConnected())
		{
			return CachedImg_Pin_Connected;
		}
		else
		{
			return CachedImg_Pin_Disconnected;
		}
	}
}

const FSlateBrush* SGraphPin::GetSecondaryPinIcon() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if( GraphPin && !GraphPin->IsPendingKill() && GraphPin->PinType.IsMap() )
	{
		return CachedImg_MapPinValue;
	}
	return nullptr;
}

const FSlateBrush* SGraphPin::GetPinBorder() const
{
	bool bIsMarkedPin = false;
	const TSharedPtr<SGraphNode> OwnerNode = OwnerNodePtr.Pin();
	if(OwnerNode.IsValid())
	{
		const TSharedPtr<SGraphPanel> OwnerPanelPtr = OwnerNodePtr.Pin()->GetOwnerPanel();
		if (OwnerPanelPtr.IsValid())
		{
			if (OwnerPanelPtr->MarkedPin.IsValid())
			{
				bIsMarkedPin = (OwnerPanelPtr->MarkedPin.Pin() == SharedThis(this));
			}
		}

		TOptional<const FSlateBrush*> OptionalPinBorder = OwnerNode->GetPinBorder(this);
		if(OptionalPinBorder.IsSet())
		{
			return OptionalPinBorder.GetValue();
		}
	}
	return (IsHovered() || bIsMarkedPin || bIsDiffHighlighted || bOnlyShowDefaultValue) ? CachedImg_Pin_BackgroundHovered : CachedImg_Pin_Background;
}


FSlateColor SGraphPin::GetPinColor() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && !GraphPin->IsPendingKill())
	{
		if (bIsDiffHighlighted)
		{
			return FSlateColor(FLinearColor(0.9f, 0.2f, 0.15f));
		}
		if (GraphPin->bOrphanedPin)
		{
			return FSlateColor(FLinearColor::Red);
		}
		if (const UEdGraphSchema* Schema = GraphPin->GetSchema())
		{
			if (!GetPinObj()->GetOwningNode()->IsNodeEnabled() || GetPinObj()->GetOwningNode()->IsDisplayAsDisabledForced() || !IsEditingEnabled() || GetPinObj()->GetOwningNode()->IsNodeUnrelated())
			{
				return Schema->GetPinColor(GraphPin) * FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);
			}

			return Schema->GetPinColor(GraphPin) * PinColorModifier;
		}
	}

	return FLinearColor::White;
}

FSlateColor SGraphPin::GetHighlightColor() const
{
	if (PinDiffColor.IsSet())
	{
		return PinDiffColor.GetValue();
	}
	
	const TSharedPtr<SGraphNode> OwnerNode = OwnerNodePtr.Pin();
	if(OwnerNode.IsValid())
	{
		TOptional<FSlateColor> OptionalHighlightColor = OwnerNode->GetHighlightColor(this);
		if(OptionalHighlightColor.IsSet())
		{
			return OptionalHighlightColor.GetValue();
		}
	}
	
	return GetPinColor();
}

FSlateColor SGraphPin::GetPinDiffColor() const
{
	if (PinDiffColor.IsSet())
	{
		return PinDiffColor.GetValue();
	}
	return FLinearColor(0.f,0.f,0.f,0.f);
}

FSlateColor SGraphPin::GetSecondaryPinColor() const
{
	if (UEdGraphPin* GraphPin = GetPinObj())
	{
		if (!GraphPin->IsPendingKill())
		{
			if (const UEdGraphSchema_K2* Schema = Cast<UEdGraphSchema_K2>(GraphPin->GetSchema()))
			{
				return Schema->GetSecondaryPinTypeColor(GraphPin->PinType);
			}
		}
	}
	return FLinearColor::White;
}

FSlateColor SGraphPin::GetPinTextColor() const
{
	const TSharedPtr<SGraphNode> OwnerNode = OwnerNodePtr.Pin();
	if(OwnerNode.IsValid())
	{
		TOptional<FSlateColor> OptionalPinTextColor = OwnerNode->GetPinTextColor(this);
		if(OptionalPinTextColor.IsSet())
		{
			return OptionalPinTextColor.GetValue();
		}
	}

	UEdGraphPin* GraphPin = GetPinObj();
	// If there is no schema there is no owning node (or basically this is a deleted node)
	if (UEdGraphNode* GraphNode = GraphPin ? GraphPin->GetOwningNodeUnchecked() : nullptr)
	{
		const bool bDisabled = (!GraphNode->IsNodeEnabled() || GraphNode->IsDisplayAsDisabledForced() || !IsEditingEnabled() || GraphNode->IsNodeUnrelated());
		if (GraphPin->bOrphanedPin)
		{
			FLinearColor PinColor = FLinearColor::Red;
			if (bDisabled)
			{
				PinColor.A = .25f;
			}
			return PinColor;
		}
		else if (bDisabled)
		{
			return FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);
		}
		if (bUsePinColorForText)
		{
			return GetPinColor();
		}
	}
	return FLinearColor::White;
}


const FSlateBrush* SGraphPin::GetPinStatusIcon() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && !GraphPin->IsPendingKill())
	{
		UEdGraphPin* WatchedPin = ((GraphPin->Direction == EGPD_Input) && (GraphPin->LinkedTo.Num() > 0)) ? GraphPin->LinkedTo[0] : GraphPin;

		if (UEdGraphNode* GraphNode = WatchedPin->GetOwningNodeUnchecked())
		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(GraphNode);

			if (FKismetDebugUtilities::DoesPinHaveWatches(Blueprint, WatchedPin))
			{
				return FAppStyle::GetBrush(TEXT("Graph.WatchedPinIcon_Pinned"));
			}
		}
	}

	return nullptr;
}

EVisibility SGraphPin::GetPinStatusIconVisibility() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin == nullptr || GraphPinObj->IsPendingKill())
	{
		return EVisibility::Collapsed;
	}

	UEdGraphNode* GraphNode = GraphPin->GetOwningNode();
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(GraphNode);
	if (!Blueprint)
	{
		return EVisibility::Collapsed;
	}

	UEdGraphPin const* WatchedPin = ((GraphPin->Direction == EGPD_Input) && (GraphPin->LinkedTo.Num() > 0)) ? GraphPin->LinkedTo[0] : GraphPin;

	return FKismetDebugUtilities::DoesPinHaveWatches(Blueprint, WatchedPin) ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SGraphPin::ClickedOnPinStatusIcon()
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin == nullptr || GraphPinObj->IsPendingKill())
	{
		return FReply::Handled();
	}

	UEdGraphPin* WatchedPin = ((GraphPin->Direction == EGPD_Input) && (GraphPin->LinkedTo.Num() > 0)) ? GraphPin->LinkedTo[0] : GraphPin;

	UEdGraphSchema const* Schema = GraphPin->GetSchema();
	Schema->ClearPinWatch(WatchedPin);

	return FReply::Handled();
}

EVisibility SGraphPin::GetDefaultValueVisibility() const
{
	// If this is only for showing default value, always show
	if (bOnlyShowDefaultValue)
	{
		return EVisibility::Visible;
	}

	// First ask schema
	UEdGraphPin* GraphPin = GetPinObj();
	const UEdGraphSchema* Schema = (GraphPin && !GraphPin->IsPendingKill()) ? GraphPin->GetSchema() : nullptr;
	if (Schema == nullptr || Schema->ShouldHidePinDefaultValue(GraphPin))
	{
		return EVisibility::Collapsed;
	}

	if (GraphPin->bNotConnectable && !GraphPin->bOrphanedPin)
	{
		// The only reason this pin exists is to show something, so do so
		return EVisibility::Visible;
	}

	if (GraphPin->Direction == EGPD_Output)
	{
		//@TODO: Should probably be a bLiteralOutput flag or a Schema call
		return EVisibility::Collapsed;
	}
	else
	{
		return IsConnected() ? EVisibility::Collapsed : EVisibility::Visible;
	}
}

void SGraphPin::SetShowLabel(bool bNewShowLabel)
{
	bShowLabel = bNewShowLabel;
}

void SGraphPin::SetOnlyShowDefaultValue(bool bNewOnlyShowDefaultValue)
{
	bOnlyShowDefaultValue = bNewOnlyShowDefaultValue;
}

TSharedPtr<IToolTip> SGraphPin::GetToolTip()
{
	// If we want the PinValueInspector tooltip, we'll create a custom tooltip window
	const UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && FKismetDebugUtilities::CanInspectPinValue(GraphPin))
	{
		return nullptr;
	}

	return SBorder::GetToolTip();
}

FText SGraphPin::GetTooltipText() const
{
	if(bGraphDataInvalid)
	{
		return FText();
	}

	FText HoverText = FText::GetEmpty();

	UEdGraphPin* GraphPin = GetPinObj();
	UEdGraphNode* GraphNode = (GraphPin && !GraphPin->IsPendingKill()) ? GraphPin->GetOwningNodeUnchecked() : nullptr;
	if (GraphNode != nullptr)
	{
		FString HoverStr;
		GraphNode->GetPinHoverText(*GraphPin, /*out*/HoverStr);
		if (!HoverStr.IsEmpty())
		{
			HoverText = FText::FromString(HoverStr);
		}
	}

	return HoverText;
}

void SGraphPin::GetInteractiveTooltipLocation(FVector2D& InOutDesiredLocation) const
{
	FVector2f OutDesiredLocation = UE::Slate::CastToVector2f(InOutDesiredLocation);
	TSharedPtr<SGraphNode> OwnerNode = OwnerNodePtr.Pin();
	if (OwnerNode.IsValid())
	{
		TSharedPtr<SGraphPanel> GraphPanel = OwnerNode->GetOwnerPanel();
		if (GraphPanel.IsValid())
		{
			// Reset to the pin's location in graph space.
			OutDesiredLocation = OwnerNode->GetPosition2f() + CachedNodeOffset;

			// Shift the desired location to the right edge of the pin's geometry.
			OutDesiredLocation.X += GetTickSpaceGeometry().Size.X;

			// Align to the first entry in the inspector's tree view.
			TSharedPtr<FPinValueInspectorTooltip> Inspector = ValueInspectorTooltip.Pin();
			if (Inspector.IsValid() && Inspector->ValueInspectorWidget.IsValid())
			{
				// @todo - Find a way to calculate these at runtime, e.g. based off of actual child widget geometry?
				static const float VerticalOffsetWithSearchFilter = 41.0f;
				static const float VerticalOffsetWithoutSearchFilter = 19.0f;

				if (Inspector->ValueInspectorWidget->ShouldShowSearchFilter())
				{
					OutDesiredLocation.Y -= VerticalOffsetWithSearchFilter;
				}
				else
				{
					OutDesiredLocation.Y -= VerticalOffsetWithoutSearchFilter;
				}
			}

			// Convert our desired location from graph coordinates into panel space.
			OutDesiredLocation -= GraphPanel->GetViewOffset();
			OutDesiredLocation *= GraphPanel->GetZoomAmount();

			// Finally, convert the modified location from panel space into screen space.
			OutDesiredLocation = GraphPanel->GetTickSpaceGeometry().LocalToAbsolute(OutDesiredLocation);
		}
	}

	InOutDesiredLocation = FVector2D(OutDesiredLocation);
}

void SGraphPin::GetInteractiveTooltipLocation(FVector2f& InOutDesiredLocation) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVector2D TempVector2f = FVector2D(InOutDesiredLocation);
	GetInteractiveTooltipLocation(TempVector2f);
	InOutDesiredLocation = UE::Slate::CastToVector2f(TempVector2f);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool SGraphPin::IsEditingEnabled() const
{
	if (OwnerNodePtr.IsValid())
	{
		return OwnerNodePtr.Pin()->IsNodeEditable() && IsEditable.Get();
	}
	return IsEditable.Get();
}

bool SGraphPin::UseLowDetailPinNames() const
{
	SGraphNode* MyOwnerNode = OwnerNodePtr.Pin().Get();
	if (MyOwnerNode)
	{
		if(MyOwnerNode->UseLowDetailPinNames())
		{
			return true;
		}

		if(MyOwnerNode->GetOwnerPanel().IsValid())
		{
			return MyOwnerNode->GetOwnerPanel()->GetCurrentLOD() <= EGraphRenderingLOD::LowDetail;
		}
	}
	return false;
}

TSharedRef<SWidget> SGraphPin::OnGetActiveDetailSlotContent(bool bChangedToHighDetail) const
{
	if (PinImage)
	{
		// Here, we need to make sure we detach PinImage before setting it into the other slot
		PinImage->ConditionallyDetatchParentWidget(PinImage->GetParentWidget().Get());

		if (bChangedToHighDetail)
		{
			if (const TSharedPtr<SHorizontalBox> RowWidget = FullPinHorizontalRowWidget.Pin())
			{
				for (int32 i = 0; i < RowWidget->GetChildren()->NumSlot(); ++i)
				{
					if (&RowWidget->GetSlot(i) == PinImageHighDetailSlot)
					{
						RowWidget->GetSlot(i).AttachWidget(PinImage.ToSharedRef());
						break;
					}
				}
				return RowWidget.ToSharedRef();
			}
		}
		else
		{
			return PinImage.ToSharedRef();
		}
	}

	return SNullWidget::NullWidget;
}

EVisibility SGraphPin::GetPinVisiblity() const
{
	// The pin becomes too small to use at low LOD, so disable the hit test.
	if(UseLowDetailPinNames())
	{
		return EVisibility::HitTestInvisible;
	}
	return EVisibility::Visible;
}

TSharedPtr<SWidget> SGraphPin::GetPinImageWidget() const
{
	return PinImage;
}

void SGraphPin::SetPinImageWidget(TSharedRef<SWidget> NewWidget)
{
	PinImage = NewWidget;
}

void SGraphPin::SetCustomPinIcon(const FSlateBrush* InConnectedBrush, const FSlateBrush* InDisconnectedBrush)
{
	Custom_Brush_Connected = InConnectedBrush;
	Custom_Brush_Disconnected = InDisconnectedBrush;
}

bool SGraphPin::HasInteractiveTooltip() const
{
	return ValueInspectorTooltip.IsValid();
}

bool SGraphPin::SnapSGraphNode(TSharedRef<SGraphNode> Node)
{
	if(!SnappedDataNodeContainer)
	{
		const bool bIsInput = (GetDirection() == EGPD_Input);
		LabelAndValue->AddSlot()
			.Padding(bIsInput ? FMargin(5.f, 0.0f, 0.0f, 0.0f) : FMargin(0.0f, 0.0f, 5.f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SAssignNew(SnappedDataNodeContainer, SBox)
				.Padding(0.0f)
			];
	}

	SnappedDataNodeContainer->SetContent(Node);
	SnappedNode = Node;
	return true;
}

void SGraphPin::ClearSnappedSGraphNode()
{
	if(SnappedDataNodeContainer) // control flow snaps won't have a valud widget
	{
		SnappedDataNodeContainer->SetContent(SNullWidget::NullWidget);
	}
	SnappedNode.Reset();
}

bool SGraphPin::IsSnapped() const
{
	if(!bIsSnappingEnabled)
	{
		return false;
	}

	UEdGraphPin* GraphPin = GetPinObj();
	if(!GraphPin)
	{
		return false;
	}
	return GraphPin->IsSnapped();
}

TSharedPtr<SGraphNode> SGraphPin::GetSnappedChildForPosition(const FGeometry& MyGeometry, const FVector2f& Position, FVector2f& OutPositionInNode) const
{
	TSet< TSharedRef<SWidget> > ValueWidgets;
	TSharedPtr<SGraphNode> SnappedNodePinned = SnappedNode.Pin();
	if(!SnappedNodePinned)
	{
		return nullptr;
	}
	ValueWidgets.Add(SnappedNodePinned.ToSharedRef());
	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
	FindChildGeometries(MyGeometry, ValueWidgets, Result);
	if ( Result.Num() > 0 )
	{
		FArrangedChildren ArrangedPins(EVisibility::Visible);
		Result.GenerateValueArray( ArrangedPins.GetInternalArray() );
		int32 HoveredPinIndex = SWidget::FindChildUnderPosition( ArrangedPins, Position );
		if ( HoveredPinIndex != INDEX_NONE )
		{
			TSharedPtr<SGraphNode> HoveredNode = StaticCastSharedRef<SGraphNode>(
				ArrangedPins[HoveredPinIndex].Widget);
			OutPositionInNode = ArrangedPins[HoveredPinIndex].Geometry.AbsoluteToLocal(Position);
			return HoveredNode->GetSnappedChildForPosition(ArrangedPins[HoveredPinIndex].Geometry, Position, OutPositionInNode);
		}
	}
	return nullptr;
}

TOptional<FArrangedWidget> SGraphPin::GetArrangedSnappedChildNode(const FGeometry& MyGeometry) const
{
	TSharedPtr<SGraphNode> SnappedNodePinned = SnappedNode.Pin();
	if(!SnappedNodePinned)
	{
		return {};
	}
	
	TSet< TSharedRef<SWidget> > ValueWidgets;
	ValueWidgets.Add(SnappedNodePinned.ToSharedRef());
	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
	FindChildGeometries(MyGeometry, ValueWidgets, Result);
	if(Result.Num() == 0)
	{
		// culled
		return {};
	}
	check(Result.Num() == 1);
	FArrangedChildren ArrangedChildNodes(EVisibility::Visible);
	Result.GenerateValueArray( ArrangedChildNodes.GetInternalArray() );
	return ArrangedChildNodes[0];
}

TSharedPtr<SGraphNode> SGraphPin::GetSnappedNode() const
{
	return SnappedNode.Pin();
}

bool SGraphPin::GetIsConnectable() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	return GraphPin ? !GraphPin->bNotConnectable : false;
}
