// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGraphPreviewer.h"

#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Visibility.h"
#include "SGraphPanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

EActiveTimerReturnType SGraphPreviewer::RefreshGraphTimer(const double InCurrentTime, const float InDeltaTime)
{
	if (NeedsRefreshCounter > 0)
	{
		GraphPanel->ZoomToFit(false);
		NeedsRefreshCounter--;
		return EActiveTimerReturnType::Continue;
	}
	else
	{
		return EActiveTimerReturnType::Stop;
	}
}

void SGraphPreviewer::Construct( const FArguments& InArgs, UEdGraph* InGraphObj )
{
	EdGraphObj = InGraphObj;
	NeedsRefreshCounter = 0;

	TSharedPtr<SOverlay> DisplayStack;

	this->ChildSlot
	[
		SAssignNew(DisplayStack, SOverlay)

		// The graph panel
		+SOverlay::Slot()
		[
			SAssignNew(GraphPanel, SGraphPanel)
			.GraphObj(EdGraphObj)
			.IsEditable(false)
			.ShowGraphStateOverlay(InArgs._ShowGraphStateOverlay)
			.InitialZoomToFit(true)
			.OnUpdateGraphPanel(this, &SGraphPreviewer::OnUpdateGraphPanel)
		]

		// Bottom-right corner text indicating the type of tool
		+SOverlay::Slot()
		.Padding(4)
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Visibility( EVisibility::HitTestInvisible )
			.TextStyle( FAppStyle::Get(), "GraphPreview.CornerText" )
			.Text( InArgs._CornerOverlayText )
		]
	];

	// Note - this will also invoke OnUpdateGraphPanel (see below)
	GraphPanel->Update();

	// Add the title bar if specified
	if (InArgs._TitleBar.IsValid())
	{
		DisplayStack->AddSlot()
			.VAlign(VAlign_Top)
			[
				InArgs._TitleBar.ToSharedRef()
			];
	}
}

void SGraphPreviewer::OnUpdateGraphPanel()
{
	if (!NeedsRefreshCounter)
	{
		// As node's bounds don't get updated immediately, to truly zoom out to fit we need to tick a couple times
		NeedsRefreshCounter = 2;
		RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SGraphPreviewer::RefreshGraphTimer));
	}
}