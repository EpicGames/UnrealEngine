// Copyright Epic Games, Inc. All Rights Reserved.


#include "SNodePanel.h"

#include "Delegates/Delegate.h"
#include "DiffResults.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MarqueeRect.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "GraphEditorSnapping/NodeSnappingManager.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Math/IntPoint.h"
#include "Misc/AssertionMacros.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateRenderer.h"
#include "ScopedTransaction.h"
#include "Settings/EditorStyleSettings.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Styling/SlateBrush.h"
#include "Templates/RemoveReference.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SNodePanel)

class FWidgetStyle;

struct FZoomLevelEntry
{
public:
	FZoomLevelEntry(float InZoomAmount, const FText& InDisplayText, EGraphRenderingLOD::Type InLOD)
		: DisplayText(FText::Format(NSLOCTEXT("GraphEditor", "Zoom", "Zoom {0}"), InDisplayText))
	 , ZoomAmount(InZoomAmount)
	 , LOD(InLOD)
	{
	}

public:
	FText DisplayText;
	float ZoomAmount;
	EGraphRenderingLOD::Type LOD;
};

struct FFixedZoomLevelsContainer : public FZoomLevelsContainer
{
	FFixedZoomLevelsContainer()
	{
		ZoomLevels.Reserve(20);
		ZoomLevels.Add(FZoomLevelEntry(0.100f, FText::FromString(TEXT("-12")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.125f, FText::FromString(TEXT("-11")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.150f, FText::FromString(TEXT("-10")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.175f, FText::FromString(TEXT("-9")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.200f, FText::FromString(TEXT("-8")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.225f, FText::FromString(TEXT("-7")), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.250f, FText::FromString(TEXT("-6")), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.375f, FText::FromString(TEXT("-5")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.500f, FText::FromString(TEXT("-4")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.675f, FText::FromString(TEXT("-3")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.750f, FText::FromString(TEXT("-2")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.875f, FText::FromString(TEXT("-1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.000f, FText::FromString(TEXT("1:1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.250f, FText::FromString(TEXT("+1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.375f, FText::FromString(TEXT("+2")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.500f, FText::FromString(TEXT("+3")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(1.675f, FText::FromString(TEXT("+4")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(1.750f, FText::FromString(TEXT("+5")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(1.875f, FText::FromString(TEXT("+6")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(2.000f, FText::FromString(TEXT("+7")), EGraphRenderingLOD::FullyZoomedIn));
	}

	float GetZoomAmount(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].ZoomAmount;
	}

	int32 GetNearestZoomLevel(float InZoomAmount) const override
	{
		for (int32 ZoomLevelIndex=0; ZoomLevelIndex < GetNumZoomLevels(); ++ZoomLevelIndex)
		{
			if (InZoomAmount <= GetZoomAmount(ZoomLevelIndex))
			{
				return ZoomLevelIndex;
			}
		}

		return GetDefaultZoomLevel();
	}
	
	FText GetZoomText(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].DisplayText;
	}
	
	int32 GetNumZoomLevels() const override
	{
		return ZoomLevels.Num();
	}
	
	int32 GetDefaultZoomLevel() const override
	{
		return 12;
	}

	EGraphRenderingLOD::Type GetLOD(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].LOD;
	}

	TArray<FZoomLevelEntry> ZoomLevels;
};

const TCHAR* XSymbol=TEXT("\xD7");

//////////////////////////////////////////////////////////////////////////
// FGraphSelectionManager

const FGraphPanelSelectionSet& FGraphSelectionManager::GetSelectedNodes() const
{
	return ObjectPtrDecay(SelectedNodes);
}

void FGraphSelectionManager::SelectSingleNode(SelectedItemType Node)
{
	SelectedNodes.Empty();
	SetNodeSelection(Node, true);
}

// Reset the selection state of all nodes
void FGraphSelectionManager::ClearSelectionSet()
{
	if (SelectedNodes.Num())
	{
		SelectedNodes.Empty();
		OnSelectionChanged.ExecuteIfBound(ObjectPtrDecay(SelectedNodes));
	}
}

// Changes the selection set to contain exactly all of the passed in nodes
void FGraphSelectionManager::SetSelectionSet(FGraphPanelSelectionSet& NewSet)
{
	SelectedNodes = ObjectPtrWrap(NewSet);
	OnSelectionChanged.ExecuteIfBound(ObjectPtrDecay(SelectedNodes));
}

void FGraphSelectionManager::SetNodeSelection(SelectedItemType Node, bool bSelect)
{
	ensureMsgf(Node != nullptr, TEXT("Node is invalid"));
	if (bSelect)
	{
		SelectedNodes.Add(Node);
		OnSelectionChanged.ExecuteIfBound(ObjectPtrDecay(SelectedNodes));
	}
	else
	{
		SelectedNodes.Remove(Node);
		OnSelectionChanged.ExecuteIfBound(ObjectPtrDecay(SelectedNodes));
	}
}

bool FGraphSelectionManager::IsNodeSelected(SelectedItemType Node) const
{
	return SelectedNodes.Contains(Node);
}


void FGraphSelectionManager::StartDraggingNode(SelectedItemType NodeBeingDragged, const FPointerEvent& MouseEvent)
{
	if (!IsNodeSelected(NodeBeingDragged))
	{
		if (MouseEvent.IsControlDown() || MouseEvent.IsShiftDown())
		{
			// Control and shift do not clear existing selection.
			SetNodeSelection(NodeBeingDragged, true);
		}
		else
		{
			SelectSingleNode(NodeBeingDragged);
		}
	}
}

void FGraphSelectionManager::ClickedOnNode(SelectedItemType Node, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsShiftDown())
	{
		// Shift always adds to selection
		SetNodeSelection(Node, true);
	}
	else if (MouseEvent.IsControlDown())
	{
		// Control toggles selection
		SetNodeSelection(Node, !IsNodeSelected(Node));
	}				
	else
	{
		// No modifiers sets selection
		SelectSingleNode(Node);
	}
}

void FGraphSelectionManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(SelectedNodes);
}

//////////////////////////////////////////////////////////////////////////
// SNodePanel

namespace NodePanelDefs
{
	// Default Zoom Padding Value
	static const float DefaultZoomPadding = 25.f;
	// Node Culling Guardband Area
	static const float GuardBandArea = 0.25f;
	// Scaling factor to reduce speed of mouse zooming
	static const float MouseZoomScaling = 0.04f;
};


SLATE_IMPLEMENT_WIDGET(SNodePanel)
void SNodePanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IsEditableAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ViewOffsetAttribute, EInvalidateWidgetReason::Layout);
}


// equivalent of SLATE_IMPLEMENT_WIDGET(SNodePanel::SNode)
FSlateWidgetClassRegistration ClassRegistration__SNodePanel_SNode = SNodePanel::SNode::StaticWidgetClass();

void SNodePanel::SNode::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Children);
	FNodeSlot::RegisterAttributes(Initializer);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, DesiredSizeScaleAttribute, EInvalidateWidgetReason::Prepass);
}

void SNodePanel::SNode::FNodeSlot::RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
{
	TBasicLayoutWidgetSlot::RegisterAttributes(AttributeInitializer);

	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FNodeSlot, AttributeInitializer, "Slot.AllowScale", AllowScale, EInvalidateWidgetReason::Prepass | EInvalidateWidgetReason::Paint);
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FNodeSlot, AttributeInitializer, "Slot.Size", Size, EInvalidateWidgetReason::Layout);
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FNodeSlot, AttributeInitializer, "Slot.Offset", Offset, EInvalidateWidgetReason::Layout)
		.UpdatePrerequisite("Slot.Size");
}

void SNodePanel::SNode::FNodeSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
{
	TBasicLayoutWidgetSlot<FNodeSlot>::Construct(SlotOwner, MoveTemp(InArgs));

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (InArgs._SlotOffset.IsSet())
	{
		SetSlotOffset(InArgs._SlotOffset);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (InArgs._SlotOffset2f.IsSet())
	{
		Offset.Assign(*this, MoveTemp(InArgs._SlotOffset2f));
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (InArgs._SlotSize.IsSet())
	{
		SetSlotSize(InArgs._SlotSize);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (InArgs._SlotSize2f.IsSet())
	{
		SetSlotSize(MoveTemp(InArgs._SlotSize2f));
	}
	if (InArgs._AllowScaling.IsSet())
	{
		AllowScale.Assign(*this, MoveTemp(InArgs._AllowScaling));
	}
}

TArray<SNodePanel::SNode::DiffHighlightInfo> SNodePanel::SNode::GetDiffHighlights(
	const FDiffSingleResult& DiffResult) const
{
	FLinearColor BackgroundColor = DiffResult.GetDisplayColor();
	BackgroundColor.A = 1.f; // give highlight some transparency so it's not so 'in your face'
		
	FLinearColor ShadingColorHSV = BackgroundColor.LinearRGBToHSV();
	ShadingColorHSV.R -= 15.f; // shift hue
	if (ShadingColorHSV.R < 0.f)
	{
		ShadingColorHSV.R += 360.f;
	}
	ShadingColorHSV.B *= 0.2f; // darken

	const FSlateBrush* BackgroundBrush;
	const FSlateBrush* ForegroundBrush;
	GetDiffHighlightBrushes(BackgroundBrush, ForegroundBrush);

	return {
		{
			BackgroundBrush,
			BackgroundColor
		},
		{
			ForegroundBrush,
			ShadingColorHSV.HSVToLinearRGB()
		},
	};
}

SNodePanel::SNodePanel()
	: ZoomLevel(0)
	, Children(this)
	, VisibleChildren(this)
	, NodeSnappingManager(nullptr)
	, IsEditableAttribute(*this, true)
	, ViewOffsetAttribute(*this, FVector2f::ZeroVector)
{
	bHasRelativeLayoutScale = true;
}

void SNodePanel::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	ArrangeChildNodes(AllottedGeometry, ArrangedChildren);
}

void SNodePanel::ArrangeChildNodes(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	FChildren* ChildrenToArrange = ArrangedChildren.Accepts(EVisibility::Hidden) ? (FChildren*)&Children : (FChildren*)&VisibleChildren;

	// First pass nodes
	ChildrenToArrange->ForEachWidget([&, this](SWidget& ChildWidget)
	{
		SNode& SomeChild = static_cast<SNode&>(ChildWidget);
		if (!SomeChild.RequiresSecondPassLayout())
		{
			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(SomeChild.AsShared(), SomeChild.GetPosition2f() - GetViewOffset(), SomeChild.GetDesiredSize(), GetZoomAmount()));
		}
	});

	// Second pass nodes
	ChildrenToArrange->ForEachWidget([&, this](SWidget& ChildWidget)
	{
		SNode& SomeChild = static_cast<SNode&>(ChildWidget);
		if (SomeChild.RequiresSecondPassLayout())
		{
			SomeChild.PerformSecondPassLayout(NodeToWidgetLookup);
			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(SomeChild.AsShared(), SomeChild.GetPosition2f() - GetViewOffset(), SomeChild.GetDesiredSize(), GetZoomAmount()));
		}
	});
}

FVector2D SNodePanel::ComputeDesiredSize( float ) const
{	
	// In this case, it would be an expensive computation that is not worth performing.
	// Users prefer to explicitly size canvases just like they do with text documents, browser pages, etc.
	return FVector2D(160.0f, 120.0f);
}

FChildren* SNodePanel::GetChildren()
{
	return &VisibleChildren;
}

FChildren* SNodePanel::GetAllChildren()
{
	return &VisibleChildren;
}

FChildren* SNodePanel::GetVisibleChildren()
{
	return &VisibleChildren;
}

FChildren* SNodePanel::GetManagedChildren()
{
	return &Children;
}

float SNodePanel::GetZoomAmount() const
{
	if (bAllowContinousZoomInterpolation)
	{
		return FMath::Lerp(ZoomLevels->GetZoomAmount(PreviousZoomLevel), ZoomLevels->GetZoomAmount(ZoomLevel), ZoomLevelGraphFade.GetLerp());
	}
	else
	{
		return ZoomLevels->GetZoomAmount(ZoomLevel);
	}
}

FText SNodePanel::GetZoomText() const
{
	return ZoomLevels->GetZoomText(ZoomLevel);
}

FSlateColor SNodePanel::GetZoomTextColorAndOpacity() const
{
	return FLinearColor( 1, 1, 1, 1.25f - ZoomLevelFade.GetLerp() );
}

UE::Slate::FDeprecateVector2DResult SNodePanel::GetViewOffset() const
{
	return ViewOffsetAttribute.Get();
}

bool SNodePanel::GetZoomTargetRect(FVector2D& TopLeft, FVector2D& BottomRight) const
{
	FVector2f TempTopLeft, TempBottomRight;
	bool bRetVal = GetZoomTargetRect(TempTopLeft, TempBottomRight);
	TopLeft = FVector2D(TempTopLeft);
	BottomRight = FVector2D(TempBottomRight);
	return bRetVal;
}

bool SNodePanel::GetZoomTargetRect(FVector2f& TopLeft, FVector2f& BottomRight) const
{
	TopLeft = ZoomTargetTopLeft;
	BottomRight = ZoomTargetBottomRight;
	
	// if the zoom target rect is all zeroed out, then notify caller that there is no target
	return !(ZoomTargetTopLeft == FVector2f::ZeroVector && ZoomTargetBottomRight == FVector2f::ZeroVector);
}

void SNodePanel::Construct()
{
	if (!ZoomLevels)
	{
		SetZoomLevelsContainer<FFixedZoomLevelsContainer>();
	}
	ZoomLevel = ZoomLevels->GetDefaultZoomLevel();
	PreviousZoomLevel = ZoomLevels->GetDefaultZoomLevel();
	PostChangedZoom();

	SetViewOffset(FVector2f::ZeroVector);
	TotalMouseDelta = 0;
	TotalMouseDeltaXY = 0;
	bDeferredZoomToSelection = false;
	bDeferredZoomToNodeExtents = false;

	ZoomTargetTopLeft = FVector2f::ZeroVector;
	ZoomTargetBottomRight = FVector2f::ZeroVector;
	ZoomPadding = NodePanelDefs::DefaultZoomPadding;

	bAllowZoom = true;
	bAllowPanning = true;
	bAllowContinousZoomInterpolation = false;
	bTeleportInsteadOfScrollingWhenZoomingToFit = false;

	DeferredSelectionTargetObjects.Empty();
	DeferredMovementTargetObject = nullptr;

	bIsPanning = false;
	bIsZoomingWithTrackpad = false;
	SetIsEditable(true);

	ZoomLevelFade = FCurveSequence( 0.0f, 1.0f );
	ZoomLevelFade.Play( this->AsShared() );

	ZoomLevelGraphFade = FCurveSequence( 0.0f, 0.5f );
	ZoomLevelGraphFade.Play( this->AsShared() );

	PastePosition = FVector2f::ZeroVector;

	DeferredPanPosition = FVector2f::ZeroVector;
	bRequestDeferredPan = false;

	OldViewOffset = GetViewOffset();
	OldZoomAmount = GetZoomAmount();
	ZoomStartOffset = FVector2f::ZeroVector;
	TotalGestureMagnify = 0.0f;

	ScopedTransactionPtr.Reset();

	bVisualUpdatePending = false;

	SelectionManager.OnSelectionChanged.BindRaw(this, &SNodePanel::OnSelectionChanged);
}

FVector2f SNodePanel::ComputeEdgePanAmount(const FGeometry& MyGeometry, const FVector2f& TargetPosition)
{
	// How quickly to ramp up the pan speed as the user moves the mouse further past the edge of the graph panel.
	static const float EdgePanSpeedCoefficient = 2.f, EdgePanSpeedPower = 0.6f;

	// Never pan faster than this - probably not really required since we raise to a power of 0.6
	static const float MaxPanSpeed = 200.0f;

	// Start panning before we reach the edge of the graph panel.
	static const float EdgePanForgivenessZone = 30.0f;

	const FVector2f LocalCursorPos = FVector2f(MyGeometry.AbsoluteToLocal( TargetPosition ));

	// If the mouse is outside of the graph area, then we want to pan in that direction.
	// The farther out the mouse is, the more we want to pan.

	FVector2f EdgePanThisTick(0,0);
	if ( LocalCursorPos.X <= EdgePanForgivenessZone )
	{
		EdgePanThisTick.X += FMath::Max( -MaxPanSpeed, EdgePanSpeedCoefficient * -FMath::Pow(EdgePanForgivenessZone - LocalCursorPos.X, EdgePanSpeedPower) );
	}
	else if( LocalCursorPos.X >= MyGeometry.GetLocalSize().X - EdgePanForgivenessZone )
	{
		EdgePanThisTick.X = FMath::Min( MaxPanSpeed, EdgePanSpeedCoefficient * FMath::Pow(LocalCursorPos.X - float(MyGeometry.GetLocalSize().X) + EdgePanForgivenessZone, EdgePanSpeedPower) );
	}

	if ( LocalCursorPos.Y <= EdgePanForgivenessZone )
	{
		EdgePanThisTick.Y += FMath::Max( -MaxPanSpeed, EdgePanSpeedCoefficient * -FMath::Pow(EdgePanForgivenessZone - LocalCursorPos.Y, EdgePanSpeedPower) );
	}
	else if( LocalCursorPos.Y >= MyGeometry.GetLocalSize().Y - EdgePanForgivenessZone )
	{
		EdgePanThisTick.Y = FMath::Min( MaxPanSpeed, EdgePanSpeedCoefficient * FMath::Pow(LocalCursorPos.Y - float(MyGeometry.GetLocalSize().Y) + EdgePanForgivenessZone, EdgePanSpeedPower) );
	}

	return EdgePanThisTick;
}

void SNodePanel::UpdateViewOffset(const FGeometry& MyGeometry, const FVector2f& TargetPosition)
{
	if (!bAllowPanning)
	{
		return;
	}
	const FVector2f PanAmount = ComputeEdgePanAmount( MyGeometry, TargetPosition ) / GetZoomAmount();
	SetViewOffset(GetViewOffset() + PanAmount);
}

void SNodePanel::RequestDeferredPan(const UE::Slate::FDeprecateVector2DParameter& UpdatePosition)
{
	bRequestDeferredPan = true;
	DeferredPanPosition = UpdatePosition;
}

FVector2f SNodePanel::GraphCoordToPanelCoord( const UE::Slate::FDeprecateVector2DParameter& GraphSpaceCoordinate ) const
{
	return (GraphSpaceCoordinate - GetViewOffset()) * GetZoomAmount();
}

UE::Slate::FDeprecateVector2DResult SNodePanel::PanelCoordToGraphCoord( const UE::Slate::FDeprecateVector2DParameter& PanelSpaceCoordinate ) const
{
	return PanelSpaceCoordinate / GetZoomAmount() + GetViewOffset();
}

FSlateRect SNodePanel::PanelRectToGraphRect( const FSlateRect& PanelSpaceRect ) const
{
	FVector2f UpperLeft = PanelCoordToGraphCoord( FVector2f(PanelSpaceRect.Left, PanelSpaceRect.Top) );
	FVector2f LowerRight = PanelCoordToGraphCoord(  FVector2f(PanelSpaceRect.Right, PanelSpaceRect.Bottom) );

	return FSlateRect(
		UpperLeft.X, UpperLeft.Y,
		LowerRight.X, LowerRight.Y );
}

void SNodePanel::OnBeginNodeInteraction(const TSharedRef<SNode>& InNodeToDrag, const FVector2D& GrabOffset)
{
	NodeUnderMousePtr = InNodeToDrag;
	NodeGrabOffset = UE::Slate::CastToVector2f(GrabOffset);
}

void SNodePanel::OnBeginNodeInteraction(const TSharedRef<SNode>& InNodeToDrag, const FVector2f& GrabOffset)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnBeginNodeInteraction(InNodeToDrag, FVector2D(GrabOffset));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void SNodePanel::OnEndNodeInteraction(const TSharedRef<SNode>& InNodeToDrag)
{
	InNodeToDrag->EndUserInteraction();
}

FNodeSnappingManager* SNodePanel::GetSnappingManager()
{
	return NodeSnappingManager.Get();
}

const FNodeSnappingManager* SNodePanel::GetSnappingManager() const
{
	return const_cast<SNodePanel*>(this)->GetSnappingManager();
}

EActiveTimerReturnType SNodePanel::HandleZoomToFit(double InCurrentTime, float InDeltaTime)
{
	// don't run this timer if we don't allow pan or zoom
	if (!bAllowPanning || !bAllowZoom)
	{
		return EActiveTimerReturnType::Stop;
	}
	
	const FVector2f DesiredViewCenter = ( ZoomTargetTopLeft + ZoomTargetBottomRight ) * 0.5f;
	const bool bDoneScrolling = ScrollToLocation(CachedGeometry, DesiredViewCenter, bTeleportInsteadOfScrollingWhenZoomingToFit ? 1000.0f : InDeltaTime);
	bool bDoneZooming = ZoomToLocation(CachedGeometry.GetLocalSize(), ZoomTargetBottomRight - ZoomTargetTopLeft, bDoneScrolling);

	if (bDoneZooming && bDoneScrolling)
	{
		// One final push to make sure we centered in the end
		SetViewOffset(DesiredViewCenter - ( 0.5f * CachedGeometry.GetLocalSize() / GetZoomAmount() ));
		
		// Reset ZoomPadding
		ZoomPadding = NodePanelDefs::DefaultZoomPadding;
		ZoomTargetTopLeft = FVector2f::ZeroVector;
		ZoomTargetBottomRight = FVector2f::ZeroVector;

		DeferredMovementTargetObject = nullptr;

		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

void SNodePanel::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	CachedGeometry = AllottedGeometry;
	bool bCanMoveToTargetObjectThisFrame = true;

	if(DeferredSelectionTargetObjects.Num() > 0)
	{
		FGraphPanelSelectionSet NewSelectionSet;
		for (const UObject* SelectionTarget : DeferredSelectionTargetObjects)
		{
			if (TSharedRef<SNode>* pWidget = NodeToWidgetLookup.Find(SelectionTarget))
			{
				NewSelectionSet.Add(const_cast<SelectedItemType>(SelectionTarget));
			}
		}

		if (NewSelectionSet.Num() > 0)
		{
			SelectionManager.SetSelectionSet(NewSelectionSet);
		}

		DeferredSelectionTargetObjects.Empty();

		// Do not allow movement to happen this Tick as the selected nodes may not yet have a size set (if they're newly added)
		bCanMoveToTargetObjectThisFrame = false;
	}
	
	if(DeferredMovementTargetObject)
	{
		// Since we want to move to a target object, do not zoom to extent
		bDeferredZoomToNodeExtents = false;

		if (bCanMoveToTargetObjectThisFrame && GetBoundsForNode(DeferredMovementTargetObject, ZoomTargetTopLeft, ZoomTargetBottomRight, ZoomPadding))
		{
			DeferredMovementTargetObject = nullptr;
			RequestZoomToFit();
		}
	}

	// Zoom to node extents
	if( bDeferredZoomToNodeExtents && bVisualUpdatePending == false )
	{
		bDeferredZoomToNodeExtents = false;
		ZoomPadding = NodePanelDefs::DefaultZoomPadding;
		if( GetBoundsForNodes(bDeferredZoomToSelection, ZoomTargetTopLeft, ZoomTargetBottomRight, ZoomPadding))
		{
			bDeferredZoomToSelection = false;
			RequestZoomToFit();
		}
	}

	// Handle any deferred panning
	if (bRequestDeferredPan)
	{
		bRequestDeferredPan = false;
		UpdateViewOffset(AllottedGeometry, DeferredPanPosition);
	}

	if ( !HasMouseCapture() )
	{
		bShowSoftwareCursor = false;
		bIsPanning = false;
	}

	PopulateVisibleChildren(AllottedGeometry);

	// Reset the current bookmark if the location and/or zoom level has been changed.
	const float CurZoomAmount = GetZoomAmount();
	if (CurrentBookmarkGuid.IsValid() && (OldViewOffset != GetViewOffset() || OldZoomAmount != CurZoomAmount))
	{
		CurrentBookmarkGuid.Invalidate();
	}

	OldZoomAmount = CurZoomAmount;
	OldViewOffset = GetViewOffset();

	SPanel::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );

	if (!bCanMoveToTargetObjectThisFrame)
	{
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

// The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
FReply SNodePanel::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const bool bIsLeftMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	const bool bIsRightMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
	const bool bIsMiddleMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton;
	const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown( EKeys::RightMouseButton );
	const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton );
	const bool bIsMiddleMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);

	TotalMouseDelta = 0;

	auto PerformMousePan = [this, &MyGeometry, &MouseEvent]()
	{
		if (!bAllowPanning)
		{
			return FReply::Unhandled();
		}
			
		// Cache current cursor position as zoom origin and software cursor position
		ZoomStartOffset = MyGeometry.AbsoluteToLocal( MouseEvent.GetLastScreenSpacePosition() );
		SoftwareCursorPosition = PanelCoordToGraphCoord( ZoomStartOffset );

		FReply ReplyState = FReply::Handled();
		ReplyState.CaptureMouse( SharedThis(this) );
		if (GetDefault<UGraphEditorSettings>()->bUseHighPrecisionMouseMovement)
		{
			ReplyState.UseHighPrecisionMouseMovement( SharedThis(this) );
		}

		SoftwareCursorPosition = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );

		if (!GetDefault<UGraphEditorSettings>()->bUseInterpolationWithManualPanning)
		{
			DeferredMovementTargetObject = nullptr; // clear any interpolation when you manually pan
		}
		CancelZoomToFit();

		Invalidate(EInvalidateWidgetReason::Paint);
		return ReplyState;
	};

	if ((bIsLeftMouseButtonEffecting && bIsRightMouseButtonDown)
	||  (bIsRightMouseButtonEffecting && (bIsLeftMouseButtonDown || FSlateApplication::Get().IsUsingTrackpad())))
	{
		// Starting zoom by holding LMB+RMB
		FReply ReplyState = FReply::Handled();
		ReplyState.CaptureMouse( SharedThis(this) );
		ReplyState.UseHighPrecisionMouseMovement( SharedThis(this) );

		DeferredMovementTargetObject = nullptr; // clear any interpolation when you manually zoom
		CancelZoomToFit();
		TotalMouseDeltaXY = 0;

		if (!FSlateApplication::Get().IsUsingTrackpad()) // on trackpad we don't know yet if user wants to zoom or bring up the context menu
		{
			bShowSoftwareCursor = true;
		}

		if (bIsLeftMouseButtonEffecting)
		{
			// Got here from panning mode (with RMB held) - clear panning mode, but use cached software cursor position
			const FVector2f WidgetSpaceCursorPos = GraphCoordToPanelCoord( SoftwareCursorPosition );
			ZoomStartOffset = WidgetSpaceCursorPos;
			this->bIsPanning = false;
		}
		else
		{
			// Cache current cursor position as zoom origin and software cursor position
			ZoomStartOffset = MyGeometry.AbsoluteToLocal( MouseEvent.GetLastScreenSpacePosition() );
			SoftwareCursorPosition = PanelCoordToGraphCoord( ZoomStartOffset );

			if (bIsRightMouseButtonEffecting)
			{
				// Clear things that may be set when left clicking
				if (NodeUnderMousePtr.IsValid())
				{
					OnEndNodeInteraction(NodeUnderMousePtr.Pin().ToSharedRef());
				}

				if ( Marquee.IsValid() )
				{
					auto PreviouslySelectedNodes = SelectionManager.SelectedNodes;
					ApplyMarqueeSelection(Marquee, ObjectPtrDecay(PreviouslySelectedNodes), SelectionManager.SelectedNodes);
					if (SelectionManager.SelectedNodes.Num() > 0 || PreviouslySelectedNodes.Num() > 0)
					{
						SelectionManager.OnSelectionChanged.ExecuteIfBound(ObjectPtrDecay(SelectionManager.SelectedNodes));
					}
				}

				Marquee = FMarqueeOperation();
			}
		}

		Invalidate(EInvalidateWidgetReason::Paint);
		return ReplyState;
	}
	else if (bIsRightMouseButtonEffecting && ( GetDefault<UGraphEditorSettings>()->PanningMouseButton == EGraphPanningMouseButton::Right || GetDefault<UGraphEditorSettings>()->PanningMouseButton == EGraphPanningMouseButton::Both ) )
	{
		return PerformMousePan();
	}
	else if (bIsMiddleMouseButtonEffecting && (GetDefault<UGraphEditorSettings>()->PanningMouseButton == EGraphPanningMouseButton::Middle || GetDefault<UGraphEditorSettings>()->PanningMouseButton == EGraphPanningMouseButton::Both))
	{
		return PerformMousePan();
	}
	else if ( bIsLeftMouseButtonEffecting )
	{
		// LEFT BUTTON is for selecting nodes and manipulating pins.
		if(FNodeSnappingManager* SnapManager = GetSnappingManager())
		{
			if(SnapManager->BeginDragWithSnap(MyGeometry, MouseEvent))
			{
				Invalidate(EInvalidateWidgetReason::Paint);
				return FReply::Handled().CaptureMouse( SharedThis(this) );
			}
		}
		else
		{
			FArrangedChildren ArrangedChildren(EVisibility::Visible);
			ArrangeChildNodes(MyGeometry, ArrangedChildren);
		
			const int32 NodeUnderMouseIndex = SWidget::FindChildUnderMouse( ArrangedChildren, MouseEvent );
			if ( NodeUnderMouseIndex != INDEX_NONE )
			{
				// PRESSING ON A NODE!

				// This changes selection and starts dragging it.
				const FArrangedWidget& NodeGeometry = ArrangedChildren[NodeUnderMouseIndex];
				const FVector2f MousePositionInNode = NodeGeometry.Geometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
				TSharedRef<SNode> NodeWidgetUnderMouse = StaticCastSharedRef<SNode>( NodeGeometry.Widget );

				if( NodeWidgetUnderMouse->CanBeSelected(MousePositionInNode) )
				{
					// Track the node that we're dragging; we will move it in OnMouseMove.
					this->OnBeginNodeInteraction(NodeWidgetUnderMouse, MousePositionInNode);
					Invalidate(EInvalidateWidgetReason::Paint);
					return FReply::Handled().CaptureMouse( SharedThis(this) );
				}
			}
		}

		// START MARQUEE SELECTION.
		const FVector2f GraphMousePos = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );
		Marquee.Start( GraphMousePos, FMarqueeOperation::OperationTypeFromMouseEvent(MouseEvent) );

		// If we're marquee selecting, then we're not clicking on a node!
		NodeUnderMousePtr.Reset();

		Invalidate(EInvalidateWidgetReason::Paint);
		return FReply::Handled().CaptureMouse( SharedThis(this) );
	}
	else
	{
		return FReply::Unhandled();
	}
}

// The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
FReply SNodePanel::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown( EKeys::RightMouseButton );
	const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton );
	const bool bIsMiddleMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);
	const FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();

	PastePosition = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );

	if ( this->HasMouseCapture() )
	{
		Invalidate(EInvalidateWidgetReason::Paint);

		const FVector2f CursorDelta = MouseEvent.GetCursorDelta();
		// Track how much the mouse moved since the mouse down.
		TotalMouseDelta += CursorDelta.Size();

		const bool bShouldZoom = bIsRightMouseButtonDown && (bIsLeftMouseButtonDown || bIsMiddleMouseButtonDown || ModifierKeysState.IsAltDown() || FSlateApplication::Get().IsUsingTrackpad());
		if (bShouldZoom)
		{
			FReply ReplyState = FReply::Handled();

			TotalMouseDeltaXY += CursorDelta.X + CursorDelta.Y;

			const int32 ZoomLevelDelta = FMath::RoundToInt(TotalMouseDeltaXY * NodePanelDefs::MouseZoomScaling);

			// Get rid of mouse movement that's been 'used up' by zooming
			if (ZoomLevelDelta != 0)
			{
				TotalMouseDeltaXY -= (static_cast<float>(ZoomLevelDelta) / NodePanelDefs::MouseZoomScaling);
			}

			// Perform zoom centered on the cached start offset
			ChangeZoomLevel(ZoomLevelDelta, ZoomStartOffset, MouseEvent.IsControlDown());

			this->bIsPanning = false;

			if (FSlateApplication::Get().IsUsingTrackpad() && ZoomLevelDelta != 0)
			{
				this->bIsZoomingWithTrackpad = true;
				bShowSoftwareCursor = true;
			}

			// Stop the zoom-to-fit in favor of user control
			CancelZoomToFit();

			return ReplyState;
		}
		else if (bIsRightMouseButtonDown)
		{
			FReply ReplyState = FReply::Handled();

			if( !CursorDelta.IsZero() )
			{
				bShowSoftwareCursor = true;
			}

			// Panning and mouse is outside of panel? Pasting should just go to the screen center.
			PastePosition = PanelCoordToGraphCoord( 0.5f * MyGeometry.GetLocalSize() );

			this->bIsPanning = true;
			SetViewOffset(GetViewOffset() - CursorDelta / GetZoomAmount());

			// Stop the zoom-to-fit in favor of user control
			CancelZoomToFit();

			return ReplyState;
		}
		else if (bIsMiddleMouseButtonDown)
		{
			FReply ReplyState = FReply::Handled();

			if (!CursorDelta.IsZero())
			{
				bShowSoftwareCursor = true;
			}

			// Panning and mouse is outside of panel? Pasting should just go to the screen center.
			PastePosition = PanelCoordToGraphCoord(0.5f * MyGeometry.Size);

			this->bIsPanning = true;
			SetViewOffset(GetViewOffset() - CursorDelta / GetZoomAmount());

			return ReplyState;
		}
		else if (bIsLeftMouseButtonDown)
		{
			TSharedPtr<SNode> NodeBeingDragged = NodeUnderMousePtr.Pin();

			if ( GetIsEditable() )
			{
				// Update the amount to pan panel
				UpdateViewOffset(MyGeometry, MouseEvent.GetScreenSpacePosition());

				const bool bCursorInDeadZone = TotalMouseDelta <= FSlateApplication::Get().GetDragTriggerDistance();

				if ( NodeBeingDragged.IsValid() )
				{
					if ( !bCursorInDeadZone )
					{
						// Note, NodeGrabOffset() comes from the node itself, so it's already scaled correctly.
						FVector2f AnchorNodeNewPos = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) ) - NodeGrabOffset;

						// Snap to grid
						const float SnapSize = static_cast<float>(GetSnapGridSize());
						AnchorNodeNewPos.X = SnapSize * FMath::RoundToFloat( AnchorNodeNewPos.X / SnapSize );
						AnchorNodeNewPos.Y = SnapSize * FMath::RoundToFloat( AnchorNodeNewPos.Y / SnapSize );

						// Dragging an unselected node automatically selects it.
						SelectionManager.StartDraggingNode(NodeBeingDragged->GetObjectBeingDisplayed(), MouseEvent);

						// Move all the selected nodes.
						{
							const FVector2f AnchorNodeOldPos = NodeBeingDragged->GetPosition2f();
							const FVector2f DeltaPos = AnchorNodeNewPos - AnchorNodeOldPos;

							// Perform movement in 3 passes:
							
							// 1. Gather all selected nodes positions and calculate new positions
							struct FDefferedNodePosition
							{ 
								SNode*		Node; 
								FVector2f	NewPosition; 
							};
							TArray<FDefferedNodePosition> DefferedNodesToMove;

							// 2. Deffer actual move transactions to mouse release or focus lost
							bool bStoreOriginalNodePositions = OriginalNodePositions.Num() == 0;
							for (decltype(SelectionManager.SelectedNodes)::TIterator NodeIt(SelectionManager.SelectedNodes); NodeIt; ++NodeIt)
							{
								if (TSharedRef<SNode>* pWidget = NodeToWidgetLookup.Find(*NodeIt))
								{
									SNode& Widget = pWidget->Get();
									FDefferedNodePosition NodePosition = { &Widget, Widget.GetPosition2f() + DeltaPos };
									DefferedNodesToMove.Add(NodePosition);

									if (bStoreOriginalNodePositions)
									{
										OriginalNodePositions.FindOrAdd(*pWidget) = Widget.GetPosition2f();
									}
								}
							}

							// 3. Move selected nodes to new positions
							SNode::FNodeSet NodeFilter;

							for (int32 NodeIdx = 0; NodeIdx < DefferedNodesToMove.Num(); ++NodeIdx)
							{
								DefferedNodesToMove[NodeIdx].Node->MoveTo(DefferedNodesToMove[NodeIdx].NewPosition, NodeFilter, false);
							}
						}
					}

					return FReply::Handled();
				}
			}

			if ( !NodeBeingDragged.IsValid() )
			{
				// We are marquee selecting
				const FVector2f GraphMousePos = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );
				Marquee.Rect.UpdateEndPoint(GraphMousePos);

				FindNodesAffectedByMarquee( /*out*/ Marquee.AffectedNodes );
				return FReply::Handled();
			}

			// Stop the zoom-to-fit in favor of user control
			CancelZoomToFit();
		}
	}

	return FReply::Unhandled();
}

// The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
FReply SNodePanel::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply ReplyState = FReply::Unhandled();

	const bool bIsLeftMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	const bool bIsRightMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
	const bool bIsMiddleMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton;
	const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown( EKeys::RightMouseButton );
	const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton );
	const bool bIsMiddleMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);

	// Did the user move the cursor sufficiently far, or is it in a dead zone?
	// In Dead zone     - implies actions like summoning context menus and general clicking.
	// Out of Dead Zone - implies dragging actions like moving nodes and marquee selection.
	const bool bCursorInDeadZone = TotalMouseDelta <= FSlateApplication::Get().GetDragTriggerDistance();

	// Set to true later if we need to finish with the software cursor
	bool bRemoveSoftwareCursor = false;

	if ((bIsLeftMouseButtonEffecting && bIsRightMouseButtonDown)
	||  (bIsRightMouseButtonEffecting && (bIsLeftMouseButtonDown || (FSlateApplication::Get().IsUsingTrackpad() && bIsZoomingWithTrackpad)))
	||	(bIsMiddleMouseButtonEffecting && bIsRightMouseButtonDown))
	{
		// Ending zoom by releasing LMB or RMB
		ReplyState = FReply::Handled();

		if (bIsLeftMouseButtonDown || FSlateApplication::Get().IsUsingTrackpad())
		{
			// If we released the right mouse button first, we need to cancel the software cursor display
			bRemoveSoftwareCursor = true;
			bIsZoomingWithTrackpad = false;
			ReplyState.ReleaseMouseCapture();
		}
	}
	else if ( bIsRightMouseButtonEffecting )
	{
		ReplyState = FReply::Handled().ReleaseMouseCapture();

		bRemoveSoftwareCursor = true;

		TSharedPtr<SWidget> WidgetToFocus;
		if (bCursorInDeadZone)
		{
			WidgetToFocus = OnSummonContextMenu(MyGeometry, MouseEvent);
		}

		this->bIsPanning = false;

		if (WidgetToFocus.IsValid())
		{
			ReplyState.SetUserFocus(WidgetToFocus.ToSharedRef(), EFocusCause::SetDirectly);
		}
	}
	else if ( bIsMiddleMouseButtonEffecting )
	{
		ReplyState = FReply::Handled().ReleaseMouseCapture();
		
		bRemoveSoftwareCursor = true;
		
		this->bIsPanning = false;
	}
	else if ( bIsLeftMouseButtonEffecting )
	{
		if (NodeUnderMousePtr.IsValid())
		{
			OnEndNodeInteraction(NodeUnderMousePtr.Pin().ToSharedRef());

			FinalizeNodeMovements();
			ScopedTransactionPtr.Reset();
		}
				
		if (OnHandleLeftMouseRelease(MyGeometry, MouseEvent))
		{

		}
		else if ( bCursorInDeadZone )
		{
			//@TODO: Move to selection manager
			if ( NodeUnderMousePtr.IsValid() )
			{
				// We clicked on a node!
				TSharedRef<SNode> NodeWidgetUnderMouse = NodeUnderMousePtr.Pin().ToSharedRef();

				SelectionManager.ClickedOnNode(NodeWidgetUnderMouse->GetObjectBeingDisplayed(), MouseEvent);

				// We're done interacting with this node.
				NodeUnderMousePtr.Reset();

				if (OnNodeSingleClicked.IsBound())
				{
					OnNodeSingleClicked.Execute(NodeWidgetUnderMouse->GetObjectBeingDisplayed());
				}

			}
			else if (this->HasMouseCapture())
			{
				// We clicked on the panel background
				this->SelectionManager.ClearSelectionSet();

				if(OnSpawnNodeByShortcutAtLocation.IsBound())
				{
					OnSpawnNodeByShortcutAtLocation.Execute(LastKeyChordDetected, PanelCoordToGraphCoord(  MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) ));
				}

				LastKeyChordDetected = FInputChord();
			}
		}
		else if ( Marquee.IsValid() )
		{
			auto PreviouslySelectedNodes = SelectionManager.SelectedNodes;
			ApplyMarqueeSelection(Marquee, ObjectPtrDecay(PreviouslySelectedNodes), SelectionManager.SelectedNodes);
			if (SelectionManager.SelectedNodes.Num() > 0 || PreviouslySelectedNodes.Num() > 0)
			{
				SelectionManager.OnSelectionChanged.ExecuteIfBound(ObjectPtrDecay(SelectionManager.SelectedNodes));
			}
		}

		// The existing marquee operation ended; reset it.
		Marquee = FMarqueeOperation();

		ReplyState = FReply::Handled().ReleaseMouseCapture();
	}

	if (bRemoveSoftwareCursor)
	{
		// If we released the right mouse button first, we need to cancel the software cursor display
		if ( this->HasMouseCapture() )
		{
			FSlateRect ThisPanelScreenSpaceRect = MyGeometry.GetLayoutBoundingRect();
			const FVector2f ScreenSpaceCursorPos = MyGeometry.LocalToAbsolute( GraphCoordToPanelCoord( SoftwareCursorPosition ) );

			FIntPoint BestPositionInViewport(
				FMath::RoundToInt( FMath::Clamp( ScreenSpaceCursorPos.X, ThisPanelScreenSpaceRect.Left, ThisPanelScreenSpaceRect.Right ) ),
				FMath::RoundToInt( FMath::Clamp( ScreenSpaceCursorPos.Y, ThisPanelScreenSpaceRect.Top, ThisPanelScreenSpaceRect.Bottom ) )
				);

			if (!bCursorInDeadZone)
			{
				ReplyState.SetMousePos(BestPositionInViewport);
			}
		}

		bShowSoftwareCursor = false;
	}

	return ReplyState;	
}

void SNodePanel::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	if (OriginalNodePositions.Num() > 0)
	{
		FinalizeNodeMovements();
	}

	Invalidate(EInvalidateWidgetReason::Paint);

	SPanel::OnMouseCaptureLost(CaptureLostEvent);
}

FReply SNodePanel::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// We want to zoom into this point; i.e. keep it the same fraction offset into the panel
	const FVector2f WidgetSpaceCursorPos = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
	const int32 ZoomLevelDelta = FMath::TruncToInt( FMath::RoundFromZero( MouseEvent.GetWheelDelta() ) );
	ChangeZoomLevel( ZoomLevelDelta, WidgetSpaceCursorPos, MouseEvent.IsControlDown() );

	// Stop the zoom-to-fit in favor of user control
	CancelZoomToFit();

	return FReply::Handled();
}

FCursorReply SNodePanel::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	return bShowSoftwareCursor ? 
		FCursorReply::Cursor( EMouseCursor::None ) :
		FCursorReply::Cursor( EMouseCursor::Default );
}

FReply SNodePanel::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if( GetIsEditable())
	{
		LastKeyChordDetected = FInputChord(InKeyEvent.GetKey(), EModifierKey::FromBools(InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsShiftDown(), InKeyEvent.IsCommandDown()));
	}

	return FReply::Unhandled();
}

FReply SNodePanel::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if(LastKeyChordDetected.Key == InKeyEvent.GetKey())
	{
		LastKeyChordDetected = FInputChord();
	}

	return FReply::Unhandled();
}

void SNodePanel::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	LastKeyChordDetected = FInputChord();
}

FReply SNodePanel::OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& GestureEvent )
{
	const EGestureEvent GestureType = GestureEvent.GetGestureType();
	const FVector2f& GestureDelta = GestureEvent.GetGestureDelta();
	if (GestureType == EGestureEvent::Magnify)
	{
		TotalGestureMagnify += GestureDelta.X;
		if (FMath::Abs(TotalGestureMagnify) > 0.07f)
		{
			// We want to zoom into this point; i.e. keep it the same fraction offset into the panel
			const FVector2f WidgetSpaceCursorPos = MyGeometry.AbsoluteToLocal(GestureEvent.GetScreenSpacePosition());
			const int32 ZoomLevelDelta = TotalGestureMagnify > 0.0f ? 1 : -1;
			ChangeZoomLevel(ZoomLevelDelta, WidgetSpaceCursorPos, GestureEvent.IsControlDown());
			TotalGestureMagnify = 0.0f;
		}

		// Stop the zoom-to-fit in favor of user control
		CancelZoomToFit();

		return FReply::Handled();
	}
	else if (GestureType == EGestureEvent::Scroll)
	{
		const EScrollGestureDirection DirectionSetting = GetDefault<ULevelEditorViewportSettings>()->ScrollGestureDirectionForOrthoViewports;
		const bool bUseDirectionInvertedFromDevice = DirectionSetting == EScrollGestureDirection::Natural || (DirectionSetting == EScrollGestureDirection::UseSystemSetting && GestureEvent.IsDirectionInvertedFromDevice());

		this->bIsPanning = true;
		SetViewOffset(GetViewOffset() - (bUseDirectionInvertedFromDevice == GestureEvent.IsDirectionInvertedFromDevice() ? GestureDelta : -GestureDelta) / GetZoomAmount());
		
		// Stop the zoom-to-fit in favor of user control
		CancelZoomToFit();

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SNodePanel::OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	TotalGestureMagnify = 0.0f;
	return FReply::Unhandled();
}

float SNodePanel::GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const
{
	return GetZoomAmount();
}

void SNodePanel::FindNodesAffectedByMarquee( FGraphPanelSelectionSet& OutAffectedNodes ) const
{
	OutAffectedNodes.Empty();

	FSlateRect MarqueeSlateRect = Marquee.Rect.ToSlateRect();
	const auto FindNodeAffectedByMarquee = [&OutAffectedNodes, &MarqueeSlateRect = std::as_const(MarqueeSlateRect)](const TSharedRef<SNode>& SomeNodeWidget)
	{
		const FVector2f NodePosition = SomeNodeWidget->GetPosition2f();
		const FVector2f NodeSize = SomeNodeWidget->GetDesiredSizeForMarquee2f();

		if (NodeSize.X > 0.f && NodeSize.Y > 0.f)
		{
			const FSlateRect NodeGeometryGraphSpace( NodePosition.X, NodePosition.Y, NodePosition.X + NodeSize.X, NodePosition.Y + NodeSize.Y );
			const bool bIsInMarqueeRect = FSlateRect::DoRectanglesIntersect( MarqueeSlateRect, NodeGeometryGraphSpace );
			if (bIsInMarqueeRect)
			{
				// This node is affected by the marquee rect
				OutAffectedNodes.Add(SomeNodeWidget->GetObjectBeingDisplayed());
			}
		}
	};

	if(const FNodeSnappingManager* SnapManager = GetSnappingManager())
	{
		SnapManager->ForEachNode([&FindNodeAffectedByMarquee](const TSharedRef<SNode>& Node)
		{
			FindNodeAffectedByMarquee(Node);
		});
		return;
	}

	for ( int32 NodeIndex=0; NodeIndex < Children.Num(); ++NodeIndex )
	{
		FindNodeAffectedByMarquee(Children[NodeIndex]);
	}	
}

void SNodePanel::ApplyMarqueeSelection( const FMarqueeOperation& InMarquee, const FGraphPanelSelectionSet& CurrentSelection, TSet<TObjectPtr<UObject>>& OutNewSelection )
{
	switch (InMarquee.Operation )
	{
	default:
	case FMarqueeOperation::Replace:
		{
			OutNewSelection = ObjectPtrWrap(InMarquee.AffectedNodes);
		}
		break;

	case FMarqueeOperation::Remove:
		{
			OutNewSelection = ObjectPtrWrap(CurrentSelection.Difference(InMarquee.AffectedNodes));
		}
		break;

	case FMarqueeOperation::Add:
		{
			OutNewSelection = ObjectPtrWrap(CurrentSelection.Union(InMarquee.AffectedNodes));
		}
		break; 

	case FMarqueeOperation::Invert:
		{
			// ToAdd = items in AffectedNodes that aren't in CurrentSelection (new selections)
			FGraphPanelSelectionSet ToAdd = InMarquee.AffectedNodes.Difference(CurrentSelection);
			// remove AffectedNodes that were already selected
			OutNewSelection = ObjectPtrWrap(CurrentSelection.Difference(InMarquee.AffectedNodes));
			OutNewSelection.Append(ObjectPtrWrap(ToAdd));
		}
		break;
	}
}

void SNodePanel::SelectAndCenterObject(const UObject* ObjectToSelect, bool bCenter)
{
	DeferredSelectionTargetObjects.Empty();
	DeferredSelectionTargetObjects.Add(ObjectToSelect);

	if( bCenter )
	{
		DeferredMovementTargetObject = ObjectToSelect;
	}

	CancelZoomToFit();
}

void SNodePanel::CenterObject(const UObject* ObjectToCenter)
{
	DeferredMovementTargetObject = ObjectToCenter;
	CancelZoomToFit();
}

/** Add a slot to the CanvasPanel dynamically */
void SNodePanel::AddGraphNode( const TSharedRef<SNodePanel::SNode>& NodeToAdd )
{
	Children.Add( NodeToAdd );
	NodeToWidgetLookup.Add( NodeToAdd->GetObjectBeingDisplayed(), NodeToAdd );
}

/** Remove all nodes from the panel */
void SNodePanel::RemoveAllNodes()
{
	Children.Empty();
	NodeToWidgetLookup.Empty();
	VisibleChildren.Empty();
}

void SNodePanel::PopulateVisibleChildren(const FGeometry& AllottedGeometry)
{
	bool bRequiresSort = false;
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const TSharedRef<SNode>& SomeChild = Children[ChildIndex];
		if ( !IsNodeCulled(SomeChild, AllottedGeometry) )
		{
			if(VisibleChildren.Find(SomeChild) == INDEX_NONE)
			{
				VisibleChildren.Add(SomeChild);
				bRequiresSort = true;
			}
		}
		else
		{
			if(VisibleChildren.Find(SomeChild) != INDEX_NONE)
			{
				VisibleChildren.Remove(SomeChild);
				bRequiresSort = true;
			}
			
		}
	}
	
	// Depth Sort Nodes
	if( bRequiresSort && (VisibleChildren.Num() > 0) )
	{
		struct SNodeLessThanSort
		{
			FORCEINLINE bool operator()(const TSharedRef<SNodePanel::SNode>& A, const TSharedRef<SNodePanel::SNode>& B) const { return A.Get() < B.Get(); }
		};
		VisibleChildren.Sort( SNodeLessThanSort() );
	}
}

// Is the given node being observed by a widget in this panel?
bool SNodePanel::Contains(UObject* Node) const
{
	return NodeToWidgetLookup.Find(Node) != nullptr;
}

void SNodePanel::RestoreViewSettings(const UE::Slate::FDeprecateVector2DParameter& InViewOffset, float InZoomAmount, const FGuid& InBookmarkGuid)
{
	SetViewOffset(InViewOffset);

	if (InZoomAmount <= 0.0f)
	{
		// Zoom into the graph; it's the first time it's ever been displayed
		ZoomLevel = ZoomLevels->GetDefaultZoomLevel();
		bDeferredZoomToNodeExtents = true;
	}
	else
	{
		ZoomLevel = ZoomLevels->GetNearestZoomLevel(InZoomAmount);
		bDeferredZoomToNodeExtents = false;

		CancelZoomToFit();
	}

	PostChangedZoom();

	// If we have been forced to a specific position, set the old values equal to the new ones.
	// This is so our locked window isn't forced to update according to this movement.
	OldViewOffset = GetViewOffset();
	OldZoomAmount = GetZoomAmount();

	// Update the current bookmark ID.
	CurrentBookmarkGuid = InBookmarkGuid;
}

uint32 SNodePanel::GetSnapGridSize()
{
	return GetDefault<UEditorStyleSettings>()->GridSnapSize;
}

inline float FancyMod(float Value, float Size)
{
	return ((Value >= 0) ? 0.0f : Size) + FMath::Fmod(Value, Size);
}

void SNodePanel::PaintBackgroundAsLines(const FSlateBrush* BackgroundImage, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const
{
	const bool bAntialias = GetDefault<UEditorStyleSettings>()->bAntiAliasGrid;

	const int32 RulePeriod = (int32)FAppStyle::GetFloat("Graph.Panel.GridRulePeriod");
	check(RulePeriod > 0);

	const FLinearColor GraphBackGroundImageColor(BackgroundImage->TintColor.GetSpecifiedColor());
	const FLinearColor RegularColor(GetDefault<UEditorStyleSettings>()->RegularColor);
	const FLinearColor RuleColor(GetDefault<UEditorStyleSettings>()->RuleColor);
	const FLinearColor CenterColor(GetDefault<UEditorStyleSettings>()->CenterColor);
	const float GraphSmallestGridSize = 8.0f;
	const float RawZoomFactor = GetZoomAmount();
	const float NominalGridSize = static_cast<float>(GetSnapGridSize());

	float ZoomFactor = RawZoomFactor;
	float Inflation = 1.0f;
	while (ZoomFactor*Inflation*NominalGridSize <= GraphSmallestGridSize)
	{
		Inflation *= 2.0f;
	}
	
	const float GridCellSize = NominalGridSize * ZoomFactor * Inflation;

	const float GraphSpaceGridX0 = FancyMod(GetViewOffset().X, Inflation * NominalGridSize * static_cast<float>(RulePeriod));
	const float GraphSpaceGridY0 = FancyMod(GetViewOffset().Y, Inflation * NominalGridSize * static_cast<float>(RulePeriod));

	float ImageOffsetX = GraphSpaceGridX0 * -ZoomFactor;
	float ImageOffsetY = GraphSpaceGridY0 * -ZoomFactor;

	const FVector2f ZeroSpace = GraphCoordToPanelCoord(FVector2f::ZeroVector);
	
	// Fill the background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		DrawLayerId,
		AllottedGeometry.ToPaintGeometry(),
		BackgroundImage,
		ESlateDrawEffect::None,
		GraphBackGroundImageColor
	);

	TArray<FVector2f> LinePoints;
	new (LinePoints) FVector2f(0.0f, 0.0f);
	new (LinePoints) FVector2f(0.0f, 0.0f);

	//If we want to use grid then show grid, otherwise don't render the grid
	if (GetDefault<UEditorStyleSettings>()->bUseGrid == true){

		// Horizontal bars
		for (int32 GridIndex = 0; ImageOffsetY < AllottedGeometry.GetLocalSize().Y; ImageOffsetY += GridCellSize, ++GridIndex)
		{
			if (ImageOffsetY >= 0.0f)
			{
				const bool bIsRuleLine = (GridIndex % RulePeriod) == 0;
				const int32 Layer = bIsRuleLine ? (DrawLayerId + 1) : DrawLayerId;

				const FLinearColor* Color = bIsRuleLine ? &RuleColor : &RegularColor;
				if (FMath::IsNearlyEqual(ZeroSpace.Y, ImageOffsetY, 1.0f))
				{
					Color = &CenterColor;
				}

				LinePoints[0] = FVector2f(0.0f, ImageOffsetY);
				LinePoints[1] = FVector2f(AllottedGeometry.GetLocalSize().X, ImageOffsetY);

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					Layer,
					AllottedGeometry.ToPaintGeometry(),
					LinePoints,
					ESlateDrawEffect::None,
					*Color,
					bAntialias);
			}
		}

		// Vertical bars
		for (int32 GridIndex = 0; ImageOffsetX < AllottedGeometry.GetLocalSize().X; ImageOffsetX += GridCellSize, ++GridIndex)
		{
			if (ImageOffsetX >= 0.0f)
			{
				const bool bIsRuleLine = (GridIndex % RulePeriod) == 0;
				const int32 Layer = bIsRuleLine ? (DrawLayerId + 1) : DrawLayerId;

				const FLinearColor* Color = bIsRuleLine ? &RuleColor : &RegularColor;
				if (FMath::IsNearlyEqual(ZeroSpace.X, ImageOffsetX, 1.0f))
				{
					Color = &CenterColor;
				}

				LinePoints[0] = FVector2f(ImageOffsetX, 0.0f);
				LinePoints[1] = FVector2f(ImageOffsetX, AllottedGeometry.GetLocalSize().Y);

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					Layer,
					AllottedGeometry.ToPaintGeometry(),
					LinePoints,
					ESlateDrawEffect::None,
					*Color,
					bAntialias);
			}
		}
	}
	DrawLayerId += 2;
}

void SNodePanel::PaintSurroundSunkenShadow(const FSlateBrush* ShadowImage, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId) const
{
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		DrawLayerId,
		AllottedGeometry.ToPaintGeometry(),
		ShadowImage
	);
}

void SNodePanel::PaintMarquee(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId) const
{
	if (Marquee.IsValid())
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			DrawLayerId,
			AllottedGeometry.ToPaintGeometry( Marquee.Rect.GetSize()*GetZoomAmount(), FSlateLayoutTransform(GraphCoordToPanelCoord(Marquee.Rect.GetUpperLeft())) ),
			FAppStyle::GetBrush(TEXT("MarqueeSelection"))
		);
	}
}

void SNodePanel::PaintSoftwareCursor(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId) const
{
	if( !bShowSoftwareCursor )
	{
		return;
	}

	// Get appropriate software cursor, depending on whether we're panning or zooming
	const FSlateBrush* Brush = FAppStyle::GetBrush(bIsPanning ? TEXT("SoftwareCursor_Grab") : TEXT("SoftwareCursor_UpDown"));

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		DrawLayerId,
		AllottedGeometry.ToPaintGeometry( Brush->ImageSize, FSlateLayoutTransform(GraphCoordToPanelCoord( SoftwareCursorPosition ) - ( Brush->ImageSize / 2 )) ),
		Brush
	);
}

void SNodePanel::PaintComment(const FString& CommentText, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId, const FLinearColor& CommentTinting, float& HeightAboveNode, const FWidgetStyle& InWidgetStyle) const
{
	//@TODO: Ideally we don't need to grab these resources for every comment being drawn
	// Get resources/settings for drawing comment bubbles
	const FSlateBrush* CommentCalloutArrow = FAppStyle::GetBrush(TEXT("Graph.Node.CommentArrow"));
	const FSlateBrush* CommentCalloutBubble = FAppStyle::GetBrush(TEXT("Graph.Node.CommentBubble"));
	const FSlateFontInfo CommentFont = FAppStyle::GetFontStyle( TEXT("Graph.Node.CommentFont") );
	const FSlateColor CommentTextColor = FAppStyle::GetColor( TEXT("Graph.Node.Comment.TextColor") );
	const FVector2f CommentBubblePadding = UE::Slate::CastToVector2f(FAppStyle::GetVector( TEXT("Graph.Node.Comment.BubblePadding") ));

	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FVector2f CommentTextSize = FontMeasureService->Measure( CommentText, CommentFont ) + (CommentBubblePadding * 2);

	const float PositionBias = HeightAboveNode;
	HeightAboveNode += CommentTextSize.Y + 8.0f;

	const FVector2f CommentBubbleOffset = FVector2f(0, -(CommentTextSize.Y + CommentCalloutArrow->ImageSize.Y) - PositionBias);
	const FVector2f CommentBubbleArrowOffset = FVector2f( CommentCalloutArrow->ImageSize.X, -CommentCalloutArrow->ImageSize.Y - PositionBias);

	// We want the watch window comment bubbles to draw on top of the node so that they are not obscured
	// and on top of the big blueprint debugging arrow
	static constexpr int32 CommentBubbleLayer = 100;

	const int32 DrawingLayer = DrawLayerId + CommentBubbleLayer;

	// Draw a comment bubble
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		DrawingLayer - 1,	// Put the background 1 layer behind, so the text is on top
		AllottedGeometry.ToPaintGeometry(CommentTextSize, FSlateLayoutTransform(CommentBubbleOffset)),
		CommentCalloutBubble,
		ESlateDrawEffect::None,
		CommentTinting
		);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		DrawingLayer - 1,	// Put the background 1 layer behind, so the text is on top
		AllottedGeometry.ToPaintGeometry( CommentCalloutArrow->ImageSize, FSlateLayoutTransform(CommentBubbleArrowOffset) ),
		CommentCalloutArrow,
		ESlateDrawEffect::None,
		CommentTinting
		);

	// Draw the comment text itself
	FSlateDrawElement::MakeText(
		OutDrawElements,
		DrawingLayer,
		AllottedGeometry.ToPaintGeometry( CommentTextSize, FSlateLayoutTransform(CommentBubbleOffset + CommentBubblePadding) ),
		CommentText,
		CommentFont,
		ESlateDrawEffect::None,
		CommentTextColor.GetColor( InWidgetStyle )
		);

}

bool SNodePanel::IsNodeCulled(const TSharedRef<SNode>& Node, const FGeometry& AllottedGeometry) const
{
	if ( Node->ShouldAllowCulling() )
	{
		const FVector2f MinClipArea = AllottedGeometry.GetDrawSize() * -NodePanelDefs::GuardBandArea;
		const FVector2f MaxClipArea = AllottedGeometry.GetDrawSize() * ( 1.f + NodePanelDefs::GuardBandArea );
		const FVector2f NodeTopLeft = GraphCoordToPanelCoord( Node->GetPosition2f() );
		const FVector2f NodeBottomRight = GraphCoordToPanelCoord( Node->GetPosition2f() + Node->GetDesiredSize() );

		return 
			NodeBottomRight.X < MinClipArea.X ||
			NodeBottomRight.Y < MinClipArea.Y ||
			NodeTopLeft.X > MaxClipArea.X ||
			NodeTopLeft.Y > MaxClipArea.Y;
	}
	else
	{
		return false;
	}

}
bool SNodePanel::GetBoundsForNode(const UObject* InNode, /*out*/ FVector2D& MinCorner, /*out*/ FVector2D& MaxCorner, float Padding) const
{
	FVector2f TempMinCorner, TempMaxCorner;
	bool bRetVal = SNodePanel::GetBoundsForNode(InNode, TempMinCorner, TempMaxCorner, Padding);
	MinCorner = FVector2D(TempMinCorner);
	MaxCorner = FVector2D(TempMaxCorner);

	return bRetVal;
}

bool SNodePanel::GetBoundsForNode(const UObject* InNode, /*out*/ FVector2f& MinCorner, /*out*/ FVector2f& MaxCorner, float Padding) const
{
	MinCorner = FVector2f(MAX_FLT, MAX_FLT);
	MaxCorner = FVector2f(-MAX_FLT, -MAX_FLT);

	bool bValid = false;

	const TSharedRef<SNode>* pWidget = (InNode) ? NodeToWidgetLookup.Find(InNode) : nullptr;
	if (pWidget)
	{
		const SNode& Widget = pWidget->Get();
		const FVector2f Lower = Widget.GetPosition2f();
		const FVector2f Upper = Lower + Widget.GetDesiredSize();

		MinCorner.X = FMath::Min(MinCorner.X, Lower.X);
		MinCorner.Y = FMath::Min(MinCorner.Y, Lower.Y);
		MaxCorner.X = FMath::Max(MaxCorner.X, Upper.X);
		MaxCorner.Y = FMath::Max(MaxCorner.Y, Upper.Y);

		bValid = true;
	}

	if (bValid)
	{
		MinCorner.X -= Padding;
		MinCorner.Y -= Padding;
		MaxCorner.X += Padding;
		MaxCorner.Y += Padding;
	}

	return bValid;
}
bool SNodePanel::GetBoundsForNodes(bool bSelectionSetOnly, FVector2D& MinCorner, FVector2D& MaxCorner, float Padding) const
{
	FVector2f TempMinCorner, TempMaxCorner;
	bool bRetVal = SNodePanel::GetBoundsForNodes(bSelectionSetOnly, TempMinCorner, TempMaxCorner, Padding);
	MinCorner = FVector2D(TempMinCorner);
	MaxCorner = FVector2D(TempMaxCorner);

	return bRetVal;
}

bool SNodePanel::GetBoundsForNodes(bool bSelectionSetOnly, FVector2f& MinCorner, FVector2f& MaxCorner, float Padding) const
{
	MinCorner = FVector2f(MAX_FLT, MAX_FLT);
	MaxCorner = FVector2f(-MAX_FLT, -MAX_FLT);

	bool bValid = false;

	if (bSelectionSetOnly && (SelectionManager.GetSelectedNodes().Num() > 0))
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectionManager.GetSelectedNodes()); NodeIt; ++NodeIt)
		{
			const TSharedRef<SNode>* pWidget = NodeToWidgetLookup.Find(*NodeIt);
			if (pWidget != nullptr)
			{
				SNode& Widget = pWidget->Get();
				const FVector2f Lower = Widget.GetPosition2f();
				const FVector2f Upper = Lower + Widget.GetDesiredSize();

				MinCorner.X = FMath::Min(MinCorner.X, Lower.X);
				MinCorner.Y = FMath::Min(MinCorner.Y, Lower.Y);
				MaxCorner.X = FMath::Max(MaxCorner.X, Upper.X);
				MaxCorner.Y = FMath::Max(MaxCorner.Y, Upper.Y);
				bValid = true;
			}
		}
	}
	else
	{
		bValid = NodeToWidgetLookup.Num() > 0;
		for (auto NodeIt = NodeToWidgetLookup.CreateConstIterator(); NodeIt; ++NodeIt)
		{
			SNode& Widget = NodeIt.Value().Get();

			const FVector2f Lower = Widget.GetPosition2f();
			const FVector2f Upper = Lower + Widget.GetDesiredSize();

			MinCorner.X = FMath::Min(MinCorner.X, Lower.X);
			MinCorner.Y = FMath::Min(MinCorner.Y, Lower.Y);
			MaxCorner.X = FMath::Max(MaxCorner.X, Upper.X);
			MaxCorner.Y = FMath::Max(MaxCorner.Y, Upper.Y);
		}
	}

	if (bValid)
	{
		MinCorner.X -= Padding;
		MinCorner.Y -= Padding;
		MaxCorner.X += Padding;
		MaxCorner.Y += Padding;
	}

	return bValid;
}

bool SNodePanel::ScrollToLocation(const FGeometry& MyGeometry, FVector2f DesiredCenterPosition, const float InDeltaTime)
{
	const FVector2f HalfOFScreenInGraphSpace = 0.5f * MyGeometry.GetLocalSize() / GetZoomAmount();
	FVector2f CurrentPosition = GetViewOffset() + HalfOFScreenInGraphSpace;

	FVector2f NewPosition = FMath::Vector2DInterpTo(CurrentPosition, DesiredCenterPosition, InDeltaTime, 10.f);
	SetViewOffset(NewPosition - HalfOFScreenInGraphSpace);

	// If within 1 pixel of target, stop interpolating
	return ((NewPosition - DesiredCenterPosition).SizeSquared() < 1.f);
}

bool SNodePanel::ZoomToLocation(const FVector2f& CurrentSizeWithoutZoom, const FVector2f& InDesiredSize, bool bDoneScrolling)
{
	if (!bAllowZoom)
	{
		return false;
	}

	if (bAllowContinousZoomInterpolation && ZoomLevelGraphFade.IsPlaying())
	{
		return false;
	}

	const int32 DefaultZoomLevel = ZoomLevels->GetDefaultZoomLevel();
	const int32 NumZoomLevels = ZoomLevels->GetNumZoomLevels();
	int32 DesiredZoom = DefaultZoomLevel;

	// Find lowest zoom level that will display all nodes
	for (int32 Zoom = 0; Zoom < DefaultZoomLevel; ++Zoom)
	{
		const FVector2f SizeWithZoom = CurrentSizeWithoutZoom / ZoomLevels->GetZoomAmount(Zoom);
		const FVector2f LeftOverSize = SizeWithZoom - InDesiredSize;
		
		if ((InDesiredSize.X > SizeWithZoom.X) || (InDesiredSize.Y > SizeWithZoom.Y))
		{
			// Use the previous zoom level, this one is too tight
			DesiredZoom = FMath::Max(0, Zoom - 1);
			break; 
		}
	}

	if (DesiredZoom != ZoomLevel)
	{
		if (bAllowContinousZoomInterpolation)
		{
			// Animate to it
			PreviousZoomLevel = ZoomLevel;
			ZoomLevel = FMath::Clamp(DesiredZoom, 0, NumZoomLevels-1);
			ZoomLevelGraphFade.Play( this->AsShared() );
			return false;
		}
		else
		{
			// Do it instantly, either first or last
			if (DesiredZoom < ZoomLevel)
			{
				// Zooming out; do it instantly
				ZoomLevel = PreviousZoomLevel = DesiredZoom;
				ZoomLevelFade.Play( this->AsShared() );
			}
			else
			{
				// Zooming in; do it last
				if (bDoneScrolling)
				{
					ZoomLevel = PreviousZoomLevel = DesiredZoom;
					ZoomLevelFade.Play( this->AsShared() );
				}
			}
		}

		PostChangedZoom();
	}

	return true;
}

void SNodePanel::ZoomToFit(bool bOnlySelection)
{
	if (!bAllowZoom)
	{
		return;
	}

	bDeferredZoomToNodeExtents = true;
	bDeferredZoomToSelection = bOnlySelection;
	ZoomPadding = NodePanelDefs::DefaultZoomPadding;
}

void SNodePanel::ZoomToTarget(const FVector2f& TopLeft, const FVector2f& BottomRight)
{
	if (!bAllowZoom)
	{
		return;
	}
	
	bDeferredZoomToNodeExtents = false;

	ZoomTargetTopLeft = TopLeft;
	ZoomTargetBottomRight = BottomRight;

	RequestZoomToFit();
}

void SNodePanel::ChangeZoomLevel(int32 ZoomLevelDelta, const FVector2f& WidgetSpaceZoomOrigin, bool bOverrideZoomLimiting)
{
	if (!bAllowZoom)
	{
		return;
	}
	
	// We want to zoom into this point; i.e. keep it the same fraction offset into the panel
	const FVector2f PointToMaintainGraphSpace = PanelCoordToGraphCoord( WidgetSpaceZoomOrigin );

	const int32 DefaultZoomLevel = ZoomLevels->GetDefaultZoomLevel();
	const int32 NumZoomLevels = ZoomLevels->GetNumZoomLevels();
	EGraphZoomLimitHandling ZoomLimitHandling = ZoomLevels->GetZoomLimitHandling();

	bool bAllowLimitBreak = (bOverrideZoomLimiting && ZoomLimitHandling == EGraphZoomLimitHandling::Default) || ZoomLimitHandling == EGraphZoomLimitHandling::AllowLimitBreak;
	const bool bAllowFullZoomRange =
		// To zoom in past 1:1 the user must press control
		(ZoomLevel == DefaultZoomLevel && ZoomLevelDelta > 0 && bAllowLimitBreak)
			 ||
		// If they are already zoomed in past 1:1, user may zoom freely
		(ZoomLevel > DefaultZoomLevel);

	const int32 OldZoomLevel = ZoomLevel;

	if ( bAllowFullZoomRange )
	{
		ZoomLevel = FMath::Clamp( ZoomLevel + ZoomLevelDelta, 0, NumZoomLevels-1 );
	}
	else
	{
		// Without control, we do not allow zooming in past 1:1.
		ZoomLevel = FMath::Clamp( ZoomLevel + ZoomLevelDelta, 0, DefaultZoomLevel );
	}

	if (OldZoomLevel != ZoomLevel)
	{
		PostChangedZoom();
	}

	// Note: This happens even when maxed out at a stop; so the user sees the animation and knows that they're at max zoom in/out
	ZoomLevelFade.Play( this->AsShared() );

	// Re-center the screen so that it feels like zooming around the cursor.
	this->SetViewOffset(PointToMaintainGraphSpace - WidgetSpaceZoomOrigin / GetZoomAmount());
}

void SNodePanel::SetZoomLevel(int32 InZoomLevel)
{
	if (!ZoomLevels)
	{
		return;
	}
	if (InZoomLevel >= 0 && InZoomLevel < ZoomLevels->GetNumZoomLevels())
	{
		ZoomLevel = InZoomLevel;
		PostChangedZoom();
	}
}

bool SNodePanel::GetBoundsForSelectedNodes( class FSlateRect& Rect, float Padding )
{
	bool Result = false;
	if(SelectionManager.GetSelectedNodes().Num()  > 0)
	{
		FVector2f MinCorner, MaxCorner;
		Result =  GetBoundsForNodes(true, MinCorner, MaxCorner,Padding);

		Rect = FSlateRect(MinCorner.X, MinCorner.Y, MaxCorner.X, MaxCorner.Y);
	}
	return Result;
}

UE::Slate::FDeprecateVector2DResult SNodePanel::GetPastePosition() const
{
	return PastePosition;
}

bool SNodePanel::HasDeferredObjectFocus() const
{
	return DeferredMovementTargetObject != nullptr;
}

bool SNodePanel::HasDeferredZoomDestination() const
{
	return HasDeferredObjectFocus() || bDeferredZoomToSelection || bDeferredZoomToNodeExtents;
}

void SNodePanel::FinalizeNodeMovements()
{
	if(FNodeSnappingManager* SnapManager = GetSnappingManager())
	{
		if(SnapManager->FinalizeNodeMovements())
		{
			return;
		}
	}

	// Process moved nodes on focus lost
	if (OriginalNodePositions.Num() > 0)
	{
		// Build up all the current positions
		TMap<SNode*, FVector2f> CurrentNodePositions;

		for (decltype(SelectionManager.SelectedNodes)::TIterator NodeIt(SelectionManager.SelectedNodes); NodeIt; ++NodeIt)
		{
			TSharedRef<SNode>* pWidget = NodeToWidgetLookup.Find(*NodeIt);
			if (pWidget != nullptr)
			{
				SNode& Widget = pWidget->Get();
				CurrentNodePositions.FindOrAdd(&Widget) = Widget.GetPosition2f();
			}
		}

		// Move all the nodes back to their original position before we start the transaction
		SNode::FNodeSet OriginalNodeFilter;

		for (TPair<TWeakPtr<SNode>, FVector2f>& OriginalNodePosition : OriginalNodePositions)
		{
			if (OriginalNodePosition.Key.IsValid())
			{
				OriginalNodePosition.Key.Pin()->MoveTo(OriginalNodePosition.Value, OriginalNodeFilter, false);
			}
		}

		OriginalNodePositions.Reset();

		if (CurrentNodePositions.Num() > 0)
		{
			FScopedTransaction NodeMoveTransaction(
				CurrentNodePositions.Num() > 1 
				? NSLOCTEXT("GraphEditor", "MoveNodesAction", "Move Nodes")
				: NSLOCTEXT("GraphEditor", "MoveNodeAction", "Move Node"));

			// Move all the nodes back to their current position but on on the undo stack
			SNode::FNodeSet CurrentNodeFilter;

			for (TPair<SNode*, FVector2f>& CurrentNodePosition : CurrentNodePositions)
			{
				CurrentNodePosition.Key->MoveTo(CurrentNodePosition.Value, CurrentNodeFilter, true);
			}
		}
	}
}

void SNodePanel::PostChangedZoom()
{
	CurrentLOD = ZoomLevels->GetLOD(ZoomLevel);

	// Invalidate the current bookmark.
	CurrentBookmarkGuid.Invalidate();
	Invalidate(EInvalidateWidgetReason::Prepass);
}

void SNodePanel::RequestZoomToFit()
{
	if (!ActiveTimerHandle.IsValid())
	{
		ActiveTimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SNodePanel::HandleZoomToFit));
	}
}

void SNodePanel::CancelZoomToFit()
{
	if (ActiveTimerHandle.IsValid())
	{
		// Reset Zoom destination
		ZoomPadding = NodePanelDefs::DefaultZoomPadding;
		ZoomTargetTopLeft = FVector2f::ZeroVector;
		ZoomTargetBottomRight = FVector2f::ZeroVector;
		UnRegisterActiveTimer(ActiveTimerHandle.Pin().ToSharedRef());
	}
}

void SNodePanel::OnSelectionChanged(const TSet<UObject*>& Objects)
{
	Invalidate(EInvalidateWidgetReason::Paint);
}

bool SNodePanel::HasMoved() const
{
	return (!FMath::IsNearlyEqual(GetZoomAmount(), OldZoomAmount) || !GetViewOffset().Equals(OldViewOffset, SMALL_NUMBER));
}
