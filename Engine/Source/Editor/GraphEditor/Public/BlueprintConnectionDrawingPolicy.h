// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Layout/ArrangedWidget.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class FArrangedChildren;
class FArrangedWidget;
class FSlateRect;
class FSlateWindowElementList;
class SGraphPin;
class SWidget;
class UBlueprint;
class UEdGraph;
class UEdGraphPin;

/////////////////////////////////////////////////////
// FKismetConnectionDrawingPolicy

// This class draws the connections for an UEdGraph using a K2-based schema
class FKismetConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
protected:
	// Times for one execution pair within the current graph
	struct FTimePair
	{
		double PredExecTime; // time that the node with the associated exec-out was executed
		double ThisExecTime; // time that the following node was executed

		FTimePair()
			: PredExecTime(0.0)
			, ThisExecTime(0.0)
		{
		}
	};
	/** 
	 * Map of exec-pins to a pair of execution times (holds timing info for the 
	 * owning-node, paired with a time of execution for the linked node)... only 
	 * one timing-pair per pin because exec outputs can only connect to one node
	 */
	typedef TMap<UEdGraphPin*, FTimePair> FExecPairingMap;

	// Map of executed nodes, and the execution pins that lead to them being ran
	TMap<UEdGraphNode*, FExecPairingMap> PredecessorPins;

	UEdGraph* GraphObj;
	double CurrentTime;
	double LatestTimeDiscovered;

	FLinearColor AttackColor;
	FLinearColor SustainColor;
	FLinearColor ReleaseColor;

	float AttackWireThickness;
	float SustainWireThickness;
	float ReleaseWireThickness;
	float DefaultDataWireThickness;
	float DefaultExecutionWireThickness;

	float TracePositionBonusPeriod;
	float TracePositionExponent;
	float AttackHoldPeriod;
	float DecayPeriod;
	float DecayExponent;
	float SustainHoldPeriod;
	float ReleasePeriod;
	float ReleaseExponent;

	// Each time a knot is encountered, input geometry is compared to output geometry to see if the pins on the knot need to be reversed
	TMap<class UK2Node_Knot*, bool> KnotToReversedDirectionMap;

protected:
	// Should this wire be treated as an execution pin, querying the execution trace for connected nodes to draw it differently?
	UE_API virtual bool TreatWireAsExecutionPin(UEdGraphPin* InputPin, UEdGraphPin* OutputPin) const;

	// Determines the target Blueprint for querying the active debugging context.
	UE_API UBlueprint* GetTargetBlueprint() const;

	// Determines if the given Blueprint context has a valid object for debugging.
	UE_API bool CanBuildRoadmap(UBlueprint* TargetBP) const;

public:
	UE_API FKismetConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	UE_API virtual void BuildExecutionRoadmap();

	UE_API virtual bool CanBuildRoadmap() const;

	UE_API void CalculateEnvelopeAlphas(double ExecutionTime, /*out*/ float& AttackAlpha, /*out*/ float& SustainAlpha) const;

	// FConnectionDrawingPolicy interface
	UE_API virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	UE_API virtual void Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& PinGeometries, FArrangedChildren& ArrangedNodes) override;
	// End of FConnectionDrawingPolicy interface

	UE_API virtual void DetermineStyleOfExecWire(float& Thickness, FLinearColor& WireColor, bool& bDrawBubbles, const FTimePair& Times);

	UE_API virtual void SetIncompatiblePinDrawState(const TSharedPtr<SGraphPin>& StartPin, const TSet< TSharedRef<SWidget> >& VisiblePins) override;
	UE_API virtual void ResetIncompatiblePinDrawState(const TSet< TSharedRef<SWidget> >& VisiblePins) override;

protected: 
	/**
	* Checks that the specified nodes are connected in the graph directly by an 
	* exec pin.
	* 
	* @param  InputNode	The node whose exec output(s) should be connected to OutputNode.
	* @param  OutputNode	The node whose exec input should be coming from the InputNode.
	* @return True if the two nodes are connected in the graph, otherwise false.
	*/
	UE_API bool AreNodesGraphicallySequential(UEdGraphNode* InputNode, UEdGraphNode* OutputNode) const;

	/**
	 * With the introduction of "reroute" (UK2Node_Knot) nodes, we have to 
	 * backtrace to find a node that was actually executed before the one in 
	 * question (knot nodes are removed at compile time, and therefore never
	 * executed).
	 *
	 * This function recursively searches for any pins feeding the OutputPin (or 
	 * the OutputPin itself); it looks for a pin on a non-reroute node (one that
	 * is listed in NodeExecutionList).
	 * 
	 * @param  OutputPin			An exec pin, leading into the node in question (an output pin on some other node).
	 * @param  NodeExecutionList	A list of pins that feed the execution of a specific node.
	 * @return Null if OutputPin did not cause the invocation of the node in question, otherwise a valid time pair denoting when the node was executed.
	 */
	UE_API FTimePair const* BackTraceExecPath(UEdGraphPin const* const OutputPin, FExecPairingMap const* const NodeExecutionList);

	UE_API bool ShouldChangeTangentForKnot(class UK2Node_Knot* Knot);
	UE_API bool GetAverageConnectedPosition(class UK2Node_Knot* Knot, EEdGraphPinDirection Direction, FVector2f& OutPos) const;
	UE_API bool FindPinCenter(UEdGraphPin* Pin, FVector2f& OutCenter) const;
	static UE_API UObject* GetObjectBeingDebugged(UBlueprint* TargetBP);
};

#undef UE_API
