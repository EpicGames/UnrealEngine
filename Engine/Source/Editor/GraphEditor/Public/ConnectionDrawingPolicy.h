// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Framework/MarqueeRect.h"
#include "GraphEditorSettings.h"
#include "GraphSplineOverlapResult.h"
#include "HAL/Platform.h"
#include "Layout/ArrangedWidget.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class FArrangedChildren;
class FArrangedWidget;
class FSlateRect;
class FSlateWindowElementList;
class SGraphPin;
class SWidget;
class UGraphEditorSettings;
struct FGeometry;
struct FSlateBrush;
template <class T> class FInterpCurve;

/////////////////////////////////////////////////////

GRAPHEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogConnectionDrawingPolicy, Log, All);

/////////////////////////////////////////////////////
// FGeometryHelper

class FGeometryHelper
{
public:
	static UE_API UE::Slate::FDeprecateVector2DResult VerticalMiddleLeftOf(const FGeometry& SomeGeometry);
	static UE_API UE::Slate::FDeprecateVector2DResult VerticalMiddleRightOf(const FGeometry& SomeGeometry);

	static UE_API UE::Slate::FDeprecateVector2DResult CenterOf(const FGeometry& SomeGeometry);
	UE_DEPRECATED(5.6, "Use Array of FVector2f instead of FVector2D for slate positions")
	static UE_API void ConvertToPoints(const FGeometry& Geom, TArray<FVector2D>& Points);
	static UE_API void ConvertToPoints(const FGeometry& Geom, TArray<FVector2f>& Points);

	/** Find the point on line segment from LineStart to LineEnd which is closest to Point */
	static UE_API UE::Slate::FDeprecateVector2DResult FindClosestPointOnLine(const UE::Slate::FDeprecateVector2DParameter& LineStart, const UE::Slate::FDeprecateVector2DParameter& LineEnd, const UE::Slate::FDeprecateVector2DParameter& TestPoint);

	static UE_API UE::Slate::FDeprecateVector2DResult FindClosestPointOnGeom(const FGeometry& Geom, const UE::Slate::FDeprecateVector2DParameter& TestPoint);
};

/////////////////////////////////////////////////////
// FConnectionParameters

struct FConnectionParams
{
	FLinearColor WireColor;
	UEdGraphPin* AssociatedPin1;
	UEdGraphPin* AssociatedPin2;

	float WireThickness;
	bool bDrawBubbles;
	bool bUserFlag1;
	bool bUserFlag2;

	EEdGraphPinDirection StartDirection;
	EEdGraphPinDirection EndDirection;

	FDeprecateSlateVector2D StartTangent;
	FDeprecateSlateVector2D EndTangent;

	FConnectionParams()
		: WireColor(FLinearColor::White)
		, AssociatedPin1(nullptr)
		, AssociatedPin2(nullptr)
		, WireThickness(1.5f)
		, bDrawBubbles(false)
		, bUserFlag1(false)
		, bUserFlag2(false)
		, StartDirection(EGPD_Output)
		, EndDirection(EGPD_Input)
		, StartTangent(FVector2f::ZeroVector)
		, EndTangent(FVector2f::ZeroVector)
	{
	}
};

#define END_DEPRECATED_VIRTUAL_FN

/////////////////////////////////////////////////////
// FConnectionDrawingPolicy

// This class draws the connections for an UEdGraph composed of pins and nodes
class FConnectionDrawingPolicy
{
protected:
	int32 WireLayerID;
	int32 ArrowLayerID;

	const FSlateBrush* ArrowImage;
	const FSlateBrush* MidpointImage;
	const FSlateBrush* BubbleImage;

	const UGraphEditorSettings* Settings;

public:
	FDeprecateSlateVector2D ArrowRadius;
	FDeprecateSlateVector2D MidpointRadius;

	FGraphSplineOverlapResult SplineOverlapResult;
	TArray<TPair<FEdGraphPinReference, FEdGraphPinReference>> ConnectionsIntersectingSliceLine;

	/** Handle for a currently relinked connection. */
	struct FRelinkConnection
	{
		UEdGraphPin* SourcePin;
		UEdGraphPin* TargetPin;
	};

protected:
	float ZoomFactor; 
	float HoverDeemphasisDarkFraction;
	float SliceDeemphasisAlphaMultiplier;
	const FSlateRect ClippingRect;
	FSlateWindowElementList& DrawElementsList;
	TMap< UEdGraphPin*, TSharedPtr<SGraphPin> > PinToPinWidgetMap;
	TSet< FEdGraphPinReference > HoveredPins;
	TMap<TSharedRef<SWidget>, FArrangedWidget>* PinGeometries;
	double LastHoverTimeEvent;
	UE_DEPRECATED(5.8, "Use AbsoluteMousePosition instead, no other change of code needed. The name of this variable was wrong as SetMousePosition was passed the result of 'AllottedGeometry.LocalToAbsolute'.")
	FDeprecateSlateVector2D LocalMousePosition;
	FVector2f AbsoluteMousePosition;

	/** List of currently relinked connections. */
	TArray<FRelinkConnection> RelinkConnections;

	/** Selected nodes in the graph panel. */
	TArray<UEdGraphNode*> SelectedGraphNodes;
	
	FMarqueeRect SliceLine;
public:
	virtual ~FConnectionDrawingPolicy() = default;
	
	UE_API FConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements);

	// Update the drawing policy with the set of hovered pins (which can be empty)
	UE_API void SetHoveredPins(const TSet< FEdGraphPinReference >& InHoveredPins, const TArray< TSharedPtr<SGraphPin> >& OverridePins, double HoverTime);

	UE_DEPRECATED(5.8, "Use SetAbsoluteMousePosition instead, no other change of code needed.")
	UE_API void SetMousePosition(const UE::Slate::FDeprecateVector2DParameter& InMousePos);
	UE_API void SetAbsoluteMousePosition(FVector2f InAbsoluteMousePos);

	// If a (valid) slice line has been set, we'll intersect any drawn splines via DrawConnection() with that line,
	// and add any intersecting pin pairs to the public ConnectionsIntersectingSliceLine array. 
	UE_API void SetSliceLine(const FMarqueeRect& InLine);

	// Update the drawing policy with the marked pin (which may not be valid)
	UE_API void SetMarkedPin(TWeakPtr<SGraphPin> InMarkedPin);

	// Set the selected nodes from the graph panel.
	void SetSelectedNodes(const TArray<UEdGraphNode*>& InSelectedNodes) { SelectedGraphNodes = InSelectedNodes; }

	// Set the list of currently relinked connections.
	void SetRelinkConnections(const TArray<FRelinkConnection>& Connections) { RelinkConnections = Connections; }

	static UE_API float MakeSplineReparamTable(const UE::Slate::FDeprecateVector2DParameter& P0, const UE::Slate::FDeprecateVector2DParameter& P0Tangent, const UE::Slate::FDeprecateVector2DParameter& P1, const UE::Slate::FDeprecateVector2DParameter& P1Tangent, FInterpCurve<float>& OutReparamTable);

	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual void DrawSplineWithArrow(const FVector2D& StartPoint, const FVector2D& EndPoint, const FConnectionParams& Params) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual void DrawSplineWithArrow(const FVector2f& StartPoint, const FVector2f& EndPoint, const FConnectionParams& Params);
	
	UE_API virtual void DrawSplineWithArrow(const FGeometry& StartGeom, const FGeometry& EndGeom, const FConnectionParams& Params);

	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual FVector2D ComputeSplineTangent(const FVector2D& Start, const FVector2D& End) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual FVector2f ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const;


	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual void DrawConnection(int32 LayerId, const FVector2D& Start, const FVector2D& End, const FConnectionParams& Params) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual void DrawConnection(int32 LayerId, const FVector2f& Start, const FVector2f& End, const FConnectionParams& Params);

	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual void DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2D& StartPoint, const FVector2D& EndPoint, UEdGraphPin* Pin) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual void DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2f& StartPoint, const FVector2f& EndPoint, UEdGraphPin* Pin);

	// If a SliceLine has already been set, draw it. Subclasses may return false to indicate they don't support drawing a
	// slice line or draw it in a different style.
	UE_API virtual bool DrawSliceLine();

	// Give specific editor modes a chance to highlight this connection or darken non-interesting connections
	UE_API virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params);

	UE_API virtual void Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes);

	UE_API virtual void DetermineLinkGeometry(
		FArrangedChildren& ArrangedNodes, 
		TSharedRef<SWidget>& OutputPinWidget,
		UEdGraphPin* OutputPin,
		UEdGraphPin* InputPin,
		/*out*/ FArrangedWidget*& StartWidgetGeometry,
		/*out*/ FArrangedWidget*& EndWidgetGeometry
		);

	// Choose whether we want to cache the pins draw state to avoid resetting it for every tick 
	virtual bool UseDrawStateCaching() const { return false; }
	
	UE_API virtual void SetIncompatiblePinDrawState(const TSharedPtr<SGraphPin>& StartPin, const TSet< TSharedRef<SWidget> >& VisiblePins);
	UE_API virtual void ResetIncompatiblePinDrawState(const TSet< TSharedRef<SWidget> >& VisiblePins);

	UE_API virtual void ApplyHoverDeemphasis(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ float& Thickness, /*inout*/ FLinearColor& WireColor);

	UE_API virtual bool IsConnectionCulled( const FArrangedWidget& StartLink, const FArrangedWidget& EndLink ) const;

	UE_API virtual TSharedPtr<IToolTip> GetConnectionToolTip(const SGraphPanel& GraphPanel, const FGraphSplineOverlapResult& OverlapData) const;

	/**
	 * Used in MouseMove to check if the connections have graphically changed and need the GraphPanel to invalidate.
	 * The default implementation checks for a difference in HoveredPins, or in difference in connection WireColor or WireThickness.
	 */
	UE_API virtual bool HaveConnectionsGraphicallyChanged(const SGraphPanel& InGraphPanel, const FVector2f& AbsoluteCursorPosition) const;

	/**
	 * Used in MouseMove to check if the connections have graphically changed and need the GraphPanel to invalidate.
	 * The default implementation checks for a difference in HoveredPins, or in difference in connection WireColor or WireThickness.
	 */
	UE_API virtual bool HaveConnectionsGraphicallyChanged(const SGraphPanel& InGraphPanel, const TSet<FEdGraphPinReference>& InNewlyHoveredPins) const;

protected:
	// Helper function used by Draw(). Called before DrawPinGeometries to populate PinToPinWidgetMap
	UE_API virtual void BuildPinToPinWidgetMap(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries);

	// Helper function used by Draw(). Iterates over the pin geometries, drawing connections between them
	UE_API virtual void DrawPinGeometries(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes);

	/**
	 * Checks if the given spline overlaps with the Cursor.
	 * @param AbsoluteCursorPosition Absolute position of the cursor
	 * @param Start Start point of the Spline
	 * @param End End point of the Spline
	 * @param Params Parameters of the Spline
	 * @param InOutOverlapResults Will hold the overlap result. Pass in previous results used to find the closest spline to the cursor
	 * @param bOutSliceLineIntersectsSpline If not null, will return true if the SliceLine intersects this spline
	 */
	UE_API void CheckSplineConnectionOverlapWithCursor(
		const FVector2f& AbsoluteCursorPosition,
		const FVector2f& Start,
		const FVector2f& End,
		const FConnectionParams& Params,
		/*out*/ FGraphSplineOverlapResult& InOutOverlapResults,
		/*out*/ bool* bOutSliceLineIntersectsSpline = nullptr
		) const;

	struct FSimpleConnectionData
	{
		FGraphPinHandle StartPinHandle;
		FGraphPinHandle EndPinHandle;
		FVector2f StartPoint;
		FVector2f EndPoint;
		FConnectionParams Params;
		bool bAreConnectionsFaded = false;
	};
	/** Keeps track of the last connections drawn to check if they would have changed on MouseMove */
	TArray<FSimpleConnectionData> LastConnections;
private:
	// Note - these helpers are temporary, to be replaced entirely by the public API's 2f/float functions once the 2D/double deprecated functions can be removed
	UE_API void DrawSplineWithArrow_DeprecationHelper(const UE::Slate::FDeprecateVector2DParameter& StartPoint, const UE::Slate::FDeprecateVector2DParameter& EndPoint, const FConnectionParams& Params);
	UE_API UE::Slate::FDeprecateVector2DResult ComputeSplineTangent_DeprecationHelper(const UE::Slate::FDeprecateVector2DParameter& Start, const UE::Slate::FDeprecateVector2DParameter& End) const;
	UE_API void DrawConnection_DeprecationHelper(int32 LayerId, const UE::Slate::FDeprecateVector2DParameter& Start, const UE::Slate::FDeprecateVector2DParameter& End, const FConnectionParams& Params);
	UE_API void DrawPreviewConnector_DeprecationHelper(const FGeometry& PinGeometry, const UE::Slate::FDeprecateVector2DParameter& StartPoint, const UE::Slate::FDeprecateVector2DParameter& EndPoint, UEdGraphPin* Pin);
};

#undef UE_API
