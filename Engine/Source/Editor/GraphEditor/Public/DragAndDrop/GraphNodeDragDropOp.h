// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

class UEdGraph;

class FGraphNodeDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FGraphNodeDragDropOp, FDecoratedDragDropOp)

	UE_DEPRECATED(5.6, "Please use the delegate accepting FVector2f.")
	DECLARE_DELEGATE_FourParams( FOnPerformDropToGraph, TSharedPtr<FDragDropOperation>, UEdGraph*, const FVector2D&, const FVector2D&);
	DECLARE_DELEGATE_FourParams(FOnPerformDropToGraphAtLocation, TSharedPtr<FDragDropOperation>, UEdGraph*, const FVector2f&, const FVector2f&);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOnPerformDropToGraph OnPerformDropToGraph;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FOnPerformDropToGraphAtLocation OnPerformDropToGraphAtLocation = FOnPerformDropToGraphAtLocation::CreateLambda(
	[this](TSharedPtr<FDragDropOperation> InOperation, UEdGraph* InGraph, const FVector2f& InNodePos, const FVector2f& InScreenPos)
		{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OnPerformDropToGraph.ExecuteIfBound(InOperation, InGraph, FVector2D(InNodePos), FVector2D(InScreenPos));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		});
};
