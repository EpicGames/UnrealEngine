// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "UObject/SoftObjectPtr.h"

struct FEdGraphNodeHandle
{
public:

	FORCEINLINE FEdGraphNodeHandle(const UEdGraphNode* InNode)
		: Graph(InNode ? InNode->GetGraph() : nullptr)
		, NodeName(InNode ? InNode->GetFName() : NAME_None)
	{}

	friend FORCEINLINE uint32 GetTypeHash(const FEdGraphNodeHandle& InHandle)
	{
		return HashCombine(GetTypeHash(InHandle.Graph.ToSoftObjectPath()), GetTypeHash(InHandle.NodeName));
	}

	FORCEINLINE bool operator ==(const FEdGraphNodeHandle& InOther) const
	{
		return Graph.GetUniqueID() == InOther.Graph.GetUniqueID() &&
			NodeName.IsEqual(InOther.NodeName, ENameCase::CaseSensitive, true);
	}

	FORCEINLINE const UEdGraph* GetGraph() const { return Graph.Get(); }

	FORCEINLINE UEdGraphNode* GetNode() const
	{
		if(const UEdGraph* EdGraph = GetGraph())
		{
			const TObjectPtr<UEdGraphNode>* Node = EdGraph->Nodes.FindByPredicate([this](TObjectPtr<UEdGraphNode> Node) -> bool
			{
				return Node->GetFName() == NodeName;
			});
			if(Node)
			{
				return Node->Get();
			}
		}
		return nullptr;
	}

private:
	TSoftObjectPtr<UEdGraph> Graph;
	FName NodeName;
};

struct FEdGraphPinHandle : FEdGraphNodeHandle
{
public:

	FORCEINLINE FEdGraphPinHandle(const UEdGraphPin* InPin)
		: FEdGraphNodeHandle(InPin ? InPin->GetOwningNode() : nullptr)
		, PinIndex(INDEX_NONE)
	{
		if (InPin)
		{
			PinName = InPin->GetFName();
			PinDirection = InPin->Direction.GetValue();
			PersistentPinGuid = InPin->PersistentGuid;
			
			if (const UEdGraphNode* Node = InPin->GetOwningNode())
			{
				PinIndex = Node->Pins.Find((UEdGraphPin*)InPin);
			}
		}
		else
		{
			PinDirection = EEdGraphPinDirection::EGPD_Input;
		}
	}

	friend FORCEINLINE uint32 GetTypeHash(const FEdGraphPinHandle& InHandle)
	{
		return InHandle.PersistentPinGuid.IsValid() ?
			HashCombine(
			   HashCombine(
				   GetTypeHash((FEdGraphNodeHandle)InHandle),
				   GetTypeHash(InHandle.PersistentPinGuid)),
			   GetTypeHash(InHandle.PinDirection))
			: HashCombine(
				HashCombine(
					GetTypeHash((FEdGraphNodeHandle)InHandle),
					GetTypeHash(InHandle.PinName)),
				GetTypeHash(InHandle.PinDirection));
	}

	FORCEINLINE bool operator ==(const FEdGraphPinHandle& InOther) const
	{
		return FEdGraphNodeHandle::operator==(InOther) &&
			((PersistentPinGuid == InOther.PersistentPinGuid) || (PinName.IsEqual(InOther.PinName, ENameCase::CaseSensitive, true))) &&
			PinDirection == InOther.PinDirection;
	}
	
	FORCEINLINE UEdGraphPin* GetPin() const
	{
		if(UEdGraphNode* EdNode = GetNode())
		{
			// first try using the persistent guid
			const UEdGraphPin*const* Pin = EdNode->Pins.FindByPredicate([this](UEdGraphPin* Pin) -> bool
			{
				return Pin->PersistentGuid.IsValid() && Pin->PersistentGuid == PersistentPinGuid && Pin->Direction == PinDirection;
			});

			// try to use the pin at the given index if name and direction matches.
			// this deals with cases where we have multiple pins of the same name.
			if(Pin == nullptr && EdNode->Pins.IsValidIndex(PinIndex))
			{
				if(EdNode->Pins[PinIndex]->GetFName() == PinName && EdNode->Pins[PinIndex]->Direction == PinDirection)
				{
					Pin = &EdNode->Pins[PinIndex];
				}
			}

			// finally fall back to asking the node for the pin by name
			if(Pin == nullptr)
			{
				Pin = EdNode->Pins.FindByPredicate([this](UEdGraphPin* Pin) -> bool
				{
					return Pin->GetFName() == PinName && Pin->Direction == PinDirection;
				});
			}
			
			if(Pin)
			{
				return (UEdGraphPin*)*Pin;
			}
		}
		return nullptr;
	}

private:
	FName PinName;
	EEdGraphPinDirection PinDirection;
	FGuid PersistentPinGuid;
	int32 PinIndex;
};

