// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define UE_API GRAPHEDITOR_API

class FAssetEditorToolkit;
class SSearchBox;

/** Item that matched the search results */
class FFindInGraphResult
{
public:
	virtual ~FFindInGraphResult() = default;

	struct FCreateParams
	{
		const FString& Value;
		TSharedPtr<FFindInGraphResult> Parent = nullptr;
		int	DuplicationIndex = 0;
		UClass* Class = nullptr;
		UEdGraphPin* Pin = nullptr;
		UEdGraphNode* GraphNode = nullptr;
	};

	/** Create a root (or only text) result */
	UE_API FFindInGraphResult(const FCreateParams& InCreateParams);

	/** By default NoOp, implement this to make your result jump to a node or node owning pin on click*/
	UE_API virtual void JumpToNode(TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit, const UEdGraphNode* InNode) const;

	/** Called when user clicks on the search item */
	UE_API virtual FReply OnClick(TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit);

	/* Get Category for this search result */
	UE_API virtual FText GetCategory() const;

	/** Create an icon to represent the result */
	UE_API virtual TSharedRef<SWidget>	CreateIcon() const;

	/** Gets the comment on this node if any */
	UE_API virtual FString GetCommentText() const;

	/** Gets the value of the pin if any */
	UE_API virtual FString GetValueText() const;

	/** Any children listed under this category */
	TArray<TSharedPtr<FFindInGraphResult>> Children;

	/** Search result Parent */
	TWeakPtr<FFindInGraphResult> Parent;

	/*The meta string that was stored in the asset registry for this item */
	FString Value;

	/*The graph may have multiple instances of whatever we are looking for, this tells us which instance # we refer to*/
	int	DuplicationIndex;

	/*The class this item refers to */
	UClass* Class;

	/** The pin that this search result refers to */
	FEdGraphPinReference Pin;

	/** The graph node that this search result refers to (if not by asset registry or UK2Node) */
	TWeakObjectPtr<UEdGraphNode> GraphNode;

	/** Display text for comment information */
	FString CommentText;
};


/** Widget for searching for items that are part of a UEdGraph */
class SFindInGraph : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFindInGraph){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit);

	/** Focuses this widget's search box */
	UE_API virtual void FocusForUse();

protected:
	typedef TSharedPtr<FFindInGraphResult> FSearchResult;
	typedef STreeView<FSearchResult> STreeViewType;

	/** Override this to create the search result of your type */
	UE_API virtual TSharedPtr<FFindInGraphResult> MakeSearchResult(const FFindInGraphResult::FCreateParams& InParams);

	/** Called when user changes the text they are searching for */
	UE_API virtual void OnSearchTextChanged(const FText& Text);

	/** Called when user changes commits text to the search box */
	UE_API virtual void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	/** Get the children of a row */
	UE_API virtual void OnGetChildren(FSearchResult InItem, TArray<FSearchResult>& OutChildren);

	/** Called when user clicks on a new result */
	UE_API virtual void OnTreeSelectionChanged(FSearchResult Item, ESelectInfo::Type SelectInfo);

	/** Called when user double clicks on a new result */
	UE_API virtual void OnTreeSelectionDoubleClick(FSearchResult Item);

	/** Called when a new row is being generated */
	UE_API virtual TSharedRef<ITableRow> OnGenerateRow(FSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Begins the search based on the SearchValue */
	UE_API virtual void InitiateSearch();

	/** Get a graph corresponding to this editor, implement if you want to want to use the default graph node search */
	UE_API virtual const UEdGraph* GetGraph();

	/** Find any results that contain all of the tokens */
	UE_API virtual void MatchTokens(const TArray<FString>& Tokens);

	/** 
	 * Implement this if you have node-specific search behavior. IE: Casting into child types to compare tokens. 
	 * @return true if all tokens are valid for this node, false otherwisee
	 */
	UE_API virtual bool MatchTokensInNode(const UEdGraphNode* Node, const TArray<FString>& Tokens);

	/** Find any results that contain all of the tokens in provided graph and subgraphs */
	UE_API virtual void MatchTokensInGraph(const UEdGraph* Graph, const TArray<FString>& Tokens);

	/** Determines if a string matches the search tokens */
	static UE_API bool StringMatchesSearchTokens(const TArray<FString>& Tokens, const FString& ComparisonString);

protected:
	/** Pointer back to the Material editor that owns us */
	TWeakPtr<FAssetEditorToolkit> AssetEditorToolkitPtr;

	/** The tree view displays the results */
	TSharedPtr<STreeViewType> TreeView;

	/** The search text box */
	TSharedPtr<SSearchBox> SearchTextField;

	/** This buffer stores the currently displayed results */
	TArray<FSearchResult> ItemsFound;

	/** we need to keep a handle on the root result, because it won't show up in the tree */
	FSearchResult RootSearchResult;

	/** The string to highlight in the results */
	FText HighlightText;

	/** The string to search for */
	FString	SearchValue;
};

#undef UE_API
