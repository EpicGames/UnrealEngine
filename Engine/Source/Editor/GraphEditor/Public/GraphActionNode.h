// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "SGraphActionMenu.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"

#define UE_API GRAPHEDITOR_API

template <typename ItemType> class STreeView;

// Utility class for building menus of graph actions
struct FGraphActionNode : TSharedFromThis<FGraphActionNode>
{
public:
	// We need to declare our copy constructors so that we can disable
	// deprecation warnings around them for ClangEditor - when all of
	// the deprecated members are deleted we can remove these:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGraphActionNode(const FGraphActionNode& Node) = delete;
	FGraphActionNode& operator=(const FGraphActionNode& Node) = delete;
	FGraphActionNode(FGraphActionNode&& Node) = delete;
	FGraphActionNode& operator=(FGraphActionNode&& Node) = delete;
	~FGraphActionNode() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** */
	static const int32 INVALID_SECTION_ID = 0;

	/** Identifies the named section that this node belongs to, if any (defaults to INVALID_SECTION_ID) */
	int32 const SectionID;
	/** Identifies the menu group that this node belongs to (defaults to zero) */
	int32 const Grouping;
	/** An action to execute when this node is picked from a menu */
	TSharedPtr<FEdGraphSchemaAction> const Action;
	UE_DEPRECATED(5.5, "!! WARNING: This array is no longer populated!! FGraphActionNode::Actions array only functioned with a single Action (GetPrimaryAction), access via Action")
	TArray< TSharedPtr<FEdGraphSchemaAction> > const Actions;

	/** */
	TArray< TSharedPtr<FGraphActionNode> > Children;

	/** Lookup table for category nodes, used to speed up menu construction */
	TMap< FString, TSharedPtr<FGraphActionNode> > CategoryNodes;

public:
	/**
	 * Static allocator for a new root node (so external users have a starting
	 * point to build graph action trees from).
	 *
	 * @return A newly allocated root node (should not be displayed in the tree view).
	 */
	static UE_API TSharedPtr<FGraphActionNode> NewRootNode();

	/**
	 * Inserts a new action node (and any accompanying category nodes) based off
	 * the provided Action. 
	 *
	 * NOTE: This does NOT insert the node in a sorted manner. Call SortChildren() 
	 *       separately or use AddChildAlphabetical
	 * 
	 * @param  Action	An action that you want the node to execute when picked.
	 * @return The new action node.
	 */
	UE_API TSharedPtr<FGraphActionNode> AddChild(const TSharedPtr<FEdGraphSchemaAction>& Action);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "FGraphActionListBuilderBase::ActionGroup has been deprecated, use TSharedPtr<FEdGraphSchemaAction> directly")
	UE_API TSharedPtr<FGraphActionNode> AddChild(FGraphActionListBuilderBase::ActionGroup const& ActionSet);
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Inserts a new action node (and any required category nodes) based off
	 * the provided Action. Inserts in alphabetical order.
	 *
	 * @param  Action	An action that you want the node to execute when picked.
	 * @return The new action node.
	 */
	TSharedPtr<FGraphActionNode> AddChildAlphabetical(const TSharedPtr<FEdGraphSchemaAction>& Action);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "FGraphActionListBuilderBase::ActionGroup has been deprecated, use TSharedPtr<FEdGraphSchemaAction> directly")
	UE_API TSharedPtr<FGraphActionNode> AddChildAlphabetical(FGraphActionListBuilderBase::ActionGroup const& ActionSet);
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TSharedPtr<FGraphActionNode> AddSection(int32 Grouping, int32 InSectionID);

	/**
	 * Sorts all child nodes by section, group, and type (additionally, can
	 * sort alphabetically if wanted).
	 * 
	 * @param  bAlphabetically	Determines if we sort alphabetically on top of section/group/type.
	 * @param  bRecursive		Determines if we should sort all decendent nodes' children ass well.
	 */
	UE_API void SortChildren(bool bAlphabetically = true, bool bRecursive = true);

	/**
	 * Returns a WeakPtr to the Parent Node
	 */
	TWeakPtr<FGraphActionNode> GetParentNode() const{ return ParentNode; }

	/**
	 * Recursively collects all child/grandchild/decendent nodes.
	 * 
	 * @param  OutNodeArray	The array to fill out with decendent nodes.
	 */
	UE_API void GetAllNodes(TArray< TSharedPtr<FGraphActionNode> >& OutNodeArray) const;

	/**
	 * Recursively collects all child/grandchild/decendent action nodes.
	 * 
	 * @param  OutNodeArray	The array to fill out with decendent nodes.
	 */
	UE_API void GetAllActionNodes(TArray<TSharedPtr<FGraphActionNode>>& OutNodeArray) const;

	/**
	 * Recursively collects all decendent action/separator nodes (leaves out 
	 * branching category-nodes).
	 * 
	 * @param  OutLeafArray	The array to fill out with decendent leaf nodes.
	 */
	UE_API void GetLeafNodes(TArray< TSharedPtr<FGraphActionNode> >& OutLeafArray) const;

	/** Returns the number of leaf nodes */
	UE_API int32 GetTotalLeafNodes() const;

	/**
	 * Takes the tree view and expands its elements for each child.
	 * 
	 * @param  TreeView		The tree responsible for visualizing this node hierarchy.
	 * @param  bRecursive	Determines if you want children/decendents to expand their children as well. 
	 */
	UE_API void ExpandAllChildren(TSharedPtr< STreeView< TSharedPtr<FGraphActionNode> > > TreeView, bool bRecursive = true);

	/**
	 * Clears all children (not recursively... the TSharedPtrs should clean up 
	 * appropriately).
	 */
	UE_API void ClearChildren();

	/**
	 * Query to determine this node's type (there are five distinguishable node
	 * types: root, section heading, category, action, & group-divider).
	 *
	 * @return True if this is the type your queried about, otherwise false.
	 */
	UE_API bool IsRootNode() const;
	UE_API bool IsSectionHeadingNode() const;
	UE_API bool IsCategoryNode() const;
	UE_API bool IsActionNode() const;
	UE_API bool IsGroupDividerNode() const;

	/**
	 * Determines if this node is a menu separator of some kind (either a
	 * "group-divider" or a "section heading").
	 *
	 * @return True if this is a menu divider, otherwise false.
	 */
	UE_API bool IsSeparator() const;
	/**
	 * Retrieves this node's display name (for category and action nodes). The
	 * text string will be empty for separator and root nodes.
	 *
	 * @return The name to present this node with in the tree view (will be an empty text string if this is a separator node)
	 */
	UE_API FText const& GetDisplayName() const;

	/**
	 * Walks the node chain backwards, constructing a category path (delimited
	 * by '|' characters). This includes this node's category (if it is a
	 * category node).
	 *
	 * @return A category path string, denoting the category hierarchy up to this node.
	 */
	UE_API FText GetCategoryPath() const;

	/**
	 * Checks to see if this node contains at least one valid action.
	 *
	 * @return True is the Actions array contains a valid entry, otherwise false.
	 */
	UE_API bool HasValidAction() const;

	/**
	 * Looks through this node's Actions array, and returns the first valid
	 * action it finds.
	 *
	 * @return This node's first valid action (will be an empty pointer if this is not an action node).
	 */
	UE_API TSharedPtr<FEdGraphSchemaAction> GetPrimaryAction() const;

	/**
	 * Accessor to the node's RenameRequestEvent (for binding purposes). Do not
	 * Execute() the delegate from this function, instead call
	 * BroadcastRenameRequest() on the node.
	 *
	 * @return The node's internal RenameRequestEvent.
	 */
	FOnRenameRequestActionNode& OnRenameRequest() { return RenameRequestEvent; }

	/**
	 * Executes the node's RenameRequestEvent if it is bound. Otherwise, it will
	 * mark the node as having a pending rename request.
	 *
	 * @return True if the broadcast went through, false if the "pending rename request" flag was set.
	 */
	UE_API bool BroadcastRenameRequest();

	/**
	 * Sometimes a call to BroadcastRenameRequest() is made before the
	 * RenameRequestEvent has been bound. When that happens, this node is
	 * marked with a pending rename request. This method determines if that is
	 * the case for this node.
	 *
	 * @return True if a call to BroadcastRenameRequest() was made without a valid RenameRequestEvent.
	 */
	UE_API bool IsRenameRequestPending() const;

	/** Returns the 'linearized' index of the node, including category nodes, useful for getting displayed position */
	UE_API int32 GetLinearizedIndex(TSharedPtr<FGraphActionNode> Node) const;
private:
	/**
	 *
	 *
	 * @param  Grouping
	 * @param  SectionID
	 */
	UE_API FGraphActionNode(int32 Grouping, int32 SectionID);

	/**
	 * Constructor for action nodes. Private so that users go through AddChild().
	 *
	 * @param  InAction
	 * @param  Grouping
	 * @param  SectionID
	 */
	UE_API FGraphActionNode(const TSharedPtr<FEdGraphSchemaAction>& InAction, int32 InGrouping, int32 InSectionID);

	/**
	 *
	 *
	 * @param  Parent
	 * @param  Grouping
	 * @param  SectionID
	 * @return
	 */
	static UE_API TSharedPtr<FGraphActionNode> NewSectionHeadingNode(TWeakPtr<FGraphActionNode> Parent, int32 Grouping, int32 SectionID);

	/**
	 *
	 *
	 * @param  Category
	 * @param  Grouping
	 * @param  SectionID
	 * @return
	 */
	static UE_API TSharedPtr<FGraphActionNode> NewCategoryNode(FString const& Category, int32 Grouping, int32 SectionID);

	/**
	 *
	 *
	 * @param  ActionNode
	 * @return
	 */
	static UE_API TSharedPtr<FGraphActionNode> NewActionNode(TSharedPtr<FEdGraphSchemaAction> const& ActionNode);

	/**
	 *
	 *
	 * @param  Parent
	 * @param  Grouping
	 * @return
	 */
	static UE_API TSharedPtr<FGraphActionNode> NewGroupDividerNode(TWeakPtr<FGraphActionNode> Parent, int32 Grouping);

	/**
	 * Iterates the CategoryStack, adding category-nodes as needed. The
	 * last category is what the node will be inserted under.
	 *
	 * @param  CategoryStack	A list of categories denoting where to nest the new node (the first element is the highest category)
	 * @param  Idx				Current point in the category stack that we have iterated to
	 * @param  NodeToAdd		The node you want inserted.
	 */
	UE_API void AddChildRecursively(const TArray<FString>& CategoryStack, int32 Idx, TSharedPtr<FGraphActionNode> NodeToAdd);

	/**
	 * Looks through this node's children to see if a there already exists a 
	 * node matching one we'd have to spawn (to parent the supplied NodeToAdd).
	 * 
	 * @param   ParentName	The name of the category NodeToAdd wants to nest under.
	 * @param   NodeToAdd	The node that we'll be adding to this child.
	 * @return  A child node matching the supplied parameters (will be empty if no match was found).
	 */
	UE_API TSharedPtr<FGraphActionNode> FindMatchingParent(FString const& ParentName, TSharedPtr<FGraphActionNode> NodeToAdd);

	/**
	 * Adds the specified node directly to this node's Children array. Will
	 * create and insert separators if needed (if the node has a new group or
	 * section).
	 *
	 * @param  NodeToAdd The node you want inserted.
	 */
	UE_API void InsertChild(TSharedPtr<FGraphActionNode> NodeToAdd);

	UE_API void AddChildGrouping(TSharedPtr<FGraphActionNode> ActionNode, TWeakPtr<FGraphActionNode> Parent, bool bInsertAlphabetically);
	UE_API void InsertChildAlphabetical(TSharedPtr<FGraphActionNode> NodeToAdd);

	/** Recursive implementation helper for GetLinearizedIndex */
	UE_API int32 GetLinearizedIndex(TSharedPtr<FGraphActionNode> Node, int32& Iter) const;
private:
	/** The category or action name (depends on what type of node this is) */
	FText DisplayText;
	/** The node that this is a direct child of (empty if this is a root node) */
	TWeakPtr<FGraphActionNode> ParentNode;

	/** Tracks what groups have already been added (so we can easily determine what group-dividers we need) */
	TSet<int32> ChildGroupings;
	/** Tracks what sections have already been added (so we can easily determine what heading we need) */
	TSet<int32> ChildSections;

	/** When the item is first created, a rename request may occur before everything is setup for it. This toggles to true in those cases */
	bool bPendingRenameRequest;
	/** Delegate to trigger when a rename was requested on this node */
	FOnRenameRequestActionNode RenameRequestEvent;

	friend struct FGraphActionNodeImpl;
	/** For sorting, when we don't alphabetically sort (so menu items don't jump around). */
	int32 InsertOrder;
	/** Root entry only, counts the total leaf entries in this tree */
	int32 TotalLeafs;
};

#undef UE_API
