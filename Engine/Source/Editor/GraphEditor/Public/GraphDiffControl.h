// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DiffResults.h"
#include "Internationalization/Text.h"

#define UE_API GRAPHEDITOR_API

class UEdGraph;
class UEdGraphNode;
struct FDiffResults;
struct FDiffSingleResult;

/** Used to find differences between revisions of a graph. */
class FGraphDiffControl
{
public:
	/** What kind of diff are we performing */
	enum class EDiffMode
	{
		/** An item present in 'RHS' but missing in 'LHS' has been added to the graph */
		Additive,
		/** An item present in 'RHS' but missing in 'LHS' has been removed from the graph */
		Subtractive,
	};

	/** Flags controlling while operations should be included in the diff */
	struct EDiffFlags
	{
		enum Type
		{
			/** Include node existence (adds and removes) in the diff */
			NodeExistance = (1<<0),
			/** Include node movement in the diff */
			NodeMovement = (1<<1),
			/** Include the node comment in the diff */
			NodeComment = (1<<2),
			/** Include the node pin changes in the diff */
			NodePins = (1<<3),
			/** Include any node specific items in the diff */
			NodeSpecificDiffs = (1<<4),
			/** Include all the things */
			All = NodeExistance | NodeMovement | NodeComment | NodePins | NodeSpecificDiffs,
		};
	};

	/** A struct holding the context data for a node diff */
	struct FNodeDiffContext
	{
		FNodeDiffContext()
			: DiffMode(EDiffMode::Additive)
			, DiffFlags(EDiffFlags::All)
			, NodeTypeDisplayName()
			, bIsRootNode(true)
		{
		}

		/** What kind of diff are we performing? */
		EDiffMode DiffMode;

		/** What items should be included in the diff? */
		EDiffFlags::Type DiffFlags;

		/** Display name used when showing the node type in messages */
		FText NodeTypeDisplayName;

		/** True if this node is a root node in the graph, false if this node is nested within another node */
		bool bIsRootNode;
	};

	/** A struct to represent a found pair of nodes that match each other (for comparisons sake) */
	struct FNodeMatch
	{
		FNodeMatch()
			: NewNode(nullptr)
			, OldNode(nullptr)
		{}

		UEdGraphNode* NewNode;
		UEdGraphNode* OldNode;

		/**
		 * Looks at the two node members and compares them to see if there are any
		 * differences. If one of the nodes is nullptr, then this will return true with
		 * a EDiffType::NODE_ADDED result.
		 * 
		 * @param  DiffsArrayOut	If supplied, this will be filled out with all the differences that were found.
		 * @return True if there were differences found, false if the two nodes are identical.
		 */
		UE_API bool Diff(const FNodeDiffContext& DiffContext, TArray<FDiffSingleResult>* DiffsResultsOut = nullptr) const;
		UE_API bool Diff(const FNodeDiffContext& DiffContext, FDiffResults& DiffsOut) const;

		/**
		 * Checks to see if this is a valid match.
		 * @return False if NewNode or OldNode is nullptr, true if both are valid.
		 */
		UE_API bool IsValid() const;
	};

	/**
	 * Looks through the supplied graph for a node that best matches the one specified.
	 * Sometimes (when diffing separate assets) there could be more than one possible
	 * match, so providing a list of already matched nodes helps us narrow it down (and 
	 * prevents us from matching one node with multiple others).
	 * 
	 * @param  OldGraph		The graph you want to search.
	 * @param  NewNode		The new node you want to match.
	 * @param  PriorMatches	Previous made matches to exclude from our search.
	 * @return A pair of nodes (including the supplied one) that best match each other (one may be nullptr if no match was found).
	 */
	static UE_API FNodeMatch FindNodeMatch(UEdGraph* OldGraph, UEdGraphNode* NewNode, TArray<FNodeMatch> const& PriorMatches);

	/** 
	 * Returns true if the two Nodes are the same 
	 * 
	 * @param  Node1		First node to check
	 * @param  Node2		Second node
	 * @param  Exclusions	Previous made matches to exclude from our search.
	 */
	static UE_API bool IsNodeMatch(UEdGraphNode* Node1, UEdGraphNode* Node2, bool bExactOnly = false, TArray<FGraphDiffControl::FNodeMatch> const* Exclusions = nullptr);

	/**
	 * Looks for node differences between the two supplied graphs. Diffs will be
	 * returned in the context of "what has changed from OldGraph in NewGraph?" 
	 * 
	 * @param  OldGraph		The baseline graph to compare against.
	 * @param  NewGraph		The second graph to look for changes in.
	 * @param  DiffsOut		All the differences that were found between the two.
	 * @return True if any differences were found, false if both graphs are identical.
	 */
	static UE_API bool DiffGraphs(UEdGraph* const OldGraph, UEdGraph* const NewGraph, TArray<FDiffSingleResult>& DiffsOut);

	/** Computes an object path for a specific graph, relative to the blueprint/asset root */
	static UE_API FString GetGraphPath(UEdGraph* Graph);
};

#undef UE_API
