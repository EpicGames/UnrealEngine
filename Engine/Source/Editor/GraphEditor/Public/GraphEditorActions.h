// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define UE_API GRAPHEDITOR_API

class FUICommandInfo;

UE_DECLARE_TCOMMANDS(class FGraphEditorCommandsImpl, GRAPHEDITOR_API)

class FGraphEditorCommandsImpl : public TCommands<FGraphEditorCommandsImpl>
{
public:

	FGraphEditorCommandsImpl()
		: TCommands<FGraphEditorCommandsImpl>( TEXT("GraphEditor"), NSLOCTEXT("Contexts", "GraphEditor", "Graph Editor"), NAME_None, FAppStyle::GetAppStyleSetName() )
	{
	}	

	virtual ~FGraphEditorCommandsImpl() = default;

	GRAPHEDITOR_API virtual void RegisterCommands() override;

	TSharedPtr< FUICommandInfo > ReconstructNodes;
	TSharedPtr< FUICommandInfo > BreakNodeLinks;

	// Execution sequence specific commands
	TSharedPtr< FUICommandInfo > AddExecutionPin;
	TSharedPtr< FUICommandInfo > InsertExecutionPinBefore;
	TSharedPtr< FUICommandInfo > InsertExecutionPinAfter;
	TSharedPtr< FUICommandInfo > RemoveExecutionPin;

	// SetFieldsInStruct specific commands
	TSharedPtr< FUICommandInfo > RemoveThisStructVarPin;
	TSharedPtr< FUICommandInfo > RemoveOtherStructVarPins;
	TSharedPtr< FUICommandInfo > RestoreAllStructVarPins;

	// Select node specific commands
	TSharedPtr< FUICommandInfo > AddOptionPin;
	TSharedPtr< FUICommandInfo > RemoveOptionPin;
	TSharedPtr< FUICommandInfo > ChangePinType;
	TSharedPtr< FUICommandInfo > DeleteAndReconnectNodes;

	// Pin visibility modes
	TSharedPtr< FUICommandInfo > ShowAllPins;
	TSharedPtr< FUICommandInfo > HideNoConnectionPins;
	TSharedPtr< FUICommandInfo > HideNoConnectionNoDefaultPins;

	// Event / Function Entry commands
	TSharedPtr< FUICommandInfo > AddParentNode;

	// CallFunction commands
	TSharedPtr< FUICommandInfo > CreateMatchingFunction;

	// Debugging commands
	TSharedPtr< FUICommandInfo > RemoveBreakpoint;
	TSharedPtr< FUICommandInfo > AddBreakpoint;
	TSharedPtr< FUICommandInfo > EnableBreakpoint;
	TSharedPtr< FUICommandInfo > DisableBreakpoint;
	TSharedPtr< FUICommandInfo > ToggleBreakpoint;

	// Encapsulation commands
	TSharedPtr< FUICommandInfo > CollapseNodes;
	TSharedPtr< FUICommandInfo > PromoteSelectionToFunction;
	TSharedPtr< FUICommandInfo > PromoteSelectionToMacro;
	TSharedPtr< FUICommandInfo > ExpandNodes;
	TSharedPtr< FUICommandInfo > CollapseSelectionToFunction;
	TSharedPtr< FUICommandInfo > CollapseSelectionToMacro;
	TSharedPtr< FUICommandInfo > ConvertFunctionToEvent;
	TSharedPtr< FUICommandInfo > ConvertEventToFunction;

	// Alignment commands
	TSharedPtr< FUICommandInfo > AlignNodesTop;
	TSharedPtr< FUICommandInfo > AlignNodesMiddle;
	TSharedPtr< FUICommandInfo > AlignNodesBottom;

	TSharedPtr< FUICommandInfo > AlignNodesLeft;
	TSharedPtr< FUICommandInfo > AlignNodesCenter;
	TSharedPtr< FUICommandInfo > AlignNodesRight;

	TSharedPtr< FUICommandInfo > StraightenConnections;

	TSharedPtr< FUICommandInfo > DistributeNodesHorizontally;
	TSharedPtr< FUICommandInfo > DistributeNodesVertically;

	TSharedPtr< FUICommandInfo > StackNodesHorizontally;
	TSharedPtr< FUICommandInfo > StackNodesVertically;
	
	// Enable/disable commands
	TSharedPtr< FUICommandInfo > EnableNodes;
	TSharedPtr< FUICommandInfo > DisableNodes;
	TSharedPtr< FUICommandInfo > EnableNodes_Always;
	TSharedPtr< FUICommandInfo > EnableNodes_DevelopmentOnly;

	//
	TSharedPtr< FUICommandInfo > SelectReferenceInLevel;
	TSharedPtr< FUICommandInfo > AssignReferencedActor;

	// Find references
	TSharedPtr< FUICommandInfo > FindReferences;
	TSharedPtr< FUICommandInfo > GetFindReferences() const { return FindReferences; }

	// Find references options that appear by context like for functions and variables
	TSharedPtr< FUICommandInfo > FindReferencesByNameLocal;
	TSharedPtr< FUICommandInfo > FindReferencesByNameGlobal;
	TSharedPtr< FUICommandInfo > FindReferencesByClassMemberLocal;
	TSharedPtr< FUICommandInfo > FindReferencesByClassMemberGlobal;
	
	TSharedPtr< FUICommandInfo > FindAndReplaceReferences;

	// Jumps to the definition of the selected node (or otherwise focuses something interesting about that node, e.g., the inner graph for a collapsed graph)
	TSharedPtr< FUICommandInfo > GoToDefinition;

	// Opens collapsed graphs, functions or macros in a new tab
	TSharedPtr< FUICommandInfo > OpenInNewTab;

	// Pin-specific actions
	TSharedPtr< FUICommandInfo > BreakThisLink;
	TSharedPtr< FUICommandInfo > BreakPinLinks;
	TSharedPtr< FUICommandInfo > PromoteToVariable;
	TSharedPtr< FUICommandInfo > PromoteToLocalVariable;
	TSharedPtr< FUICommandInfo > SplitStructPin;
	TSharedPtr< FUICommandInfo > RecombineStructPin;
	TSharedPtr< FUICommandInfo > StartWatchingPin;
	TSharedPtr< FUICommandInfo > StopWatchingPin;
	TSharedPtr< FUICommandInfo > ResetPinToDefaultValue;
	TSharedPtr< FUICommandInfo > SelectAllInputNodes;
	TSharedPtr< FUICommandInfo > SelectAllOutputNodes;

	//create a comment node
	TSharedPtr< FUICommandInfo > CreateComment;
	
	// Create a custom event node
	TSharedPtr< FUICommandInfo > CreateCustomEvent;

	// Zoom in and out on the graph editor
	TSharedPtr< FUICommandInfo > ZoomIn;
	TSharedPtr< FUICommandInfo > ZoomOut;

	// Go to node documentation
	TSharedPtr< FUICommandInfo > GoToDocumentation;

	// Open the context menu at last known mouse position
	TSharedPtr< FUICommandInfo > SummonCreateNodeMenu;

	// Quick jump commands
	struct FQuickJumpCommandInfo
	{
		TSharedPtr< FUICommandInfo > QuickJump;
		TSharedPtr< FUICommandInfo > SetQuickJump;
		TSharedPtr< FUICommandInfo > ClearQuickJump;
	};
	TArray< FQuickJumpCommandInfo > QuickJumpCommands;
	TSharedPtr< FUICommandInfo > ClearAllQuickJumps;
};

class FGraphEditorCommands
{
public:
	static UE_API void Register();

	static UE_API const FGraphEditorCommandsImpl& Get();
	
	/** Build "Find References" submenu when a context allows for it */
	static UE_API void BuildFindReferencesMenu(FMenuBuilder& MenuBuilder);

	static UE_API void Unregister();
};

#undef UE_API
