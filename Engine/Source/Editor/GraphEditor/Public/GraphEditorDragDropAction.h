// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API GRAPHEDITOR_API

class SGraphNode;
class SGraphPanel;
class SWidget;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UEdGraphSchema;
struct FEdGraphSchemaAction;
struct FSlateBrush;

// Base class for drag-drop actions that pass into the graph editor and perform an action when dropped
class FGraphEditorDragDropAction : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FGraphEditorDragDropAction, FDragDropOperation)

	UE_API FGraphEditorDragDropAction();

	UE_API void SetHoveredPin(UEdGraphPin* InPin);
	UE_API void SetHoveredNode(const TSharedPtr<SGraphNode>& InNode);
	UE_API void SetHoveredNode(UEdGraphNode* InNode);
	UE_API void SetHoveredGraph(const TSharedPtr<SGraphPanel>& InGraph);
	UE_API void SetHoveredCategoryName(const FText& InHoverCategoryName);
	UE_API void SetHoveredAction(TSharedPtr<struct FEdGraphSchemaAction> Action);
	void SetDropTargetValid( bool bValid ) { bDropTargetValid = bValid; }

	// Interface to override
	virtual void HoverTargetChanged() {}
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual FReply DroppedOnPin(FVector2D ScreenPosition, FVector2D GraphPosition) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition);
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual FReply DroppedOnNode(FVector2D ScreenPosition, FVector2D GraphPosition) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition);
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual FReply DroppedOnPanel(const TSharedRef< SWidget >& Panel, FVector2D ScreenPosition, FVector2D GraphPosition, UEdGraph& Graph) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual FReply DroppedOnPanel(const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph);
	virtual FReply DroppedOnAction(TSharedRef<struct FEdGraphSchemaAction> Action) { return FReply::Unhandled(); }
	virtual FReply DroppedOnCategory(FText Category) { return FReply::Unhandled(); }
	// End of interface to override
	
	virtual bool IsSupportedBySchema(const UEdGraphSchema* Schema) const { return true; }

	UE_API bool HasFeedbackMessage();
	UE_API void SetFeedbackMessage(const TSharedPtr<SWidget>& Message);
	UE_API void SetSimpleFeedbackMessage(const FSlateBrush* Icon, const FSlateColor& IconColor, const FText& Message, const FSlateBrush* SecondaryIcon = nullptr, const FSlateColor SecondaryColor = FSlateColor());

protected:
	UE_API UEdGraphPin* GetHoveredPin() const;
	UE_API UEdGraphNode* GetHoveredNode() const;
	UE_API UEdGraph* GetHoveredGraph() const;

	/** Constructs the window and widget if applicable */
	UE_API virtual void Construct() override;

	UE_API virtual EVisibility GetIconVisible() const;
	UE_API virtual EVisibility GetErrorIconVisible() const;

private:
	// The pin that the drag action is currently hovering over
	FEdGraphPinReference HoveredPin;

	// The node that the drag action is currently hovering over
	TWeakObjectPtr<UEdGraphNode> HoveredNode;

	// The graph that the drag action is currently hovering over
	TSharedPtr<SGraphPanel> HoveredGraph;

protected:

	// Name of category we are hovering over
	FText HoveredCategoryName;

	// Action we are hovering over
	TWeakPtr<struct FEdGraphSchemaAction> HoveredAction;

	// drop target status
	bool bDropTargetValid;
};

// Drag-drop action where an FEdGraphSchemaAction should be performed when dropped
class FGraphSchemaActionDragDropAction : public FGraphEditorDragDropAction
{
public:
	
	DRAG_DROP_OPERATOR_TYPE(FGraphSchemaActionDragDropAction, FGraphEditorDragDropAction)

	// FGraphEditorDragDropAction interface
	UE_API virtual void HoverTargetChanged() override;
	UE_API virtual FReply DroppedOnPanel(const TSharedRef< SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	UE_API virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	// End of FGraphEditorDragDropAction

	static TSharedRef<FGraphSchemaActionDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InActionNode )
	{
		TSharedRef<FGraphSchemaActionDragDropAction> Operation = MakeShareable(new FGraphSchemaActionDragDropAction);
		Operation->SourceAction = InActionNode;
		Operation->Construct();
		return Operation;
	}

protected:
	UE_API virtual void GetDefaultStatusSymbol(const FSlateBrush*& PrimaryBrushOut, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut) const;

	/** */
	TSharedPtr<FEdGraphSchemaAction> SourceAction;
};

#undef UE_API
