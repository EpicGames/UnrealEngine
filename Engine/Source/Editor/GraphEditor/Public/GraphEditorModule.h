// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UICommandList.h"
#include "GraphEditor.h"
#include "HAL/Platform.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FExtender;
class FUICommandList;
class SWidget;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
struct FDiffSingleResult;

/**
 * Graph editor public interface
 */
class FGraphEditorModule : public IModuleInterface
{

public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Delegates to be called to extend the graph menus */
	
	DECLARE_DELEGATE_RetVal_FiveParams( TSharedRef<FExtender>, FGraphEditorMenuExtender_SelectedNode, const TSharedRef<FUICommandList>, const UEdGraph*, const UEdGraphNode*, const UEdGraphPin*, bool);
	virtual TArray<FGraphEditorMenuExtender_SelectedNode>& GetAllGraphEditorContextMenuExtender() {return GraphEditorContextMenuExtender;}

private:
	friend class SGraphEditor;

	/**
	 * DO NOT CALL THIS METHOD. Use SNew(SGraphEditor) to make instances of SGraphEditor.
	 *
	 * @return A GraphEditor implementation.
	 */
	virtual TSharedRef<SGraphEditor> PRIVATE_MakeGraphEditor(
		const TSharedPtr<FUICommandList>& InAdditionalCommands,
		const TAttribute<bool>& InIsEditable,
		const TAttribute<bool>& InDisplayAsReadOnly,
		const TAttribute<bool>& InIsEmpty,
		TAttribute<FGraphAppearanceInfo> Appearance,
		TSharedPtr<SWidget> InTitleBar,
		UEdGraph* InGraphToEdit,
		SGraphEditor::FGraphEditorEvents GraphEvents,
		bool InAutoExpandActionMenu,
		TSharedPtr<TArray<FDiffSingleResult>> DiffResults,
		TAttribute<int32> FocusedDiffResult,
		FSimpleDelegate InOnNavigateHistoryBack,
		FSimpleDelegate InOnNavigateHistoryForward,
		TAttribute<bool> ShowGraphStateOverlay,
		TAttribute<bool> AllowConnectionSlicing,
		FOnGraphEditorTickDelegate InOnTickDelegate,
		bool bGraphSupportsInvalidation);

private:
	/** All extender delegates for the graph menus */
	TArray<FGraphEditorMenuExtender_SelectedNode> GraphEditorContextMenuExtender;
};
