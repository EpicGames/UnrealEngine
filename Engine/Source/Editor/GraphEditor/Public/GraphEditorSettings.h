// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphNode_Comment.h"
#include "HAL/Platform.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "GraphEditorSettings.generated.h"

#define UE_API GRAPHEDITOR_API

UENUM()
enum class EGraphPanningMouseButton : uint8
{
	Right	UMETA(DisplayName = "Right"),
	Middle	UMETA(DisplayName = "Middle"),
	Both	UMETA(DisplayName = "Right or Middle")
};

struct FPropertyChangedEvent;

/**
 * Implements settings for the graph editor.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UGraphEditorSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:

#if WITH_EDITOR
	// UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif

public:

	/** The visual styling to use for graph editor pins (in Blueprints, materials, etc...) */
	UPROPERTY(config, EditAnywhere, Category=GeneralStyle)
	TEnumAsByte<EBlueprintPinStyleType> DataPinStyle;

	/** Switch between using the right and middle mouse button for panning (defaults to right) */
	UPROPERTY(config, EditAnywhere, Category = GeneralStyle)
	EGraphPanningMouseButton PanningMouseButton;

	/** Whether to use high precision mouse movement for more accurate mouse movement without mouse ballistics (defaults to true) */
	UPROPERTY(config, EditAnywhere, Category = GeneralStyle)
	bool bUseHighPrecisionMouseMovement;

	/** Whether to interpolate movement when manually panning a graph for smoother movement (defaults to false) */
	UPROPERTY(config, EditAnywhere, Category = GeneralStyle)
	bool bUseInterpolationWithManualPanning;

	/** The amount of padding above a pin (defaults to 4) */
	UPROPERTY(config, EditAnywhere, Category=GeneralStyle, AdvancedDisplay)
	float PaddingAbovePin;

	/** The amount of padding below a pin (defaults to 4) */
	UPROPERTY(config, EditAnywhere, Category=GeneralStyle, AdvancedDisplay)
	float PaddingBelowPin;

	/** The amount of horizontal padding towards the center of a node on an input pin (defaults to 10) */
	UPROPERTY(config, EditAnywhere, Category=GeneralStyle, AdvancedDisplay)
	float PaddingRightOfInput;

	/** The amount of horizontal padding towards the center of a node on an output pin (defaults to 10) */
	UPROPERTY(config, EditAnywhere, Category=GeneralStyle, AdvancedDisplay)
	float PaddingLeftOfOutput;

	/** The amount of padding towards the node edge (defaults to 10, can be negative to make pins overlap or stick out of a node */
	UPROPERTY(config, EditAnywhere, Category=GeneralStyle, AdvancedDisplay)
	float PaddingTowardsNodeEdge;

	/** Returns the padding to use for an input pin on a graph node */
	UE_API FMargin GetInputPinPadding() const;

	/** Returns the padding to use for an output pin on a graph node */
	UE_API FMargin GetOutputPinPadding() const;

	/** Returns the padding to use for a node region that won't contain a pin */
	UE_API FMargin GetNonPinNodeBodyPadding() const;

	/** Returns the size adjustment from node render geometry to shadow render geometry */
	UE_API UE::Slate::FDeprecateVector2DResult GetShadowDeltaSize() const;

public:

	/** If enabled, allows splines to be Alt+Clicked to break them or Ctrl+Dragged to move them as if these actions were taking place on the associated pin. */
	UPROPERTY(EditAnywhere, config, Category=Splines)
	bool bTreatSplinesLikePins;

	/** The distance threshold controlling how close the mouse has to be to the spline in order to trigger a hover response */ 
	UPROPERTY(EditAnywhere, config, Category=Splines, meta=(EditCondition=bTreatSplinesLikePins))
	float SplineHoverTolerance;

	/** The additional distance around the spline to count as close, preventing other actions if the user just misses the spline when clicking*/
	UPROPERTY(EditAnywhere, config, Category = Splines, meta = (EditCondition = bTreatSplinesLikePins))
	float SplineCloseTolerance;

	/** The maximum value to clamp the absolute value of the horizontal distance between endpoints when calculating tangents (when the wire is moving forward) */
	UPROPERTY(config, EditAnywhere, Category=Splines, AdvancedDisplay)
	float ForwardSplineHorizontalDeltaRange;

	/** The maximum value to clamp the absolute value of the vertical distance between endpoints when calculating tangents (when the wire is moving forward) */
	UPROPERTY(config, EditAnywhere, Category=Splines, AdvancedDisplay)
	float ForwardSplineVerticalDeltaRange;

	/** The amount that the horizontal delta affects the generated tangent handle of splines (when the wire is moving forward) */
	UPROPERTY(config, EditAnywhere, Category=Splines, AdvancedDisplay)
	FDeprecateSlateVector2D ForwardSplineTangentFromHorizontalDelta;

	/** The amount that the vertical delta affects the generated tangent handle of splines (when the wire is moving forward) */
	UPROPERTY(config, EditAnywhere, Category=Splines, AdvancedDisplay)
	FDeprecateSlateVector2D ForwardSplineTangentFromVerticalDelta;

	/** The maximum value to clamp the absolute value of the horizontal distance between endpoints when calculating tangents (when the wire is moving backwards) */
	UPROPERTY(config, EditAnywhere, Category=Splines, AdvancedDisplay)
	float BackwardSplineHorizontalDeltaRange;

	/** The maximum value to clamp the absolute value of the vertical distance between endpoints when calculating tangents (when the wire is moving backwards) */
	UPROPERTY(config, EditAnywhere, Category=Splines, AdvancedDisplay)
	float BackwardSplineVerticalDeltaRange;

	/** The amount that the horizontal delta affects the generated tangent handle of splines (when the wire is moving backwards) */
	UPROPERTY(config, EditAnywhere, Category=Splines, AdvancedDisplay)
	FDeprecateSlateVector2D BackwardSplineTangentFromHorizontalDelta;

	/** The amount that the vertical delta affects the generated tangent handle of splines (when the wire is moving backwards) */
	UPROPERTY(config, EditAnywhere, Category=Splines, AdvancedDisplay)
	FDeprecateSlateVector2D BackwardSplineTangentFromVerticalDelta;

	/** Compute a tangent given the start and end points of the spline */
	UE_API UE::Slate::FDeprecateVector2DResult ComputeSplineTangent(const UE::Slate::FDeprecateVector2DParameter& Start, const UE::Slate::FDeprecateVector2DParameter& End) const;

public:

	/** The default color is used only for types not specifically defined below.  Generally if it's seen, it means another type needs to be defined so that the wire in question can have an appropriate color. */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor DefaultPinTypeColor;

	/** Execution pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor ExecutionPinTypeColor;

	/** Boolean pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor BooleanPinTypeColor;

	/** Byte pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor BytePinTypeColor;

	/** Class pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor ClassPinTypeColor;

	/** Integer pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor IntPinTypeColor;

	/** Integer64 pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor Int64PinTypeColor;

	/** Floating-point pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor FloatPinTypeColor;

	/** Double pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor DoublePinTypeColor;

	/** Real pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor RealPinTypeColor;

	/** Name pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor NamePinTypeColor;

	/** Asset pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor SoftObjectPinTypeColor;

	/** Asset Class pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor SoftClassPinTypeColor;

	/** Delegate pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor DelegatePinTypeColor;

	/** Object pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor ObjectPinTypeColor;

	/** Interface pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor InterfacePinTypeColor;

	/** String pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor StringPinTypeColor;

	/** Text pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor TextPinTypeColor;

	/** Struct pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor StructPinTypeColor;

	/** Wildcard pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor WildcardPinTypeColor;

	/** Vector pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor VectorPinTypeColor;

	/** Rotator pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor RotatorPinTypeColor;

	/** Transform pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor TransformPinTypeColor;

	/** Index pin type color */
	UPROPERTY(EditAnywhere, config, Category=PinColors)
	FLinearColor IndexPinTypeColor;

public:

	/** Event node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor EventNodeTitleColor;

	/** CallFunction node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor FunctionCallNodeTitleColor;

	/** Pure function call node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor PureFunctionCallNodeTitleColor;

	/** Parent class function call node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor ParentFunctionCallNodeTitleColor;

	/** Function Terminator node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor FunctionTerminatorNodeTitleColor;

	/** Exec Branch node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor ExecBranchNodeTitleColor;

	/** Exec Sequence node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor ExecSequenceNodeTitleColor;

	/** Result node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor ResultNodeTitleColor;

	/** Default Comment node title color */
	UPROPERTY(EditAnywhere, config, Category=NodeTitleColors)
	FLinearColor DefaultCommentNodeTitleColor;

	/** Preview node title color */
	UPROPERTY(EditAnywhere, config, Category = NodeTitleColors)
	FLinearColor PreviewNodeTitleColor;

	/** The color of the line shown while slicing connections */
	UPROPERTY(EditAnywhere, config, Category=Slicing)
	FLinearColor SliceLineColor;

	/** The length of each dash in the slice line */
	UPROPERTY(EditAnywhere, config, Category=Slicing)
	float SliceLineDashLength;

	/** The visual thickness of the slice line, does not affect collision */
	UPROPERTY(EditAnywhere, config, Category=Slicing)
	float SliceLineThickness;

public:
	/** The thickness of a data wire */
	UPROPERTY(EditAnywhere, config, Category=Tracing)
	float DefaultDataWireThickness;

	/** The thickness of an execution wire when not debugging */
	UPROPERTY(EditAnywhere, config, Category=Tracing)
	float DefaultExecutionWireThickness;

	/** The color to display execution wires that were just executed */
	UPROPERTY(EditAnywhere, config, Category=Tracing)
	FLinearColor TraceAttackColor;

	UPROPERTY(EditAnywhere, config, Category=Tracing)
	float TraceAttackWireThickness;

	/** How long is the attack color fully visible */
	UPROPERTY()
	float TraceAttackHoldPeriod;

	/** How long does it take to fade from the attack to the sustain color */
	UPROPERTY()
	float TraceDecayPeriod;

	UPROPERTY()
	float TraceDecayExponent;

	UPROPERTY(EditAnywhere, config, Category=Tracing)
	FLinearColor TraceSustainColor;

	UPROPERTY(EditAnywhere, config, Category=Tracing)
	float TraceSustainWireThickness;

	/** How long is the sustain color fully visible */
	UPROPERTY()
	float TraceSustainHoldPeriod;

	/** The color to fade to for execution wires on release */
	UPROPERTY(EditAnywhere, config, Category=Tracing)
	FLinearColor TraceReleaseColor;

	/** The thickness to drop down to during release / for unexecuted wires when debugging */
	UPROPERTY(EditAnywhere, config, Category=Tracing)
	float TraceReleaseWireThickness;

	/** How long does it take to fade from the sustain to the release color */
	UPROPERTY()
	float TraceReleasePeriod;

	UPROPERTY()
	float TraceReleaseExponent;

	/** How much of a bonus does an exec get for being near the top of the trace stack, and how does that fall off with position? */
	UPROPERTY()
	float TracePositionBonusPeriod;

	UPROPERTY()
	float TracePositionExponent;

public:
	/** The amount of padding to add in order to auto collate multiple created nodes when using tab context menu (defaults to 20) */
	UPROPERTY(config, EditAnywhere, Category=ContextMenu)
	float PaddingAutoCollateIncrement;

	/** If a key press (default Tab) should open the create node context menu when the mouse is on top of blank areas of the graph (defaults to true) */
	UPROPERTY(config, EditAnywhere, Category=ContextMenu)
	bool bOpenCreateMenuOnBlankGraphAreas;

	/** Whether a comment node should move any fully enclosed nodes around when it is moved */
	UPROPERTY(EditAnywhere, config, Category = CommentNodes)
	TEnumAsByte<ECommentBoxMode::Type> DefaultCommentNodeMoveMode;

	/** Whether to show a zoom-invariant comment bubble when zoomed out (making the comment node's text readable at any distance) */
	UPROPERTY(EditAnywhere, config, Category = CommentNodes)
	bool bShowCommentBubbleWhenZoomedOut;
};

#undef UE_API
