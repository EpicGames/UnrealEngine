// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPin.h"

#define UE_API GRAPHEDITOR_API

/////////////////////////////////////////////////////
// FGraphSplineOverlapResult

struct FGraphSplineOverlapResult
{
public:
	FGraphSplineOverlapResult(bool InCloseToSpline = false)
		: Pin1Handle(nullptr)
		, Pin2Handle(nullptr)
		, BestPinHandle(nullptr)
		, Pin1(nullptr)
		, Pin2(nullptr)
		, DistanceSquared(FLT_MAX)
		, DistanceSquaredToPin1(FLT_MAX)
		, DistanceSquaredToPin2(FLT_MAX)
		, bCloseToSpline(InCloseToSpline)
	{
	}

	FGraphSplineOverlapResult(UEdGraphPin* InPin1, UEdGraphPin* InPin2, float InDistanceSquared, float InDistanceSquaredToPin1, float InDistanceSquaredToPin2, bool InCloseToSpline)
		: Pin1Handle(InPin1)
		, Pin2Handle(InPin2)
		, BestPinHandle(nullptr)
		, Pin1(InPin1)
		, Pin2(InPin2)
		, DistanceSquared(InDistanceSquared)
		, DistanceSquaredToPin1(InDistanceSquaredToPin1)
		, DistanceSquaredToPin2(InDistanceSquaredToPin2)
		, bCloseToSpline(InCloseToSpline)
	{
	}

	bool IsValid() const
	{
		return DistanceSquared < FLT_MAX;
	}

	UE_API void ComputeBestPin();

	float GetDistanceSquared() const
	{
		return DistanceSquared;
	}

	bool GetCloseToSpline() const
	{
		return bCloseToSpline;
	}

	void SetCloseToSpline(bool InCloseToSpline) 
	{
		bCloseToSpline = InCloseToSpline;
	}

	TSharedPtr<class SGraphPin> GetBestPinWidget(const class SGraphPanel& InGraphPanel) const
	{
		TSharedPtr<class SGraphPin> Result;
		if (IsValid())
		{
			Result = BestPinHandle.FindInGraphPanel(InGraphPanel);
		}
		return Result;
	}

	FGraphPinHandle GetBestPinHandle() const
	{
		return BestPinHandle;
	}

	FGraphPinHandle GetPin1Handle() const { return Pin1Handle; }
	FGraphPinHandle GetPin2Handle() const { return Pin2Handle; }

	UE_API bool GetPins(const class SGraphPanel& InGraphPanel, UEdGraphPin*& OutPin1, UEdGraphPin*& OutPin2) const;
	UE_API void GetPinWidgets(const class SGraphPanel& InGraphPanel, TSharedPtr<class SGraphPin>& OutPin1, TSharedPtr<class SGraphPin>& OutPin2) const;

protected:
	FGraphPinHandle Pin1Handle;
	FGraphPinHandle Pin2Handle;
	FGraphPinHandle BestPinHandle;
	UEdGraphPin* Pin1;
	UEdGraphPin* Pin2;
	float DistanceSquared;
	float DistanceSquaredToPin1;
	float DistanceSquaredToPin2;
	bool  bCloseToSpline;
};

#undef UE_API
