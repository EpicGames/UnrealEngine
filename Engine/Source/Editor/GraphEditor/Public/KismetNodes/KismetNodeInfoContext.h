// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformMath.h"
#include "SNodePanel.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"

class UBlueprint;
class UEdGraphNode;
class UEdGraphPin;

// Context used to aid debugging displays for nodes
struct FKismetNodeInfoContext : public FNodeInfoContext
{
public:
	struct FObjectUUIDPair
	{
	public:
		class UObject* Object;
		int32 UUID;
	public:
		FObjectUUIDPair(class UObject* InObject, int32 InUUID) : Object(InObject), UUID(InUUID) {}

		FString GetDisplayName() const
		{
			if (AActor* Actor = Cast<AActor>(Object))
			{
				return Actor->GetActorLabel();
			}
			else
			{
				return (Object != NULL) ? Object->GetName() : TEXT("(null)");
			}
		}
	};

	TMap<class UEdGraphNode*, TArray<FObjectUUIDPair> > NodesWithActiveLatentActions;

	// Set of pins with watches
	TSet<UEdGraphPin*> WatchedPinSet;

	// Set of nodes with a pin that is being watched
	TSet<UEdGraphNode*> WatchedNodeSet;

	// Source blueprint for the graph
	UBlueprint* SourceBlueprint;

	// Object being debugged for the graph
	UObject* ActiveObjectBeingDebugged;

public:
	FKismetNodeInfoContext(class UEdGraph* SourceGraph);
};

