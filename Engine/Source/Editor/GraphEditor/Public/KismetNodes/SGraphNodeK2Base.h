// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "SGraphNode.h"
#include "SNodePanel.h"
#include "Templates/SharedPointer.h"

#define UE_API GRAPHEDITOR_API

class SToolTip;
class UObject;
struct FGraphInformationPopupInfo;
struct FLinearColor;
struct FNodeInfoContext;
struct FOverlayBrushInfo;
struct FSlateBrush;

class SGraphNodeK2Base : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2Base) { }
		SLATE_ARGUMENT(TSharedPtr<ISlateStyle>, Style)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	//~ Begin SGraphNode Interface
	UE_API virtual void UpdateGraphNode() override;

	//~ Begin SNodePanel::SNode Interface
	UE_API virtual void GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const override;
	UE_API virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	UE_API virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	UE_API virtual void GetDiffHighlightBrushes(const FSlateBrush*& BackgroundOut, const FSlateBrush*& ForegroundOut) const override;
	UE_API void PerformSecondPassLayout(const TMap< UObject*, TSharedRef<SNode> >& NodeToWidgetLookup) const override;

protected :
	//~ Begin SGraphNode Interface
	UE_API virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	//~ End SGraphNode Interface

	/** Set up node in 'standard' mode */
	UE_API void UpdateStandardNode();
	/** Set up node in 'compact' mode */
	UE_API void UpdateCompactNode();

	/** Get title in compact mode */
	UE_API FText GetNodeCompactTitle() const;

	/** Retrieves text to tack on to the top of the tooltip (above the standard text) */
	UE_API FText GetToolTipHeading() const;

	/** Returns the slate style to use for this node */
	UE_API const ISlateStyle& GetStyleSet() const;

	/** The slate style to use. We'll fall back to App Style if it's unset. */
	TSharedPtr<ISlateStyle> Style;

protected:
	static UE_API const FLinearColor BreakpointHitColor;
	static UE_API const FLinearColor LatentBubbleColor;
	static UE_API const FLinearColor TimelineBubbleColor;
	static UE_API const FLinearColor PinnedWatchColor;
};

#undef UE_API
