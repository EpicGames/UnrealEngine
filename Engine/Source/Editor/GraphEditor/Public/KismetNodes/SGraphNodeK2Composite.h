// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SToolTip;
class SWidget;
class UEdGraph;
class UK2Node_Composite;

class SGraphNodeK2Composite : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2Composite){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UK2Node_Composite* InNode);

	// SGraphNode interface
	 UE_API virtual void UpdateGraphNode() override;
	UE_API virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	// End of SGraphNode interface
protected:
	UE_API virtual UEdGraph* GetInnerGraph() const;

	UE_API FText GetPreviewCornerText() const;
	UE_API FText GetTooltipTextForNode() const;

	UE_API virtual TSharedRef<SWidget> CreateNodeBody();
};

#undef UE_API
