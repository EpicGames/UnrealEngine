// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class UK2Node;

class SGraphNodeK2Default : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2Default){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UK2Node* InNode);
};

#undef UE_API
