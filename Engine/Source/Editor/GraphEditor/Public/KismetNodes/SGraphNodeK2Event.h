// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/GameViewportClient.h"
#include "KismetNodes/SGraphNodeK2Default.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SGraphPin;
class SNodeTitle;
class SOverlay;
class SWidget;

class SGraphNodeK2Event : public SGraphNodeK2Default
{
public:
	SGraphNodeK2Event() : SGraphNodeK2Default(), bHasDelegateOutputPin(false) {}

protected:
	UE_API virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle) override;
	UE_API virtual bool UseLowDetailNodeTitles() const override;
	UE_API virtual void AddPin( const TSharedRef<SGraphPin>& PinToAdd ) override;


	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override
	{
		TitleAreaWidget = DefaultTitleAreaWidget;
	}

private:
	bool ParentUseLowDetailNodeTitles() const
	{
		return SGraphNodeK2Default::UseLowDetailNodeTitles();
	}

	UE_API EVisibility GetTitleVisibility() const;

	TSharedPtr<SOverlay> TitleAreaWidget;
	bool bHasDelegateOutputPin;
};

#undef UE_API
