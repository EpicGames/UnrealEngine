// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class FText;
class SWidget;
class UK2Node;
struct FMargin;
struct FSlateBrush;

class SGraphNodeK2Var : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeK2Var){}
	SLATE_END_ARGS()

	UE_API void Construct( const FArguments& InArgs, UK2Node* InNode );

	UE_API virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;

	UE_API virtual void GetDiffHighlightBrushes(const FSlateBrush*& BackgroundOut, const FSlateBrush*& ForegroundOut) const override;

	// SGraphNode interface
	UE_API virtual void UpdateGraphNode() override;
	// End of SGraphNode interface

protected:
	UE_API FSlateColor GetVariableColor() const;

	// Allow derived classes to override title widget
	UE_API virtual TSharedRef<SWidget> UpdateTitleWidget(FText InTitleText, TSharedPtr<SWidget> InTitleWidget, EHorizontalAlignment& InOutTitleHAlign, FMargin& InOutTitleMargin) const;
};

#undef UE_API
