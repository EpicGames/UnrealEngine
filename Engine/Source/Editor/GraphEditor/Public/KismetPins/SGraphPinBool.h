// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SWidget;
class UEdGraphPin;

template <>
struct TWidgetTypeTraits<class SGraphPinBool>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPinBool : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinBool) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	/** Determine if the check box should be checked or not */
	UE_API ECheckBoxState IsDefaultValueChecked() const;

	/** Called when check box is changed */
	UE_API void OnDefaultValueCheckBoxChanged( ECheckBoxState InIsChecked );
};

#undef UE_API
