// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "KismetPins/SGraphPinObject.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SWidget;
class UClass;
class UEdGraphPin;

/////////////////////////////////////////////////////
// SGraphPinClass

class SGraphPinClass : public SGraphPinObject
{
public:
	SLATE_BEGIN_ARGS(SGraphPinClass) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	void SetAllowAbstractClasses(bool bAllow) { bAllowAbstractClasses = bAllow; }
protected:

	// Called when a new class was picked via the asset picker
	UE_API void OnPickedNewClass(UClass* ChosenClass);

	//~ Begin SGraphPinObject Interface
	UE_API virtual FReply OnClickUse() override;
	virtual bool AllowSelfPinWidget() const override { return false; }
	UE_API virtual TSharedRef<SWidget> GenerateAssetPicker() override;
	UE_API virtual FText GetDefaultComboText() const override;
	UE_API virtual FOnClicked GetOnUseButtonDelegate() override;
	UE_API virtual const FAssetData& GetAssetData(bool bRuntimePath) const override;
	//~ End SGraphPinObject Interface

	/** Cached AssetData without the _C */
	mutable FAssetData CachedEditorAssetData;

	/** Whether abstract classes should be filtered out in the class viewer */
	bool bAllowAbstractClasses;
};

#undef UE_API
