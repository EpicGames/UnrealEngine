// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Math/Color.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SWidget;
class UEdGraphPin;
struct FGeometry;
struct FPointerEvent;

template <>
struct TWidgetTypeTraits<class SGraphPinColor>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPinColor : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinColor) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	/** Return the current color value stored in the pin */
	UE_API FLinearColor GetColor () const;

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	/** Called when clicking on the color button */
	UE_API FReply OnColorBoxClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Called when the color picker commits a color value */
	UE_API void OnColorCommitted(FLinearColor color);


private:
	TSharedPtr<SWidget> DefaultValueWidget;
};

#undef UE_API
