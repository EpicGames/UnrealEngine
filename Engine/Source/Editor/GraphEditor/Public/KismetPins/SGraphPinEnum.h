// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class SWidget;
class UEdGraphPin;

template <>
struct TWidgetTypeTraits<class SGraphPinEnum>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPinEnum : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinEnum) {}

	/** The array of values to allow in the combobox. All values permitted if this array is empty. */
	SLATE_ARGUMENT(TArray<FString>, ValidEnumValues)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:

	/**
	 *	Function to create class specific widget.
	 *
	 *	@return Reference to the newly created widget object
	 */
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;

private:

	/**
	 *	Function to get current string associated with the combo box selection
	 *
	 *	@return currently selected string
	 */
	UE_API FString OnGetText() const;

	/**
	 *	Function to generate the list of indexes from the enum object
	 *
	 *	@param OutComboBoxIndexes - Int array reference to store the list of indexes
	 */
	UE_API void GenerateComboBoxIndexes( TArray< TSharedPtr<int32> >& OutComboBoxIndexes );

	/**
	 *	Function to set the newly selected index
	 *
	 * @param NewSelection The newly selected item in the combo box
	 * @param SelectInfo Provides context on how the selection changed
	 */
	UE_API void ComboBoxSelectionChanged( TSharedPtr<int32> NewSelection, ESelectInfo::Type SelectInfo );

	/**
	 * Returns the friendly name of the enum at index EnumIndex
	 *
	 * @param EnumIndex	- The index of the enum to return the friendly name for
	 */
	UE_API FText OnGetFriendlyName(int32 EnumIndex);

	/**
	 * Returns the tooltip of the enum at index EnumIndex
	 *
	 * @param EnumIndex	- The index of the enum to return the tooltip for
	 */
	UE_API FText OnGetTooltip(int32 EnumIndex);

	/** The array of values to allow in the combobox. All values permitted if this array is empty. */
	TArray<FString> ValidEnumValues;

	/** Shared pointer to the combobox widget. */
	TSharedPtr<class SPinComboBox> ComboBox;
};

#undef UE_API
