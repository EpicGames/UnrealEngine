// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SWidget;
class UEdGraphPin;
struct FSlateBrush;


template <>
struct TWidgetTypeTraits<class SGraphPinExec>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPinExec : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinExec)	{}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	UE_API virtual const FSlateBrush* GetPinIcon() const override;
	//~ End SGraphPin Interface

	UE_API void CachePinIcons();

	const FSlateBrush* CachedImg_Pin_ConnectedHovered;
	const FSlateBrush* CachedImg_Pin_DisconnectedHovered;
};

#undef UE_API
