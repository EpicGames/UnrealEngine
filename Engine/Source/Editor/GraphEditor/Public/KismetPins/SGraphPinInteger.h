// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "KismetPins/SGraphPinNum.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SWidget;
class UEdGraphPin;

template <>
struct TWidgetTypeTraits<class SGraphPinInteger>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPinInteger : public SGraphPinNum<int32>
{
public:
	SLATE_BEGIN_ARGS(SGraphPinInteger) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPinString Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPinString Interface
};

#undef UE_API
