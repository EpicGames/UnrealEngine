// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Misc/DefaultValueHelper.h"
#include "ScopedTransaction.h"
#include "SGraphPin.h"

template <typename NumericType>
class SGraphPinNum;

template <typename NumericType>
struct TWidgetTypeTraits<SGraphPinNum<NumericType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

template <typename NumericType>
class SGraphPinNum : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinNum) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
	{
		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	}

protected:
	// SGraphPin interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		return SNew(SBox)
			.MinDesiredWidth(18.0f)
			.MaxDesiredWidth(400.0f)
			[
				SNew(SNumericEntryBox<NumericType>)
				.EditableTextBoxStyle(FAppStyle::Get(), "Graph.EditableTextBox")
				.BorderForegroundColor(FSlateColor::UseForeground())
				.Visibility(this, &SGraphPinNum::GetDefaultValueVisibility)
				.IsEnabled(this, &SGraphPinNum::GetDefaultValueIsEditable)
				.Value(this, &SGraphPinNum::GetNumericValue)
				.OnValueCommitted(this, &SGraphPinNum::SetNumericValue)
			];
	}

	TOptional<NumericType> GetNumericValue() const
	{
		NumericType Num = NumericType();
		LexFromString(Num, *GraphPinObj->GetDefaultAsString());
		return Num;
	}

	void SetNumericValue(NumericType InValue, ETextCommit::Type CommitType)
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		if (GetNumericValue() != InValue)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeNumberPinValue", "Change Number Pin Value"));
			GraphPinObj->Modify();

			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, *LexToString(InValue));
		}
	}
};
