// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Misc/DefaultValueHelper.h"
#include "ScopedTransaction.h"
#include "SGraphPin.h"
#include "Editor.h"
#include "NumericPropertyParams.h"

template <typename NumericType>
class SGraphPinNumSlider : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinNumSlider)
		:_MinDesiredBoxWidth(18.0f)
		,_ShouldShowDisabledWhenConnected(false)
	{}
	
		SLATE_ARGUMENT(float, MinDesiredBoxWidth)
		SLATE_ARGUMENT(bool, ShouldShowDisabledWhenConnected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj, FProperty* InProperty)
	{
		PinProperty = InProperty;
		MinDesiredBoxWidth = InArgs._MinDesiredBoxWidth;
		bShouldShowDisabledWhenConnected = InArgs._ShouldShowDisabledWhenConnected;
		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);

	}

protected:
	// SGraphPin interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		const typename TNumericPropertyParams<NumericType>::FMetaDataGetter MetaDataGetter = TNumericPropertyParams<NumericType>::FMetaDataGetter::CreateLambda([&](const FName& Key)
			{
				return (PinProperty) ? PinProperty->GetMetaData(Key) : FString();
			});

		TNumericPropertyParams<NumericType> NumericPropertyParams(PinProperty, PinProperty ? MetaDataGetter : nullptr);
		
		const bool bAllowSpin = !(PinProperty && PinProperty->GetBoolMetaData("NoSpinbox"));

		// Save last committed value to compare when value changes
		LastSliderCommittedValue = GetNumericValue().GetValue();
		
		return SNew(SBox)
			.MinDesiredWidth(MinDesiredBoxWidth)
			.MaxDesiredWidth(400.0f)
			[
				SNew(SNumericEntryBox<NumericType>)
				.EditableTextBoxStyle(FAppStyle::Get(), "Graph.EditableTextBox")
				.BorderForegroundColor(FSlateColor::UseForeground())
				.Visibility(this, &SGraphPinNumSlider::GetDefaultValueVisibility)
				.IsEnabled(this, &SGraphPinNumSlider::GetDefaultValueIsEditable)
				.Value(this, &SGraphPinNumSlider::GetNumericValue)
				.MinValue(NumericPropertyParams.MinValue)
				.MaxValue(NumericPropertyParams.MaxValue)
				.MinSliderValue(NumericPropertyParams.MinSliderValue)
				.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
				.SliderExponent(NumericPropertyParams.SliderExponent)
				.Delta(NumericPropertyParams.Delta)
				.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
				.AllowWheel(bAllowSpin)
				.WheelStep(NumericPropertyParams.WheelStep)
				.AllowSpin(bAllowSpin)
				.OnValueCommitted(this, &SGraphPinNumSlider::OnValueCommitted)
				.OnValueChanged(this, &SGraphPinNumSlider::OnValueChanged)
				.OnBeginSliderMovement(this, &SGraphPinNumSlider::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &SGraphPinNumSlider::OnEndSliderMovement)
			];
	}

	void OnValueChanged( NumericType NewValue )
	{
		SliderValue = NewValue;
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, *LexToSanitizedString(NewValue), false);
	}

	void OnValueCommitted( NumericType NewValue, ETextCommit::Type CommitInfo )
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		if (LastSliderCommittedValue != NewValue)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeNumberPinValue", "Change Number Pin Value"));
			GraphPinObj->Modify();
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, *LexToSanitizedString(NewValue));
			LastSliderCommittedValue = NewValue;
		}
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement()
	{
		GEditor->BeginTransaction(NSLOCTEXT("GraphEditor", "ChangeNumberPinValueSlider", "Change Number Pin Value slider"));
		GraphPinObj->Modify();

		SliderValue = LastSliderCommittedValue = GetNumericValue().GetValue();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement( NumericType NewValue )
	{
		bIsUsingSlider = false;
		GEditor->EndTransaction();
	}
	
	TOptional<NumericType> GetNumericValue() const
	{
		NumericType Num = NumericType();
		LexFromString(Num, *GraphPinObj->GetDefaultAsString());
		return bIsUsingSlider ? SliderValue : Num;
	}

	virtual EVisibility GetDefaultValueVisibility() const override
	{
		// If this is only for showing default value, always show
		if (bOnlyShowDefaultValue)
		{
			return EVisibility::Visible;
		}

		// First ask schema
		UEdGraphPin* GraphPin = GetPinObj();
		const UEdGraphSchema* Schema = (GraphPin && !GraphPin->IsPendingKill()) ? GraphPin->GetSchema() : nullptr;
		if (Schema == nullptr || Schema->ShouldHidePinDefaultValue(GraphPin))
		{
			return EVisibility::Collapsed;
		}

		if (GraphPin->bNotConnectable && !GraphPin->bOrphanedPin)
		{
			// The only reason this pin exists is to show something, so do so
			return EVisibility::Visible;
		}

		if (GraphPin->Direction == EGPD_Output)
		{
			//@TODO: Should probably be a bLiteralOutput flag or a Schema call
			return EVisibility::Collapsed;
		}
		else
		{
			return IsConnected() && !bShouldShowDisabledWhenConnected ? EVisibility::Collapsed : EVisibility::Visible;
		}
	}
private:
	FProperty* PinProperty;
	NumericType LastSliderCommittedValue;
	NumericType SliderValue;
	float MinDesiredBoxWidth = 0.0f;
	bool bIsUsingSlider = false;
	bool bShouldShowDisabledWhenConnected = false;
};
