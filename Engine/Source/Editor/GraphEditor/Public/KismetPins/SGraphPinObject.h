// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "SGraphPin.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SButton;
class SWidget;
class UEdGraphPin;

class SGraphPinObject : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinObject) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	virtual bool DoesWidgetHandleSettingEditingEnabled() const override { return true; }
	//~ End SGraphPin Interface

	/** Delegate to be called when the use current selected item in asset browser button is clicked */
	UE_API virtual FOnClicked GetOnUseButtonDelegate();
	/** Delegate to be called when the browse for item button is clicked */
	UE_API virtual FOnClicked GetOnBrowseButtonDelegate();

	/** Clicked Use button */
	UE_API virtual FReply OnClickUse();
	/** Clicked Browse button */
	UE_API virtual FReply OnClickBrowse();
	/** Get text tooltip for object */
	UE_API FText GetObjectToolTip() const;
	/** Get text tooltip for object */
	UE_API FString GetObjectToolTipAsString() const;
	/** Get string value for object */
	UE_API FText GetValue() const;
	/** Get name of the object*/
	UE_API FText GetObjectName() const;
	/** Get default text for the picker combo */
	UE_API virtual FText GetDefaultComboText() const;
	/** Allow self pin widget */
	virtual bool AllowSelfPinWidget() const { return true; }
	/** True if this specific pin should be treated as a self pin */
	UE_API virtual bool ShouldDisplayAsSelfPin() const;
	/** Generate asset picker window */
	UE_API virtual TSharedRef<SWidget> GenerateAssetPicker();
	/** Called to validate selection from picker window */
	UE_API virtual void OnAssetSelectedFromPicker(const struct FAssetData& AssetData);
	/** Called when enter is pressed when items are selected in the picker window */
	UE_API void OnAssetEnterPressedInPicker(const TArray<FAssetData>& InSelectedAssets);

	/** Used to update the combo button text */
	UE_API FText OnGetComboTextValue() const;
	/** Combo Button Color and Opacity delegate */
	UE_API FSlateColor OnGetComboForeground() const;
	/** Button Color and Opacity delegate */
	UE_API FSlateColor OnGetWidgetForeground() const;
	/** Button Color and Opacity delegate */
	UE_API FSlateColor OnGetWidgetBackground() const;

	/** Returns asset data of currently selected object, if bRuntimePath is true this will include _C for blueprint classes, for false it will point to UBlueprint instead */
	UE_API virtual const FAssetData& GetAssetData(bool bRuntimePath) const;

protected:
	/** Object manipulator buttons. */
	TSharedPtr<SButton> UseButton;
	TSharedPtr<SButton> BrowseButton;

	/** Menu anchor for opening and closing the asset picker */
	TSharedPtr<class SMenuAnchor> AssetPickerAnchor;

	/** Cached AssetData of object selected */
	mutable FAssetData CachedAssetData;
};

#undef UE_API
