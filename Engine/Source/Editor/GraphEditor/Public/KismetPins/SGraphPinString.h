// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SWidget;
class UEdGraphPin;

template <>
struct TWidgetTypeTraits<class SGraphPinString>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPinString : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinString) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	UE_API FText GetTypeInValue() const;
	UE_API virtual void SetTypeInValue(const FText& NewTypeInValue, ETextCommit::Type CommitInfo);

	/** @return True if the pin default value field is read-only */
	UE_API bool GetDefaultValueIsReadOnly() const;

private:
	UE_API TSharedRef<SWidget> GenerateComboBoxEntry(TSharedPtr<FString> Value);
	UE_API void HandleComboBoxSelectionChanged(TSharedPtr<FString> Value, ESelectInfo::Type InSelectInfo);
	UE_API TSharedPtr<SWidget> TryBuildComboBoxWidget();

	TArray<TSharedPtr<FString>> ComboBoxOptions;
};

#undef UE_API
