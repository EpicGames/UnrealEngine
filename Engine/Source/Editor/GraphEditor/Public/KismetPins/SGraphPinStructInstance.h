// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"
#include "HAL/Platform.h"
#include "IStructureDetailsView.h"
#include "Internationalization/Text.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StructOnScope.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#include "SGraphPinStructInstance.generated.h"

#define UE_API GRAPHEDITOR_API

class FStructOnScope;
class IStructureDetailsView;
class SComboButton;
class SWidget;
class UEdGraphPin;
class UScriptStruct;
struct FPropertyChangedEvent;

/**
 * Base type for editing simple structs as pin default values, by displaying a nested version of a struct customization.
 * To make this work, create an inherited USTRUCT that includes a copy of the struct to be edited and overrides the functions.
 * Then, pass in StructName::StaticStruct() when creating SGraphPinStructInstance from a pin factory
 */
USTRUCT()
struct FPinStructEditWrapper
{
	GENERATED_BODY()

	virtual ~FPinStructEditWrapper() = default;

	/** Returns a text representation of the data */
	virtual FText GetPreviewDescription() const { return FText(); }

	/** Returns what script struct to use to parse the nested data */
	virtual const UScriptStruct* GetDataScriptStruct() const { return nullptr; }

	/** Returns address of nested data */
	virtual uint8* GetDataMemory() { return nullptr; }
};

/**
 * Example implementation:

USTRUCT()
struct FDataTypeEditWrapper : public FPinStructEditWrapper
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default, Meta = (ShowOnlyInnerProperties))
	FDataType Data;

	virtual FText GetPreviewDescription() const override { return Data.ToText(); }
	virtual const UScriptStruct* GetDataScriptStruct() const override { return FDataType::StaticStruct(); }
	virtual uint8* GetDataMemory() override { return (uint8*)&Data; }
};

 */


/** 
 * This is a pin for showing a details customization for a struct instance.
 * It can be used directly by passing in a StructEditWrapper parameter when creating from a pin factory
 * Or, it can be subclassed for other types of struct display
 */
class SGraphPinStructInstance : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinStructInstance) 
		: _StructEditWrapper(nullptr)
	{}
		SLATE_ARGUMENT(const UScriptStruct*, StructEditWrapper)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:

	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	/** Parses the struct ata from the pin to fill in the struct instance */
	UE_API virtual void ParseDefaultValueData();

	/** Call to push changes from edit instance to pin */
	UE_API virtual void SaveDefaultValueData();

	/** Refreshes cached description and edit data after an edit change */
	UE_API virtual void RefreshCachedData();

	/** Creates widget used to edit the struct instance */
	UE_API virtual TSharedRef<SWidget> GetEditContent();

	/** Creates widget for displaying preview on the pin */
	UE_API virtual TSharedRef<SWidget> GetDescriptionContent();

	/** Slate accessor to shows cached description value */
	UE_API virtual FText GetCachedDescriptionText() const;

	/** Returns the base instance inside EditStruct, if null this is assumed to be a subclass that overrides other functions */
	UE_API virtual FPinStructEditWrapper* GetEditWrapper() const;

	/** Called when struct is modified by the details view */
	UE_API virtual void PropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent);


	/** Combo Button widget used to show edit content */
	TSharedPtr<SComboButton> ComboButton;

	/** Instance of FPinStructEditWrapper that wraps what we actually want to edit */
	TSharedPtr<FStructOnScope> EditWrapperInstance;

	/** Details view that points into EditWrapperInstance */
	TSharedPtr<IStructureDetailsView> StructureDetailsView;

	/** Cached description text */
	FText CachedDescription;
};

#undef UE_API
