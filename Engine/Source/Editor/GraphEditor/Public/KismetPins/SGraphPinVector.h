// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SVectorTextBox.h"
#include "ScopedTransaction.h"

class SWidget;
class UEdGraphPin;

template <typename NumericType>
class SGraphPinVector;

template <typename NumericType>
struct TWidgetTypeTraits<SGraphPinVector<NumericType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

template <typename NumericType>
class SGraphPinVector : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinVector) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
	{
		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	}

protected:

	/**
	 *	Function to create class specific widget.
	 *
	 *	@return Reference to the newly created widget object
	 */
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();
		bIsRotator = (GraphPinObj->PinType.PinSubCategoryObject == RotatorStruct) ? true : false;
	
		return	SNew( SVectorTextBox<NumericType>, bIsRotator )
				.VisibleText_0(this, &SGraphPinVector::GetCurrentValue_0)
				.VisibleText_1(this, &SGraphPinVector::GetCurrentValue_1)
				.VisibleText_2(this, &SGraphPinVector::GetCurrentValue_2)
				.Visibility( this, &SGraphPin::GetDefaultValueVisibility)
				.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
				.OnNumericCommitted_Box_0(this, &SGraphPinVector::OnChangedValueTextBox_0)
				.OnNumericCommitted_Box_1(this, &SGraphPinVector::OnChangedValueTextBox_1)
				.OnNumericCommitted_Box_2(this, &SGraphPinVector::OnChangedValueTextBox_2);
	}

private:

	// Enum values to identify text boxes.
	enum ETextBoxIndex
	{
		TextBox_0,
		TextBox_1,
		TextBox_2,
	};

	using FVectorType = UE::Math::TVector<NumericType>;

	// Rotator is represented as X->Roll, Y->Pitch, Z->Yaw

	/*
	 *	Function to get current value in text box 0
	 *
	 *	@return current string value
	 */
	FString GetCurrentValue_0() const
	{
		// Text box 0: Rotator->Roll, Vector->X
		return GetValue(bIsRotator ? TextBox_2 : TextBox_0);
	}

	/*
	 *	Function to get current value in text box 1
	 *
	 *	@return current string value
	 */
	FString GetCurrentValue_1() const
	{
		// Text box 1: Rotator->Pitch, Vector->Y
		return GetValue(bIsRotator ? TextBox_0 : TextBox_1);
	}

	/*
	 *	Function to get current value in text box 2
	 *
	 *	@return current string value
	 */
	FString GetCurrentValue_2() const
	{
		// Text box 2: Rotator->Yaw, Vector->Z
		return GetValue(bIsRotator ? TextBox_1 : TextBox_2);
	}

	/*
	 *	Function to get a string array of the components composing the default string.
	 *
	 *	@return The array filled with each component as string
	 *
	 */
	TArray<FString> GetComponentArray() const
	{
		TArray<FString> VecComponentStrings;

		FString DefaultString = GraphPinObj->GetDefaultAsString();
		// Parse string to split its contents separated by ','
		DefaultString.TrimStartInline();
		DefaultString.TrimEndInline();
		DefaultString.ParseIntoArray(VecComponentStrings, TEXT(","), true);

		return VecComponentStrings;
	}
	
	/*
	 *	Function to get current value based on text box index value
	 *
	 *	@param: Text box index
	 *
	 *	@return current string value
	 */
	FString GetValue(ETextBoxIndex Index) const
	{
		const TArray<FString> VecComponentStrings = GetComponentArray();

		if (Index < VecComponentStrings.Num())
		{
			return VecComponentStrings[Index];
		}
		else
		{
			return FString(TEXT("0"));
		}
	}

	/*
	 *	Function to store value when text box 0 value in modified
	 *
	 *	@param 0: Updated numeric Value
	 */
	void OnChangedValueTextBox_0(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		const EAxis::Type Axis = bIsRotator ? /* update roll */ EAxis::Z : EAxis::X;
		SetNewValueHelper(Axis, NewValue);
	}

	/*
	 *	Function to store value when text box 1 value in modified
	 *
	 *	@param 0: Updated numeric Value
	 */
	void OnChangedValueTextBox_1(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		const EAxis::Type Axis = bIsRotator ? /* update pitch */ EAxis::X : EAxis::Y;
		SetNewValueHelper(Axis, NewValue);
	}

	/*
	 *	Function to store value when text box 2 value in modified
	 *
	 *	@param 0: Updated numeric Value
	 */
	void OnChangedValueTextBox_2(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		const EAxis::Type Axis = bIsRotator ? /* update yaw */ EAxis::Y : EAxis::Z;
		SetNewValueHelper(Axis, NewValue);
	}

	void SetNewValueHelper(EAxis::Type Axis, NumericType NewValue)
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		FVectorType NewVector = ConvertDefaultValueStringToVector();
		const NumericType OldValue = NewVector.GetComponentForAxis(Axis);

		if (OldValue == NewValue)
		{
			return;
		}

		NewVector.SetComponentForAxis(Axis, NewValue);

		const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeVectorPinValue", "Change Vector Pin Value"));
		GraphPinObj->Modify();

		// Create the new value string
		FString DefaultValue = FString::Format(TEXT("{0},{1},{2}"), { NewVector.X, NewVector.Y, NewVector.Z });

		//Set new default value
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, DefaultValue);
	}

	/*
	 * @brief Converts the default string value to a value of VectorType.
	 * 
	 * Example: it converts the default string value "2.00,3.00,4.00" to the corresponding 3D vector.
	 */
	FVectorType ConvertDefaultValueStringToVector() const
	{
		const TArray<FString> VecComponentStrings = GetComponentArray();

		// Construct the vector from the string parts.
		FVectorType Vec = FVectorType::ZeroVector;
		TDefaultNumericTypeInterface<NumericType> NumericTypeInterface{};

		// If default string value contained a fully specified 3D vector, set the vector components, otherwise leave it zero'ed.
		if (VecComponentStrings.Num() == 3)
		{
			Vec.X = NumericTypeInterface.FromString(VecComponentStrings[0], 0).Get(0);
			Vec.Y = NumericTypeInterface.FromString(VecComponentStrings[1], 0).Get(0);
			Vec.Z = NumericTypeInterface.FromString(VecComponentStrings[2], 0).Get(0);
		}

		return Vec;
	}

private:
	// Flag is true if the widget is used to represent a rotator; false otherwise
	bool bIsRotator;
};
