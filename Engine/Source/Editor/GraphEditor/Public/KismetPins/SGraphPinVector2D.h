// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SVector2DTextBox.h"
#include "ScopedTransaction.h"

class SWidget;
class UEdGraphPin;


template <typename NumericType>
class SGraphPinVector2D;

template <typename NumericType>
struct TWidgetTypeTraits<SGraphPinVector2D<NumericType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


template <typename NumericType>
class SGraphPinVector2D : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinVector2D) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
	{
		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	}

protected:

	/**
	 *	Function to create class specific widget.
	 *
	 *	@return Reference to the newly created widget object
	 */
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		// Create widget
		return SNew(SVector2DTextBox<NumericType>)
			.VisibleText_X(this, &SGraphPinVector2D::GetCurrentValue_X)
			.VisibleText_Y(this, &SGraphPinVector2D::GetCurrentValue_Y)
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
			.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
			.OnNumericCommitted_Box_X(this, &SGraphPinVector2D::OnChangedValueTextBox_X)
			.OnNumericCommitted_Box_Y(this, &SGraphPinVector2D::OnChangedValueTextBox_Y);
	}

private:

	// Enum values to identify text boxes.
	enum ETextBoxIndex
	{
		TextBox_X,
		TextBox_Y
	};

	/*
	 *	Function to get current value in text box 0
	 *
	 *	@return current string value
	 */
	FString GetCurrentValue_X() const
	{
		return FString::Printf(TEXT("%f"), GetValue().X);
	}

	/*
	 *	Function to get current value in text box 1
	 *
	 *	@return current string value
	 */
	FString GetCurrentValue_Y() const
	{
		return FString::Printf(TEXT("%f"), GetValue().Y);
	}

	/*
	 *	Function to getch current value based on text box index value
	 *
	 *	@param: Text box index
	 *
	 *	@return current value
	 */
	FVector2D GetValue() const
	{
		const FString& DefaultString = GraphPinObj->GetDefaultAsString();

		FVector2D Value;
		Value.InitFromString(DefaultString);

		return Value;
	}

	static FString MakeVector2DString(NumericType X, NumericType Y)
	{
		return FString::Format(TEXT("(X={0},Y={1})"), { X, Y });
	}

	/*
	 *	Function to store value when text box 0 value in modified
	 *
	 *	@param 0: Updated numeric value
	 */
	void OnChangedValueTextBox_X(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		const FVector2D OldValue = GetValue();
		if (NewValue != OldValue.X)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeVectorPinValue", "Change Vector Pin Value"));
			GraphPinObj->Modify();

			const FString Vector2DString = MakeVector2DString(NewValue, OldValue.Y);
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, Vector2DString);
		}
	}

	/*
	 *	Function to store value when text box 1 value in modified
	 *
	 *	@param 0: Updated numeric value
	 */
	void OnChangedValueTextBox_Y(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		if (GraphPinObj->IsPendingKill())
		{
			return;
		}

		const FVector2D OldValue = GetValue();
		if (NewValue != OldValue.Y)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeVectorPinValue", "Change Vector Pin Value"));
			GraphPinObj->Modify();

			const FString Vector2DString = MakeVector2DString(OldValue.X, NewValue);
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, Vector2DString);
		}
	}
};
