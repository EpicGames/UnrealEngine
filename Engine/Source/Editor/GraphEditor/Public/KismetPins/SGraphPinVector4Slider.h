// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Math/Vector4.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SVector4Slider.h"
#include "ScopedTransaction.h"

class SWidget;
class UEdGraphPin;

template <typename NumericType>
class SGraphPinVector4Slider : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinVector4Slider) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj, FProperty* InProperty)
	{
		PinProperty = InProperty;
		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	}

protected:

	/**
	 *	Function to create class specific widget.
	 *
	 *	@return Reference to the newly created widget object
	 */
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();

		return SNew(SVector4Slider<NumericType>, PinProperty)
			.VisibleText_0(this, &SGraphPinVector4Slider::GetCurrentValue_0)
			.VisibleText_1(this, &SGraphPinVector4Slider::GetCurrentValue_1)
			.VisibleText_2(this, &SGraphPinVector4Slider::GetCurrentValue_2)
			.VisibleText_3(this, &SGraphPinVector4Slider::GetCurrentValue_3)
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
			.IsEnabled(this, &SGraphPin::GetDefaultValueIsEditable)
			.OnNumericCommitted_Box_0(this, &SGraphPinVector4Slider::OnChangedValueTextBox_0)
			.OnNumericCommitted_Box_1(this, &SGraphPinVector4Slider::OnChangedValueTextBox_1)
			.OnNumericCommitted_Box_2(this, &SGraphPinVector4Slider::OnChangedValueTextBox_2)
			.OnNumericCommitted_Box_3(this, &SGraphPinVector4Slider::OnChangedValueTextBox_3);
	}

private:

	// Enum values to identify text boxes.
	enum ETextBoxIndex
	{
		TextBox_0,
		TextBox_1,
		TextBox_2,
		TextBox_3,
	};

	// Rotator is represented as X->Roll, Y->Pitch, Z->Yaw

	FString GetCurrentValue_0() const
	{
		// Text box 0: Rotator->Roll, Vector->X
		return GetValue(TextBox_0);
	}

	FString GetCurrentValue_1() const
	{
		// Text box 1: Rotator->Pitch, Vector->Y
		return GetValue(TextBox_1);
	}

	FString GetCurrentValue_2() const
	{
		// Text box 2: Rotator->Yaw, Vector->Z
		return GetValue(TextBox_2);
	}

	FString GetCurrentValue_3() const
	{
		// Text box 3: Vector->W
		return GetValue(TextBox_3);
	}

	/*
	*	Function to getch current value based on text box index value
	*
	*	@param: Text box index
	*
	*	@return current string value
	*/
	FString GetValue(ETextBoxIndex Index) const
	{
		FString DefaultString = GraphPinObj->GetDefaultAsString();
		TArray<FString> ResultString;

		// Parse string to split its contents separated by ','
		DefaultString.TrimStartAndEndInline();
		DefaultString.ParseIntoArray(ResultString, TEXT(","), true);

		if (Index < ResultString.Num())
		{
			return ResultString[Index];
		}
		else
		{
			return FString(TEXT("0"));
		}
	}

	/*
	*	Function to store value when text box value in modified
	*
	*	@param 0: Updated numeric value
	*/
	void OnChangedValueTextBox_0(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		const FString ValueStr = FString::Printf(TEXT("%f"), NewValue);

		FString DefaultValue;
		// Update X value
		DefaultValue = ValueStr + FString(TEXT(",")) + GetValue(TextBox_1) + FString(TEXT(",")) + GetValue(TextBox_2) + FString(TEXT(",")) + GetValue(TextBox_3);

		if (GraphPinObj->GetDefaultAsString() != DefaultValue)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeVector4PinValue", "Change Vector4 Pin Value"));
			GraphPinObj->Modify();

			// Set new default value
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, DefaultValue);
		}
	}

	void OnChangedValueTextBox_1(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		const FString ValueStr = FString::Printf(TEXT("%f"), NewValue);

		FString DefaultValue;
		// Update Y value
		DefaultValue = GetValue(TextBox_0) + FString(TEXT(",")) + ValueStr + FString(TEXT(",")) + GetValue(TextBox_2) + FString(TEXT(",")) + GetValue(TextBox_3);

		if (GraphPinObj->GetDefaultAsString() != DefaultValue)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeVector4PinValue", "Change Vector4 Pin Value"));
			GraphPinObj->Modify();

			// Set new default value
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, DefaultValue);
		}
	}

	void OnChangedValueTextBox_2(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		const FString ValueStr = FString::Printf(TEXT("%f"), NewValue);

		FString DefaultValue;
		// Update Z value
		DefaultValue = GetValue(TextBox_0) + FString(TEXT(",")) + GetValue(TextBox_1) + FString(TEXT(",")) + ValueStr + FString(TEXT(",")) + GetValue(TextBox_3);

		if (GraphPinObj->GetDefaultAsString() != DefaultValue)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeVector4PinValue", "Change Vector4 Pin Value"));
			GraphPinObj->Modify();

			// Set new default value
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, DefaultValue);
		}
	}

	void OnChangedValueTextBox_3(NumericType NewValue, ETextCommit::Type CommitInfo)
	{
		const FString ValueStr = FString::Printf(TEXT("%f"), NewValue);

		FString DefaultValue;
		// Update W value
		DefaultValue = GetValue(TextBox_0) + FString(TEXT(",")) + GetValue(TextBox_1) + FString(TEXT(",")) + GetValue(TextBox_2) + FString(TEXT(",")) + ValueStr;

		if (GraphPinObj->GetDefaultAsString() != DefaultValue)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeVector4PinValue", "Change Vector4 Pin Value"));
			GraphPinObj->Modify();

			// Set new default value
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, DefaultValue);
		}
	}

private:

	// Property to get metadata values out of for the slider widget
	FProperty* PinProperty;
};
