// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "NumericPropertyParams.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "Vector2DTextBox"

// Class implementation to create 2 editable text boxes to represent vector2D graph pin
template <typename NumericType>
class SVector2DSlider : public SCompoundWidget
{
public:

	// Notification for numeric value committed
	DECLARE_DELEGATE_TwoParams(FOnNumericValueCommitted, NumericType, ETextCommit::Type);

	SLATE_BEGIN_ARGS( SVector2DSlider ){}	
		SLATE_ATTRIBUTE( FString, VisibleText_X )
		SLATE_ATTRIBUTE( FString, VisibleText_Y )
		SLATE_EVENT( FOnNumericValueCommitted, OnNumericCommitted_Box_X )
		SLATE_EVENT( FOnNumericValueCommitted, OnNumericCommitted_Box_Y )
	SLATE_END_ARGS()

	// Construct editable text boxes with the appropriate getter & setter functions along with tool tip text
	void Construct( const FArguments& InArgs, FProperty* InProperty)
	{
		PinProperty = InProperty;
		VisibleText_X = InArgs._VisibleText_X;
		VisibleText_Y = InArgs._VisibleText_Y;
		OnNumericCommitted_X = InArgs._OnNumericCommitted_Box_X;
		OnNumericCommitted_Y = InArgs._OnNumericCommitted_Box_Y;

		const FLinearColor LabelClr = FLinearColor( 1.f, 1.f, 1.f, 0.4f );

		const typename TNumericPropertyParams<NumericType>::FMetaDataGetter MetaDataGetter = TNumericPropertyParams<NumericType>::FMetaDataGetter::CreateLambda([&](const FName& Key)
			{
				return PinProperty->GetMetaData(Key);
			});

		TNumericPropertyParams<NumericType> NumericPropertyParams(PinProperty, PinProperty ? MetaDataGetter : nullptr);

		const bool bAllowSpin = !(PinProperty && PinProperty->GetBoolMetaData("NoSpinbox"));

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth().Padding(2) .HAlign(HAlign_Fill)
				[
					// Create Text box 0 
					SNew( SNumericEntryBox<NumericType> )
					.LabelVAlign(VAlign_Center)
					.MaxFractionalDigits(3)
					.MinDesiredValueWidth(45.0f)
					.Label()
					[
						SNew( STextBlock )
						.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
						.Text( LOCTEXT("VectorNodeXAxisValueLabel", "X") )
						.ColorAndOpacity( LabelClr )
					]
					.MinValue(NumericPropertyParams.MinValue)
					.MaxValue(NumericPropertyParams.MaxValue)
					.MinSliderValue(NumericPropertyParams.MinSliderValue)
					.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
					.SliderExponent(NumericPropertyParams.SliderExponent)
					.Delta(NumericPropertyParams.Delta)
					.Value( this, &SVector2DSlider::GetTypeInValue_X )
					.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
					.AllowWheel(bAllowSpin)
					.WheelStep(NumericPropertyParams.WheelStep)
					.AllowSpin(bAllowSpin)
					.OnValueCommitted( InArgs._OnNumericCommitted_Box_X )
					.OnValueChanged(this, &SVector2DSlider::OnValueChanged_X)
					.OnBeginSliderMovement(this, &SVector2DSlider::OnBeginSliderMovement_X)
					.OnEndSliderMovement(this, &SVector2DSlider::OnEndSliderMovement_X)
					.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
					.UndeterminedString( LOCTEXT("MultipleValues", "Multiple Values") )
					.ToolTipText(LOCTEXT("VectorNodeXAxisValueLabel_ToolTip", "X value") )
					.EditableTextBoxStyle( &FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>( "Graph.VectorEditableTextBox" ))
					.BorderForegroundColor( FLinearColor::White )
					.BorderBackgroundColor( FLinearColor::White )
				]
				+SHorizontalBox::Slot()
				.AutoWidth().Padding(2) .HAlign(HAlign_Fill)
				[
					// Create Text box 1
					SNew( SNumericEntryBox<NumericType> )
					.LabelVAlign(VAlign_Center)
					.MaxFractionalDigits(3)
					.MinDesiredValueWidth(45.0f)
					.Label()
					[
						SNew( STextBlock )
						.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
						.Text( LOCTEXT("VectorNodeYAxisValueLabel", "Y") )
						.ColorAndOpacity( LabelClr )
					]
					.MinValue(NumericPropertyParams.MinValue)
					.MaxValue(NumericPropertyParams.MaxValue)
					.MinSliderValue(NumericPropertyParams.MinSliderValue)
					.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
					.SliderExponent(NumericPropertyParams.SliderExponent)
					.Delta(NumericPropertyParams.Delta)
					.Value( this, &SVector2DSlider::GetTypeInValue_Y )
					.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
					.AllowWheel(bAllowSpin)
					.WheelStep(NumericPropertyParams.WheelStep)
					.AllowSpin(bAllowSpin)
					.OnValueCommitted( InArgs._OnNumericCommitted_Box_Y )
					.OnValueChanged(this, &SVector2DSlider::OnValueChanged_Y)
					.OnBeginSliderMovement(this, &SVector2DSlider::OnBeginSliderMovement_Y)
					.OnEndSliderMovement(this, &SVector2DSlider::OnEndSliderMovement_Y)
					.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
					.UndeterminedString( LOCTEXT("MultipleValues", "Multiple Values") )
					.ToolTipText(LOCTEXT("VectorNodeYAxisValueLabel_ToolTip", "Y value"))
					.EditableTextBoxStyle( &FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>( "Graph.VectorEditableTextBox" ))
					.BorderForegroundColor( FLinearColor::White )
					.BorderBackgroundColor( FLinearColor::White )
				]
			]
		];
	}

private:

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_X(NumericType NewValue)
	{
		SliderValue_X = NewValue;
	}

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_Y(NumericType NewValue)
	{
		SliderValue_Y = NewValue;
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_X()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_X(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_X != NewValue)
		{
			OnNumericCommitted_X.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_X = NewValue;
		}
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_Y()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_Y(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_Y != NewValue)
		{
			OnNumericCommitted_Y.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_Y = NewValue;
		}
	}

	void SyncValues()
	{
		SliderValue_X = LastSliderCommittedValue_X = GetTypeInValue_X().GetValue();
		SliderValue_Y = LastSliderCommittedValue_Y = GetTypeInValue_Y().GetValue();
	}

	NumericType GetValueType(const FString& InString) const
	{
		static_assert(std::is_floating_point_v<NumericType>);

		if constexpr (std::is_same_v<float, NumericType>)
		{
			return FCString::Atof(*InString);
		}
		else if constexpr (std::is_same_v<double, NumericType>)
		{
			return FCString::Atod(*InString);
		}
		else
		{
			return NumericType{};
		}
	}

	// Get value for X text box
	TOptional<NumericType> GetTypeInValue_X() const
	{
		return bIsUsingSlider ? SliderValue_X : GetValueType(VisibleText_X.Get());
	}

	// Get value for Y text box
	TOptional<NumericType> GetTypeInValue_Y() const
	{
		return bIsUsingSlider ? SliderValue_Y : GetValueType(VisibleText_Y.Get());
	}

	TAttribute<FString> VisibleText_X;
	TAttribute<FString> VisibleText_Y;

	FProperty* PinProperty;
	FOnNumericValueCommitted OnNumericCommitted_X;
	FOnNumericValueCommitted OnNumericCommitted_Y;

	NumericType LastSliderCommittedValue_X;
	NumericType LastSliderCommittedValue_Y;
	NumericType SliderValue_X;
	NumericType SliderValue_Y;
	bool bIsUsingSlider = false;
};

#undef LOCTEXT_NAMESPACE
