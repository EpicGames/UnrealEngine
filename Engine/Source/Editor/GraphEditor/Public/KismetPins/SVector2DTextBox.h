// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "Vector2DTextBox"


template <typename NumericType>
class SVector2DTextBox;

template <typename NumericType>
struct TWidgetTypeTraits<SVector2DTextBox<NumericType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

// Class implementation to create 2 editable text boxes to represent vector2D graph pin
template <typename NumericType>
class SVector2DTextBox : public SCompoundWidget
{
public:

	// Notification for numeric value committed
	DECLARE_DELEGATE_TwoParams(FOnNumericValueCommitted, NumericType, ETextCommit::Type);

	SLATE_BEGIN_ARGS( SVector2DTextBox ){}	
		SLATE_ATTRIBUTE( FString, VisibleText_X )
		SLATE_ATTRIBUTE( FString, VisibleText_Y )
		SLATE_EVENT( FOnNumericValueCommitted, OnNumericCommitted_Box_X )
		SLATE_EVENT( FOnNumericValueCommitted, OnNumericCommitted_Box_Y )
	SLATE_END_ARGS()

	// Construct editable text boxes with the appropriate getter & setter functions along with tool tip text
	void Construct( const FArguments& InArgs )
	{
		VisibleText_X = InArgs._VisibleText_X;
		VisibleText_Y = InArgs._VisibleText_Y;
		const FLinearColor LabelClr = FLinearColor( 1.f, 1.f, 1.f, 0.4f );

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth().Padding(2) .HAlign(HAlign_Fill)
				[
					// Create Text box 0 
					SNew( SNumericEntryBox<NumericType> )
					.LabelVAlign(VAlign_Center)
					.Label()
					[
						SNew( STextBlock )
						.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
						.Text( LOCTEXT("VectorNodeXAxisValueLabel", "X") )
						.ColorAndOpacity( LabelClr )
					]
					.Value( this, &SVector2DTextBox::GetTypeInValue_X )
					.OnValueCommitted( InArgs._OnNumericCommitted_Box_X )
					.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
					.UndeterminedString( LOCTEXT("MultipleValues", "Multiple Values") )
					.ToolTipText(LOCTEXT("VectorNodeXAxisValueLabel_ToolTip", "X value") )
					.EditableTextBoxStyle( &FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>( "Graph.VectorEditableTextBox" ))
					.BorderForegroundColor( FLinearColor::White )
					.BorderBackgroundColor( FLinearColor::White )
				]
				+SHorizontalBox::Slot()
				.AutoWidth().Padding(2) .HAlign(HAlign_Fill)
				[
					// Create Text box 1
					SNew( SNumericEntryBox<NumericType> )
					.LabelVAlign(VAlign_Center)
					.Label()
					[
						SNew( STextBlock )
						.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
						.Text( LOCTEXT("VectorNodeYAxisValueLabel", "Y") )
						.ColorAndOpacity( LabelClr )
					]
					.Value( this, &SVector2DTextBox::GetTypeInValue_Y )
					.OnValueCommitted( InArgs._OnNumericCommitted_Box_Y )
					.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
					.UndeterminedString( LOCTEXT("MultipleValues", "Multiple Values") )
					.ToolTipText(LOCTEXT("VectorNodeYAxisValueLabel_ToolTip", "Y value"))
					.EditableTextBoxStyle( &FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>( "Graph.VectorEditableTextBox" ))
					.BorderForegroundColor( FLinearColor::White )
					.BorderBackgroundColor( FLinearColor::White )
				]
			]
		];
	}

private:

	NumericType GetValueType(const FString& InString) const
	{
		static_assert(std::is_floating_point_v<NumericType>);

		if constexpr (std::is_same_v<float, NumericType>)
		{
			return FCString::Atof(*InString);
		}
		else if constexpr (std::is_same_v<double, NumericType>)
		{
			return FCString::Atod(*InString);
		}
		else
		{
			return NumericType{};
		}
	}

	// Get value for X text box
	TOptional<NumericType> GetTypeInValue_X() const
	{
		return GetValueType(VisibleText_X.Get());
	}

	// Get value for Y text box
	TOptional<NumericType> GetTypeInValue_Y() const
	{
		return GetValueType(VisibleText_Y.Get());
	}

	TAttribute<FString> VisibleText_X;
	TAttribute<FString> VisibleText_Y;
};

#undef LOCTEXT_NAMESPACE
