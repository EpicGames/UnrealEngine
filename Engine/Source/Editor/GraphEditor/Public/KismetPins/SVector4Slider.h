// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "NumericPropertyParams.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "VectorTextBox"

// Class implementation to create 3 editable text boxes to represent vector/rotator graph pin
template <typename NumericType>
class SVector4Slider : public SCompoundWidget
{
public:

	// Notification for numeric value committed
	DECLARE_DELEGATE_TwoParams(FOnNumericValueCommitted, NumericType, ETextCommit::Type);

	SLATE_BEGIN_ARGS(SVector4Slider) {}
		SLATE_ATTRIBUTE(FString, VisibleText_0)
		SLATE_ATTRIBUTE(FString, VisibleText_1)
		SLATE_ATTRIBUTE(FString, VisibleText_2)
		SLATE_ATTRIBUTE(FString, VisibleText_3)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_0)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_1)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_2)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_3)
	SLATE_END_ARGS()
	
	// Construct editable text boxes with the appropriate getter & setter functions along with tool tip text
	void Construct(const FArguments& InArgs, FProperty* InProperty)
	{
		PinProperty = InProperty;
		VisibleText_0 = InArgs._VisibleText_0;
		VisibleText_1 = InArgs._VisibleText_1;
		VisibleText_2 = InArgs._VisibleText_2;
		VisibleText_3 = InArgs._VisibleText_3;
		const FLinearColor LabelClr = FLinearColor(1.f, 1.f, 1.f, 0.4f);

		OnNumericCommitted_0 = InArgs._OnNumericCommitted_Box_0;
		OnNumericCommitted_1 = InArgs._OnNumericCommitted_Box_1;
		OnNumericCommitted_2 = InArgs._OnNumericCommitted_Box_2;
		OnNumericCommitted_3 = InArgs._OnNumericCommitted_Box_3;

		const typename TNumericPropertyParams<NumericType>::FMetaDataGetter MetaDataGetter = TNumericPropertyParams<NumericType>::FMetaDataGetter::CreateLambda([&](const FName& Key)
			{
				return PinProperty->GetMetaData(Key);
			});

		TNumericPropertyParams<NumericType> NumericPropertyParams(PinProperty, PinProperty ? MetaDataGetter : nullptr);

		const bool bAllowSpin = !(PinProperty && PinProperty->GetBoolMetaData("NoSpinbox"));

		this->ChildSlot
			[
				SNew(SBox)
				.MinDesiredWidth(275)
				.MaxDesiredWidth(400)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth().Padding(2).HAlign(HAlign_Fill)
					[
						// Create Text box 0 
						SNew(SNumericEntryBox<NumericType>)
						.LabelVAlign(VAlign_Center)
						.MaxFractionalDigits(3)
						.MinDesiredValueWidth(45.0f)
						.Label()
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
							.Text(LOCTEXT("VectorNodeXAxisValueLabel", "X"))
							.ColorAndOpacity(LabelClr)
						]
						.MinValue(NumericPropertyParams.MinValue)
						.MaxValue(NumericPropertyParams.MaxValue)
						.MinSliderValue(NumericPropertyParams.MinSliderValue)
						.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
						.SliderExponent(NumericPropertyParams.SliderExponent)
						.Delta(NumericPropertyParams.Delta)
						.Value(this, &SVector4Slider::GetTypeInValue_0)
						.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
						.AllowWheel(bAllowSpin)
						.WheelStep(NumericPropertyParams.WheelStep)
						.AllowSpin(bAllowSpin)
						.OnValueCommitted(InArgs._OnNumericCommitted_Box_0)
						.OnValueChanged(this, &SVector4Slider::OnValueChanged_0)
						.OnBeginSliderMovement(this, &SVector4Slider::OnBeginSliderMovement_0)
						.OnEndSliderMovement(this, &SVector4Slider::OnEndSliderMovement_0)
						.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
						.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
						.ToolTipText(LOCTEXT("VectorNodeXAxisValueLabel_ToolTip", "X value"))
						.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
						.BorderForegroundColor(FLinearColor::White)
						.BorderBackgroundColor(FLinearColor::White)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth().Padding(2).HAlign(HAlign_Fill)
					[
						// Create Text box 1
						SNew(SNumericEntryBox<NumericType>)
						.LabelVAlign(VAlign_Center)
						.MaxFractionalDigits(3)
						.MinDesiredValueWidth(45.0f)
						.Label()
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
							.Text(LOCTEXT("VectorNodeYAxisValueLabel", "Y"))
							.ColorAndOpacity(LabelClr)
						]
						.MinValue(NumericPropertyParams.MinValue)
						.MaxValue(NumericPropertyParams.MaxValue)
						.MinSliderValue(NumericPropertyParams.MinSliderValue)
						.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
						.SliderExponent(NumericPropertyParams.SliderExponent)
						.Delta(NumericPropertyParams.Delta)
						.Value(this, &SVector4Slider::GetTypeInValue_1)
						.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
						.AllowWheel(bAllowSpin)
						.WheelStep(NumericPropertyParams.WheelStep)
						.AllowSpin(bAllowSpin)
						.OnValueCommitted(InArgs._OnNumericCommitted_Box_1)
						.OnValueChanged(this, &SVector4Slider::OnValueChanged_1)
						.OnBeginSliderMovement(this, &SVector4Slider::OnBeginSliderMovement_1)
						.OnEndSliderMovement(this, &SVector4Slider::OnEndSliderMovement_1)
						.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
						.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
						.ToolTipText(LOCTEXT("VectorNodeYAxisValueLabel_ToolTip", "Y value"))
						.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
						.BorderForegroundColor(FLinearColor::White)
						.BorderBackgroundColor(FLinearColor::White)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth().Padding(2).HAlign(HAlign_Fill)
					[
						// Create Text box 2
						SNew(SNumericEntryBox<NumericType>)
						.LabelVAlign(VAlign_Center)
						.MaxFractionalDigits(3)
						.MinDesiredValueWidth(45.0f)
						.Label()
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
							.Text(LOCTEXT("VectorNodeZAxisValueLabel", "Z"))
							.ColorAndOpacity(LabelClr)
						]
						.MinValue(NumericPropertyParams.MinValue)
						.MaxValue(NumericPropertyParams.MaxValue)
						.MinSliderValue(NumericPropertyParams.MinSliderValue)
						.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
						.SliderExponent(NumericPropertyParams.SliderExponent)
						.Delta(NumericPropertyParams.Delta)
						.Value(this, &SVector4Slider::GetTypeInValue_2)
						.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
						.AllowWheel(bAllowSpin)
						.WheelStep(NumericPropertyParams.WheelStep)
						.AllowSpin(bAllowSpin)
						.OnValueCommitted(InArgs._OnNumericCommitted_Box_2)
						.OnValueChanged(this, &SVector4Slider::OnValueChanged_2)
						.OnBeginSliderMovement(this, &SVector4Slider::OnBeginSliderMovement_2)
						.OnEndSliderMovement(this, &SVector4Slider::OnEndSliderMovement_2)
						.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
						.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
						.ToolTipText(LOCTEXT("VectorNodeZAxisValueLabel_ToolTip", "Z value"))
						.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
						.BorderForegroundColor(FLinearColor::White)
						.BorderBackgroundColor(FLinearColor::White)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth().Padding(2).HAlign(HAlign_Fill)
					[
						// Create Text box 3
						SNew(SNumericEntryBox<NumericType>)
						.LabelVAlign(VAlign_Center)
						.MaxFractionalDigits(3)
						.MinDesiredValueWidth(45.0f)
						.Label()
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
							.Text(LOCTEXT("VectorNodeWAxisValueLabel", "W"))
							.ColorAndOpacity(LabelClr)
						]
						.MinValue(NumericPropertyParams.MinValue)
						.MaxValue(NumericPropertyParams.MaxValue)
						.MinSliderValue(NumericPropertyParams.MinSliderValue)
						.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
						.SliderExponent(NumericPropertyParams.SliderExponent)
						.Delta(NumericPropertyParams.Delta)
						.Value(this, &SVector4Slider::GetTypeInValue_3)
						.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
						.AllowWheel(bAllowSpin)
						.WheelStep(NumericPropertyParams.WheelStep)
						.AllowSpin(bAllowSpin)
						.OnValueCommitted(InArgs._OnNumericCommitted_Box_3)
						.OnValueChanged(this, &SVector4Slider::OnValueChanged_3)
						.OnBeginSliderMovement(this, &SVector4Slider::OnBeginSliderMovement_3)
						.OnEndSliderMovement(this, &SVector4Slider::OnEndSliderMovement_3)
						.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
						.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
						.ToolTipText(LOCTEXT("VectorNodeWAxisValueLabel_ToolTip", "W value"))
						.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
						.BorderForegroundColor(FLinearColor::White)
						.BorderBackgroundColor(FLinearColor::White)
					]
				]
			];
		}

private:

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_0()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_0(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_0 != NewValue)
		{
			OnNumericCommitted_0.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_0 = NewValue;
		}
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_1()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_1(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_1 != NewValue)
		{
			OnNumericCommitted_1.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_1 = NewValue;
		}
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_2()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_2(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_2 != NewValue)
		{
			OnNumericCommitted_2.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_2 = NewValue;
		}
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_3()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_3(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_3 != NewValue)
		{
			OnNumericCommitted_3.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_3 = NewValue;
		}
	}

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_0(NumericType NewValue)
	{
		SliderValue_0 = NewValue;
	}

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_1(NumericType NewValue)
	{
		SliderValue_1 = NewValue;
	}

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_2(NumericType NewValue)
	{
		SliderValue_2 = NewValue;
	}

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_3(NumericType NewValue)
	{
		SliderValue_3 = NewValue;
	}

	void SyncValues()
	{
		SliderValue_0 = LastSliderCommittedValue_0 = GetTypeInValue_0().GetValue();
		SliderValue_1 = LastSliderCommittedValue_1 = GetTypeInValue_1().GetValue();
		SliderValue_2 = LastSliderCommittedValue_2 = GetTypeInValue_2().GetValue();
		SliderValue_3 = LastSliderCommittedValue_3 = GetTypeInValue_3().GetValue();
	}
	NumericType GetValueType(const FString& InString) const
	{
		static_assert(std::is_floating_point_v<NumericType>);

		if constexpr (std::is_same_v<float, NumericType>)
		{
			return FCString::Atof(*InString);
		}
		else if constexpr (std::is_same_v<double, NumericType>)
		{
			return FCString::Atod(*InString);
		}
		else
		{
			return NumericType{};
		}
	}

	TOptional<NumericType> GetTypeInValue_0() const
	{
		return bIsUsingSlider ? SliderValue_0 : GetValueType(VisibleText_0.Get());
	}

	TOptional<NumericType> GetTypeInValue_1() const
	{
		return bIsUsingSlider ? SliderValue_1 : GetValueType(VisibleText_1.Get());
	}

	TOptional<NumericType> GetTypeInValue_2() const
	{
		return bIsUsingSlider ? SliderValue_2 : GetValueType(VisibleText_2.Get());
	}

	TOptional<NumericType> GetTypeInValue_3() const
	{
		return bIsUsingSlider ? SliderValue_3 : GetValueType(VisibleText_3.Get());
	}

	TAttribute<FString> VisibleText_0;
	TAttribute<FString> VisibleText_1;
	TAttribute<FString> VisibleText_2;
	TAttribute<FString> VisibleText_3;
	FProperty* PinProperty;

	FOnNumericValueCommitted OnNumericCommitted_0;
	FOnNumericValueCommitted OnNumericCommitted_1;
	FOnNumericValueCommitted OnNumericCommitted_2;
	FOnNumericValueCommitted OnNumericCommitted_3;

	NumericType LastSliderCommittedValue_0;
	NumericType LastSliderCommittedValue_1;
	NumericType LastSliderCommittedValue_2;
	NumericType LastSliderCommittedValue_3;
	NumericType SliderValue_0;
	NumericType SliderValue_1;
	NumericType SliderValue_2;
	NumericType SliderValue_3;
	bool bIsUsingSlider = false;
};

#undef LOCTEXT_NAMESPACE
