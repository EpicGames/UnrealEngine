// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "VectorTextBox"


template <typename NumericType>
class SVector4TextBox;

template <typename NumericType>
struct TWidgetTypeTraits<SVector4TextBox<NumericType>>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


// Class implementation to create 4 editable text boxes to represent vector/rotator graph pin
template <typename NumericType>
class SVector4TextBox : public SCompoundWidget
{
public:

	// Notification for numeric value committed
	DECLARE_DELEGATE_TwoParams(FOnNumericValueCommitted, NumericType, ETextCommit::Type);

	SLATE_BEGIN_ARGS(SVector4TextBox) {}
		SLATE_ATTRIBUTE(FString, VisibleText_0)
		SLATE_ATTRIBUTE(FString, VisibleText_1)
		SLATE_ATTRIBUTE(FString, VisibleText_2)
		SLATE_ATTRIBUTE(FString, VisibleText_3)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_0)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_1)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_2)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_3)
	SLATE_END_ARGS()
	
	// Construct editable text boxes with the appropriate getter & setter functions along with tool tip text
	void Construct(const FArguments& InArgs)
	{
		VisibleText_0 = InArgs._VisibleText_0;
		VisibleText_1 = InArgs._VisibleText_1;
		VisibleText_2 = InArgs._VisibleText_2;
		VisibleText_3 = InArgs._VisibleText_3;
		const FLinearColor LabelClr = FLinearColor(1.f, 1.f, 1.f, 0.4f);

		this->ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth().Padding(2).HAlign(HAlign_Fill)
					[
						// Create Text box 0 
						SNew(SNumericEntryBox<NumericType>)
						.LabelVAlign(VAlign_Center)
						.Label()
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
							.Text(LOCTEXT("VectorNodeXAxisValueLabel", "X"))
							.ColorAndOpacity(LabelClr)
						]
						.Value(this, &SVector4TextBox::GetTypeInValue_0)
							.OnValueCommitted(InArgs._OnNumericCommitted_Box_0)
							.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
							.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
							.ToolTipText(LOCTEXT("VectorNodeXAxisValueLabel_ToolTip", "X value"))
							.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
							.BorderForegroundColor(FLinearColor::White)
							.BorderBackgroundColor(FLinearColor::White)
					]
					+ SHorizontalBox::Slot()
						.AutoWidth().Padding(2).HAlign(HAlign_Fill)
						[
							// Create Text box 1
							SNew(SNumericEntryBox<NumericType>)
							.LabelVAlign(VAlign_Center)
							.Label()
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
								.Text(LOCTEXT("VectorNodeYAxisValueLabel", "Y"))
								.ColorAndOpacity(LabelClr)
							]
							.Value(this, &SVector4TextBox::GetTypeInValue_1)
								.OnValueCommitted(InArgs._OnNumericCommitted_Box_1)
								.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
								.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
								.ToolTipText(LOCTEXT("VectorNodeYAxisValueLabel_ToolTip", "Y value"))
								.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
								.BorderForegroundColor(FLinearColor::White)
								.BorderBackgroundColor(FLinearColor::White)
						]
					+ SHorizontalBox::Slot()
						.AutoWidth().Padding(2).HAlign(HAlign_Fill)
						[
							// Create Text box 2
							SNew(SNumericEntryBox<NumericType>)
							.LabelVAlign(VAlign_Center)
							.Label()
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
								.Text(LOCTEXT("VectorNodeZAxisValueLabel", "Z"))
								.ColorAndOpacity(LabelClr)
							]
							.Value(this, &SVector4TextBox::GetTypeInValue_2)
								.OnValueCommitted(InArgs._OnNumericCommitted_Box_2)
								.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
								.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
								.ToolTipText(LOCTEXT("VectorNodeZAxisValueLabel_ToolTip", "Z value"))
								.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
								.BorderForegroundColor(FLinearColor::White)
								.BorderBackgroundColor(FLinearColor::White)
						]
					+ SHorizontalBox::Slot()
						.AutoWidth().Padding(2).HAlign(HAlign_Fill)
						[
							// Create Text box 3
							SNew(SNumericEntryBox<NumericType>)
							.LabelVAlign(VAlign_Center)
							.Label()
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
								.Text(LOCTEXT("VectorNodeWAxisValueLabel", "W"))
								.ColorAndOpacity(LabelClr)
							]
							.Value(this, &SVector4TextBox::GetTypeInValue_3)
								.OnValueCommitted(InArgs._OnNumericCommitted_Box_3)
								.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
								.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
								.ToolTipText(LOCTEXT("VectorNodeWAxisValueLabel_ToolTip", "W value"))
								.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
								.BorderForegroundColor(FLinearColor::White)
								.BorderBackgroundColor(FLinearColor::White)
						]

				]
			];
		}

private:

	NumericType GetValueType(const FString& InString) const
	{
		static_assert(std::is_floating_point_v<NumericType>);

		if constexpr (std::is_same_v<float, NumericType>)
		{
			return FCString::Atof(*InString);
		}
		else if constexpr (std::is_same_v<double, NumericType>)
		{
			return FCString::Atod(*InString);
		}
		else
		{
			return NumericType{};
		}
	}

	TOptional<NumericType> GetTypeInValue_0() const
	{
		return GetValueType(VisibleText_0.Get());
	}

	TOptional<NumericType> GetTypeInValue_1() const
	{
		return GetValueType(VisibleText_1.Get());
	}

	TOptional<NumericType> GetTypeInValue_2() const
	{
		return GetValueType(VisibleText_2.Get());
	}

	TOptional<NumericType> GetTypeInValue_3() const
	{
		return GetValueType(VisibleText_3.Get());
	}

	TAttribute<FString> VisibleText_0;
	TAttribute<FString> VisibleText_1;
	TAttribute<FString> VisibleText_2;
	TAttribute<FString> VisibleText_3;
};

#undef LOCTEXT_NAMESPACE
