// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "NumericPropertyParams.h"
#include "Editor.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "Vector2DTextBox"

// Class implementation to create 2 editable text boxes to represent vector2D graph pin
template <typename NumericType>
class SVectorSlider : public SCompoundWidget
{
public:

	// Notification for numeric value committed
	DECLARE_DELEGATE_TwoParams(FOnNumericValueCommitted, NumericType, ETextCommit::Type);

	SLATE_BEGIN_ARGS( SVectorSlider ){}	
		SLATE_ATTRIBUTE( FString, VisibleText_0 )
		SLATE_ATTRIBUTE( FString, VisibleText_1 )
		SLATE_ATTRIBUTE( FString, VisibleText_2)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_0)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_1)
		SLATE_EVENT(FOnNumericValueCommitted, OnNumericCommitted_Box_2)
	SLATE_END_ARGS()

	// Construct editable text boxes with the appropriate getter & setter functions along with tool tip text
	void Construct( const FArguments& InArgs,const bool bInIsRotator, FProperty* InProperty)
	{
		bIsRotator = bInIsRotator;
		const bool bUseRPY = false;

		PinProperty = InProperty;
		VisibleText_0 = InArgs._VisibleText_0;
		VisibleText_1 = InArgs._VisibleText_1;
		VisibleText_2 = InArgs._VisibleText_2;
		const FLinearColor LabelClr = FLinearColor(1.f, 1.f, 1.f, 0.4f);

		OnNumericCommitted_0 = InArgs._OnNumericCommitted_Box_0;
		OnNumericCommitted_1 = InArgs._OnNumericCommitted_Box_1;
		OnNumericCommitted_2 = InArgs._OnNumericCommitted_Box_2;

		const typename TNumericPropertyParams<NumericType>::FMetaDataGetter MetaDataGetter = TNumericPropertyParams<NumericType>::FMetaDataGetter::CreateLambda([&](const FName& Key)
			{
				return PinProperty->GetMetaData(Key);
			});

		TNumericPropertyParams<NumericType> NumericPropertyParams(PinProperty, PinProperty ? MetaDataGetter : nullptr);

		const bool bAllowSpin = !(PinProperty && PinProperty->GetBoolMetaData("NoSpinbox"));

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth().Padding(2) .HAlign(HAlign_Fill)
				[
					// Create Text box 0 
					SNew( SNumericEntryBox<NumericType> )
					.LabelVAlign(VAlign_Center)
					.MaxFractionalDigits(3)
					.MinDesiredValueWidth(45.0f)
					.Label()
					[
						SNew( STextBlock )
						.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
						.Text( bIsRotator && bUseRPY ? LOCTEXT("VectorNodeRollValueLabel", "R") : LOCTEXT("VectorNodeXAxisValueLabel", "X") )
						.ColorAndOpacity( LabelClr )
					]
					.MinValue(NumericPropertyParams.MinValue)
					.MaxValue(NumericPropertyParams.MaxValue)
					.MinSliderValue(NumericPropertyParams.MinSliderValue)
					.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
					.SliderExponent(NumericPropertyParams.SliderExponent)
					.Delta(NumericPropertyParams.Delta)
					.Value( this, &SVectorSlider::GetTypeInValue_0 )
					.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
					.AllowWheel(bAllowSpin)
					.WheelStep(NumericPropertyParams.WheelStep)
					.AllowSpin(bAllowSpin)
					.OnValueCommitted( InArgs._OnNumericCommitted_Box_0 )
					.OnValueChanged(this, &SVectorSlider::OnValueChanged_0)
					.OnBeginSliderMovement(this, &SVectorSlider::OnBeginSliderMovement_0)
					.OnEndSliderMovement(this, &SVectorSlider::OnEndSliderMovement_0)
					.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
					.UndeterminedString( LOCTEXT("MultipleValues", "Multiple Values") )
					.ToolTipText(bIsRotator ? LOCTEXT("VectorNodeRollValueLabel_ToolTip", "Roll value (around X)") : LOCTEXT("VectorNodeXAxisValueLabel_ToolTip", "X value"))
					.EditableTextBoxStyle( &FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>( "Graph.VectorEditableTextBox" ))
					.BorderForegroundColor( FLinearColor::White )
					.BorderBackgroundColor( FLinearColor::White )
				]
				+SHorizontalBox::Slot()
				.AutoWidth().Padding(2) .HAlign(HAlign_Fill)
				[
					// Create Text box 1
					SNew( SNumericEntryBox<NumericType> )
					.LabelVAlign(VAlign_Center)
					.MaxFractionalDigits(3)
					.MinDesiredValueWidth(45.0f)
					.Label()
					[
						SNew( STextBlock )
						.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
						.Text( bIsRotator && bUseRPY ? LOCTEXT("VectorNodePitchValueLabel", "P") : LOCTEXT("VectorNodeYAxisValueLabel", "Y") )
						.ColorAndOpacity( LabelClr )
					]
					.MinValue(NumericPropertyParams.MinValue)
					.MaxValue(NumericPropertyParams.MaxValue)
					.MinSliderValue(NumericPropertyParams.MinSliderValue)
					.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
					.SliderExponent(NumericPropertyParams.SliderExponent)
					.Delta(NumericPropertyParams.Delta)
					.Value( this, &SVectorSlider::GetTypeInValue_1 )
					// LinearDeltaSensitivity needs to be left unset if not provided, rather than being set to some default
					.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
					.AllowWheel(bAllowSpin)
					.WheelStep(NumericPropertyParams.WheelStep)
					.AllowSpin(bAllowSpin)
					.OnValueCommitted( InArgs._OnNumericCommitted_Box_1 )
					.OnValueChanged(this, &SVectorSlider::OnValueChanged_1)
					.OnBeginSliderMovement(this, &SVectorSlider::OnBeginSliderMovement_1)
					.OnEndSliderMovement(this, &SVectorSlider::OnEndSliderMovement_1)
					.Font( FAppStyle::GetFontStyle( "Graph.VectorEditableTextBox" ) )
					.UndeterminedString( LOCTEXT("MultipleValues", "Multiple Values") )
					.ToolTipText(bIsRotator ? LOCTEXT("VectorNodePitchValueLabel_ToolTip", "Pitch value (around Y)") : LOCTEXT("VectorNodeYAxisValueLabel_ToolTip", "Y value"))
					.EditableTextBoxStyle( &FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>( "Graph.VectorEditableTextBox" ))
					.BorderForegroundColor( FLinearColor::White )
					.BorderBackgroundColor( FLinearColor::White )
				]
				+ SHorizontalBox::Slot()
				.AutoWidth().Padding(2).HAlign(HAlign_Fill)
				[
					// Create Text box 2
					SNew(SNumericEntryBox<NumericType>)
					.MaxFractionalDigits(3)
					.MinDesiredValueWidth(45.0f)
					.LabelVAlign(VAlign_Center)
					.Label()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
						.Text(bIsRotator && bUseRPY ? LOCTEXT("VectorNodeYawValueLabel", "Y") : LOCTEXT("VectorNodeZAxisValueLabel", "Z"))
						.ColorAndOpacity(LabelClr)
					]
					.MinValue(NumericPropertyParams.MinValue)
					.MaxValue(NumericPropertyParams.MaxValue)
					.MinSliderValue(NumericPropertyParams.MinSliderValue)
					.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
					.SliderExponent(NumericPropertyParams.SliderExponent)
					.Delta(NumericPropertyParams.Delta)
					.Value(this, &SVectorSlider::GetTypeInValue_2)
					// LinearDeltaSensitivity needs to be left unset if not provided, rather than being set to some default
					.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
					.AllowWheel(bAllowSpin)
					.WheelStep(NumericPropertyParams.WheelStep)
					.AllowSpin(bAllowSpin)
					.OnValueCommitted(InArgs._OnNumericCommitted_Box_2)
					.OnValueChanged(this, &SVectorSlider::OnValueChanged_2)
					.OnBeginSliderMovement(this, &SVectorSlider::OnBeginSliderMovement_2)
					.OnEndSliderMovement(this, &SVectorSlider::OnEndSliderMovement_2)
					.Font(FAppStyle::GetFontStyle("Graph.VectorEditableTextBox"))
					.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
					.ToolTipText(bIsRotator ? LOCTEXT("VectorNodeYawValueLabel_Tooltip", "Yaw value (around Z)") : LOCTEXT("VectorNodeZAxisValueLabel_ToolTip", "Z value"))
					.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Graph.VectorEditableTextBox"))
					.BorderForegroundColor(FLinearColor::White)
					.BorderBackgroundColor(FLinearColor::White)
				]
			]
		];
	}

private:

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_0()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_0(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_0 != NewValue)
		{
			OnNumericCommitted_0.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_0 = NewValue;
		}
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_1()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_1(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_1 != NewValue)
		{
			OnNumericCommitted_1.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_1 = NewValue;
		}
	}

	/**
	* Called when the slider begins to move.
	*/
	void OnBeginSliderMovement_2()
	{
		SyncValues();
		bIsUsingSlider = true;
	}

	/**
	* Called when the slider stops moving.
	*/
	void OnEndSliderMovement_2(NumericType NewValue)
	{
		bIsUsingSlider = false;
		if (LastSliderCommittedValue_2 != NewValue)
		{
			OnNumericCommitted_2.ExecuteIfBound(NewValue, ETextCommit::Default);
			LastSliderCommittedValue_2 = NewValue;
		}
	}

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_0(NumericType NewValue)
	{
		SliderValue_0 = NewValue;
	}

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_1(NumericType NewValue)
	{
		SliderValue_1 = NewValue;
	}

	/**
	* Called when the slider value changes
	*/
	void OnValueChanged_2(NumericType NewValue)
	{
		SliderValue_2 = NewValue;
	}

	void SyncValues()
	{
		SliderValue_0 = LastSliderCommittedValue_0 = GetTypeInValue_0().GetValue();
		SliderValue_1 = LastSliderCommittedValue_1 = GetTypeInValue_1().GetValue();
		SliderValue_2 = LastSliderCommittedValue_2 = GetTypeInValue_2().GetValue();
	}

	NumericType GetValueType(const FString& InString) const
	{
		static_assert(std::is_floating_point_v<NumericType>);

		if constexpr (std::is_same_v<float, NumericType>)
		{
			return FCString::Atof(*InString);
		}
		else if constexpr (std::is_same_v<double, NumericType>)
		{
			return FCString::Atod(*InString);
		}
		else
		{
			return NumericType{};
		}
	}

	// Get value for X text box
	TOptional<NumericType> GetTypeInValue_0() const
	{
		return bIsUsingSlider ? SliderValue_0 :GetValueType(VisibleText_0.Get());
	}

	// Get value for Y text box
	TOptional<NumericType> GetTypeInValue_1() const
	{
		return bIsUsingSlider ? SliderValue_1 : GetValueType(VisibleText_1.Get());
	}
	// Get value for Y text box
	TOptional<NumericType> GetTypeInValue_2() const
	{
		return bIsUsingSlider ? SliderValue_2 : GetValueType(VisibleText_2.Get());
	}


	TAttribute<FString> VisibleText_0;
	TAttribute<FString> VisibleText_1;
	TAttribute<FString> VisibleText_2;

	FProperty* PinProperty;
	FOnNumericValueCommitted OnNumericCommitted_0;
	FOnNumericValueCommitted OnNumericCommitted_1;
	FOnNumericValueCommitted OnNumericCommitted_2;

	NumericType LastSliderCommittedValue_0;
	NumericType LastSliderCommittedValue_1;
	NumericType LastSliderCommittedValue_2;
	NumericType SliderValue_0;
	NumericType SliderValue_1;
	NumericType SliderValue_2;
	bool bIsUsingSlider = false;
	bool bIsRotator;
};

#undef LOCTEXT_NAMESPACE
