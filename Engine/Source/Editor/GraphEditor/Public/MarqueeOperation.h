// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GraphEditor.h"
#include "Framework/MarqueeRect.h"

struct FMarqueeOperation;
struct Rect;

/** Helper for managing marquee operations */
struct FMarqueeOperation
{
	FMarqueeOperation()
		: Operation(Add)
	{
	}

	enum Type
	{
		/** Holding down Alt removes nodes */
		Remove,
		/** Holding down Shift adds to the selection */
		Add,
		/** When nothing is pressed, marquee replaces selection */
		Replace,
		/** Holding down Ctrl toggles the selection state of all encompassed nodes */
		Invert,
	} Operation;

	bool IsValid() const
	{
		return Rect.IsValid();
	}

	void Start(const UE::Slate::FDeprecateVector2DParameter& InStartLocation, FMarqueeOperation::Type InOperationType)
	{
		Rect = FMarqueeRect(InStartLocation);
		Operation = InOperationType;
	}

	void End()
	{
		Rect = FMarqueeRect();
	}


	/** Given a mouse event, figure out what the marquee selection should do based on the state of Shift and Ctrl keys */
	static FMarqueeOperation::Type OperationTypeFromMouseEvent(const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.IsControlDown())
		{
			return FMarqueeOperation::Invert;
		}
		else if (MouseEvent.IsShiftDown())
		{
			return FMarqueeOperation::Add;
		}
		else if (MouseEvent.IsAltDown())
		{
			return FMarqueeOperation::Remove;
		}
		else
		{
			return FMarqueeOperation::Replace;
		}
	}

public:
	/** The marquee rectangle being dragged by the user */
	FMarqueeRect Rect;

	/** Nodes that will be selected or unselected by the current marquee operation */
	FGraphPanelSelectionSet AffectedNodes;
};


