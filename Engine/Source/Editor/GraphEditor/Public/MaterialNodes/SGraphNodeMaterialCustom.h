// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.6, "This header has moved to the MaterialEditor module. Use MaterialEditor/MaterialNodes/SGraphNodeMaterialCustom.h.")

// Include the new file so that the project still compiles, but has warnings
#include "MaterialEditor/MaterialNodes/SGraphNodeMaterialCustom.h"