// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

#define UE_API GRAPHEDITOR_API

class FConnectionDrawingPolicy;
class FSlateRect;
class FSlateWindowElementList;
class SGraphNode;
class SGraphPin;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UEdGraphSchema;

/** Class that decides which widget type to create for a given data object */
class FNodeFactory
{
public:
	/** Create a widget for the supplied node */
	static UE_API TSharedPtr<SGraphNode> CreateNodeWidget(UEdGraphNode* InNode);

	/** Create a widget for the supplied pin */
	static UE_API TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* InPin);

	/** Create a K2 schema default widget for the supplied pin */
	static UE_API TSharedPtr<SGraphPin> CreateK2PinWidget(UEdGraphPin* InPin);

    /** Create a pin connection factory for the supplied schema */
    static UE_API FConnectionDrawingPolicy* CreateConnectionPolicy(const UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);
};


class FGraphNodeFactory
{
public:
	/** Create a widget for the supplied node */
	UE_API virtual TSharedPtr<SGraphNode> CreateNodeWidget(UEdGraphNode* InNode);

	/** Create a widget for the supplied pin */
	UE_API virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* InPin);

	/** Create a pin connection factory for the supplied schema */
	UE_API virtual FConnectionDrawingPolicy* CreateConnectionPolicy(const UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);
};

#undef UE_API
