// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Input/CursorReply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SNodePanel.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API GRAPHEDITOR_API

class SMultiLineEditableTextBox;
class UEdGraphNode;
struct FGeometry;
struct FPointerEvent;

DECLARE_DELEGATE_RetVal( bool, FIsGraphNodeHovered );
DECLARE_DELEGATE_OneParam( FOnCommentBubbleToggled, bool );


template <>
struct TWidgetTypeTraits<class SCommentBubble>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SCommentBubble : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SCommentBubble )
		: _GraphNode( NULL )
		, _ColorAndOpacity( FLinearColor::White )
		, _AllowPinning( false )
		, _EnableTitleBarBubble( false )
		, _EnableBubbleCtrls( false )
		, _InvertLODCulling( false )
		, _GraphLOD( EGraphRenderingLOD::DefaultDetail )
	{}

		/** the GraphNode this bubble should interact with */
		SLATE_ARGUMENT( UEdGraphNode*, GraphNode )

		/** Attribute to determine the visibility buttons check state */
		SLATE_ATTRIBUTE( ECheckBoxState, ToggleButtonCheck )

		/** The comment text for the bubble */
		SLATE_ATTRIBUTE( FString, Text )

		/** Called when the comment text is committed */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Called when the comment bubble is toggled */
		SLATE_EVENT( FOnCommentBubbleToggled, OnToggled )

		/** The comment hint text for the bubble */
		SLATE_ATTRIBUTE( FText, HintText )

		/** Color and opacity for the comment bubble */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/** Allow bubble to be pinned */
		SLATE_ARGUMENT( bool, AllowPinning )

		/** Enable the title bar bubble to toggle */
		SLATE_ARGUMENT( bool, EnableTitleBarBubble )

		/** Enable the controls within the bubble */
		SLATE_ARGUMENT( bool, EnableBubbleCtrls )

		/** Invert LOD culling */
		SLATE_ARGUMENT( bool, InvertLODCulling )

		/** The current level of detail */
		SLATE_ATTRIBUTE( EGraphRenderingLOD::Type, GraphLOD )

		/** Delegate to determine if the parent node is in the hovered state. */
		SLATE_EVENT( FIsGraphNodeHovered, IsGraphNodeHovered )

	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 */
	UE_API void Construct( const FArguments& InArgs );


	//~ Begin SWidget Interface
	UE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	//~ End SWidget Interface

	/** Updates the comment Visibility */
	UE_API void TickVisibility(const double InCurrentTime, const float InDeltaTime);

	/** Returns the offset from the SNode center slot */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API FVector2D GetOffset() const;
	UE_API FVector2f GetOffset2f() const;

	/** Returns the offset to the arrow center accounting for zoom on either the comment bubble or the title bar button based on current state */
	UE_API float GetArrowCenterOffset() const;

	/** Returns the bubble size */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API FVector2D GetSize() const;
	UE_API FVector2f GetSize2f() const;

	/** Returns if comment bubble is visible */
	UE_API bool IsBubbleVisible() const;

	/** Returns if graph scaling can be applied to this bubble */
	UE_API bool IsScalingAllowed() const;

	/** Called to display/hide the comment bubble */
	UE_API void OnCommentBubbleToggle( ECheckBoxState State );

	/** Directly sets the bubble's visibility, without assuming it was from a user action (no undo transaction, or OnToggled callback)*/
	UE_API void SetCommentBubbleVisibility(bool bVisible);

	/** Called when a node's comment bubble pinned state is changed */
	UE_API void OnPinStateToggle( ECheckBoxState State ) const;

	/** Called to update the bubble widget layout */
	UE_API void UpdateBubble();

	/** Returns if the text block currently has keyboard focus*/
	UE_API bool TextBlockHasKeyboardFocus() const;

protected:

	/** Returns the current scale button tooltip */
	UE_API FText GetScaleButtonTooltip() const;

	/** Called to determine if the comment bubble's current visibility */
	UE_API EVisibility GetBubbleVisibility() const;

	/** Called to determine if the toggle button's current visibility */
	UE_API EVisibility GetToggleButtonVisibility() const;

	/** Returns the color for the toggle bubble including the opacity value */
	UE_API FSlateColor GetToggleButtonColor() const;

	/** Returns the color of the main bubble */
	UE_API FSlateColor GetBubbleColor() const;

	/** Called when the comment text is committed */
	UE_API void OnCommentTextCommitted( const FText& NewText, ETextCommit::Type CommitInfo );

	UE_API FSlateColor GetTextBackgroundColor() const;
	UE_API FSlateColor GetTextForegroundColor() const;
	UE_API FSlateColor GetReadOnlyTextForegroundColor() const;
	/** Returns bubble toggle check state */
	UE_API ECheckBoxState GetToggleButtonCheck() const;

	/** Returns pinned check state */
	UE_API ECheckBoxState GetPinnedButtonCheck() const;

	/** Called to determine if the comment bubble is readonly */
	UE_API bool IsReadOnly() const;

protected:

	/** The GraphNode this widget interacts with */
	UEdGraphNode* GraphNode;
	/** Cached inline editable text box */
	TSharedPtr<SMultiLineEditableTextBox> TextBlock;

	/** The Comment Bubble color and opacity value */
	TAttribute<FSlateColor> ColorAndOpacity;
	/** Attribute to query node comment */
	TAttribute<FString> CommentAttribute;
	/** Attribute to query current LOD */
	TAttribute<EGraphRenderingLOD::Type> GraphLOD;
	/** Hint Text */
	TAttribute<FText> HintText;
	/** Toggle button checked state  */
	TAttribute<ECheckBoxState> ToggleButtonCheck;

	/** Optional delegate to call when the comment text is committed */
	FOnTextCommitted OnTextCommittedDelegate;
	/** Optional delegate to call when the comment bubble is toggled */
	FOnCommentBubbleToggled OnToggledDelegate;
	/** Delegate to determine if the graph node is currently hovered */
	FIsGraphNodeHovered IsGraphNodeHovered;

	/** The luminance (R + G + B) of the bubble's color, used to control text foreground color */
	float BubbleLuminance;
	/** Allow pin behaviour */
	bool bAllowPinning;
	/** Allow in bubble controls */
	bool bEnableBubbleCtrls;
	/** Invert the LOD culling behavior, used by comment nodes */
	bool bInvertLODCulling;
	/** Enable the title bar bubble toggle */
	bool bEnableTitleBarBubble;
	/** Used to Control hover fade up/down for widgets */
	float OpacityValue;
	/** Cached comment */
	FString CachedComment;
	/** Cached FText comment */
	FText CachedCommentText;

};

#undef UE_API
