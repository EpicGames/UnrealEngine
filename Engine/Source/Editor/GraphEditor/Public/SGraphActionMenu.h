// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define UE_API GRAPHEDITOR_API

class FReferenceCollector;
class ITableRow;
class IToolTip;
class SEditableTextBox;
class SPanel;
class SSearchBox;
class SWidget;
class UEdGraph;
class UEdGraphPin;
struct FCreateWidgetForActionData;
struct FGeometry;
struct FGraphActionNode;
struct FKeyEvent;
struct FPointerEvent;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;

/** Delegate for hooking up an inline editable text block to be notified that a rename is requested. */
DECLARE_DELEGATE( FOnRenameRequestActionNode );
/** Delegate executed when the mouse button goes down */
DECLARE_DELEGATE_RetVal_OneParam( bool, FCreateWidgetMouseButtonDown, TWeakPtr<FEdGraphSchemaAction> );

/** Default widget for GraphActionMenu */
class SDefaultGraphActionWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SDefaultGraphActionWidget ) {}
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FCreateWidgetForActionData* InCreateData);
	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	/** The item that we want to display with this widget */
	TWeakPtr<FEdGraphSchemaAction> ActionPtr;
	/** Delegate executed when mouse button goes down */
	FCreateWidgetMouseButtonDown MouseButtonDownDelegate;
};

struct FCreateWidgetForActionData
{
	/** True if we want to use the mouse delegate */
	bool bHandleMouseButtonDown;
	/** Delegate for mouse button going down */
	FCreateWidgetMouseButtonDown MouseButtonDownDelegate;
	/** The action being used for the widget */
	TSharedPtr< FEdGraphSchemaAction > Action;
	/** The delegate to determine if the current action is selected in the row */
	FIsSelected IsRowSelectedDelegate;
	/** This will be returned, hooked up to request a rename */
	FOnRenameRequestActionNode* const OnRenameRequest;
	/** The text to highlight */
	TAttribute<FText> HighlightText;
	/** True if the widget should be read only - no renaming allowed */
	bool bIsReadOnly;

	FCreateWidgetForActionData(FOnRenameRequestActionNode* const InOnRenameRequest)
		: OnRenameRequest(InOnRenameRequest)
		, bIsReadOnly(false)
	{

	}
};

struct FCustomExpanderData
{
	/** The menu row associated with the widget being customized */
	TSharedPtr<ITableRow> TableRow;

	/** The action associated with the menu row being customized */
	TSharedPtr<FEdGraphSchemaAction> RowAction;

	/** The widget container that the custom expander will belong to */
	TSharedPtr<SPanel> WidgetContainer;
};

/** Class that displays a list of graph actions and them to be searched and selected */
class SGraphActionMenu : public SCompoundWidget, public FGCObject
{
public:
	/** Delegate that can be used to create a widget for a particular action */
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<SWidget>, FOnCreateWidgetForAction, FCreateWidgetForActionData* const );
	/** Delegate that can be used to create a custom "expander" widget for a particular row */
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<SExpanderArrow>, FOnCreateCustomRowExpander, FCustomExpanderData const& );
	/** Delegate executed when an action is selected */
	DECLARE_DELEGATE_TwoParams( FOnActionSelected, const TArray< TSharedPtr<FEdGraphSchemaAction> >&, ESelectInfo::Type );
	/** Delegate executed when an action is double clicked */
	DECLARE_DELEGATE_OneParam( FOnActionDoubleClicked, const TArray< TSharedPtr<FEdGraphSchemaAction> >& );
	/** Delegate executed when an action is dragged */
	DECLARE_DELEGATE_RetVal_TwoParams( FReply, FOnActionDragged, const TArray< TSharedPtr<FEdGraphSchemaAction> >&, const FPointerEvent& );
	/** Delegate executed when a category is dragged */
	DECLARE_DELEGATE_RetVal_TwoParams( FReply, FOnCategoryDragged, const FText&, const FPointerEvent& );
	/** Delegate executed when the list of all actions needs to be refreshed */
	DECLARE_DELEGATE_OneParam( FOnCollectAllActions, FGraphActionListBuilderBase& );
	/** Delegate executed when the list of all actions needs to be refreshed, should return any sections that should always be visible, even if they don't have children. */
	DECLARE_DELEGATE_OneParam( FOnCollectStaticSections, TArray<int32>& )
	/** Delegate executed when a category is being renamed so any post-rename actions can be handled */
	DECLARE_DELEGATE_ThreeParams( FOnCategoryTextCommitted, const FText&, ETextCommit::Type, TWeakPtr< FGraphActionNode >);
	/** Delegate executed to check if the selected action is valid for renaming */
	DECLARE_DELEGATE_RetVal_OneParam( bool, FCanRenameSelectedAction, TWeakPtr< FGraphActionNode > );
	/** Delegate to get the name of a section if the widget is a section separator. */
	DECLARE_DELEGATE_RetVal_OneParam( FText, FGetSectionTitle, int32 );
	/** Delegate to get the tooltip of a section if the widget is a section separator. */
	DECLARE_DELEGATE_RetVal_OneParam( TSharedPtr<IToolTip>, FGetSectionToolTip, int32 );
	/** Delegate to get the widget that appears on the section bar in the section separator. */
	DECLARE_DELEGATE_RetVal_TwoParams( TSharedRef<SWidget>, FGetSectionWidget, TSharedRef<SWidget>, int32 );
	/** Delegate to get the filter text */
	DECLARE_DELEGATE_RetVal( FText, FGetFilterText);
	/** Delegate to check if an action matches a specified name (used for renaming items etc.) */
	DECLARE_DELEGATE_RetVal_TwoParams( bool, FOnActionMatchesName, FEdGraphSchemaAction*, const FName& );
	/** Delegate that can be used to create and/or get a custom action list. */
	DECLARE_DELEGATE_RetVal(TSharedRef<FGraphActionListBuilderBase>, FOnGetActionList);

	SLATE_BEGIN_ARGS(SGraphActionMenu)
		: _AutoExpandActionMenu(false)
		, _AlphaSortItems(true)
		, _SortItemsRecursively(true)
		, _ShowFilterTextBox(true)
		, _UseSectionStyling(false)
		, _bAllowPreselectedItemActivation(false)
		, _DefaultRowExpanderBaseIndentLevel(0)
		, _GraphObj(nullptr)
		, _bAutomaticallySelectSingleAction(false)
		{ }

		SLATE_EVENT( FOnActionSelected, OnActionSelected )
		SLATE_EVENT( FOnActionDoubleClicked, OnActionDoubleClicked )
		SLATE_EVENT( FOnActionDragged, OnActionDragged )
		SLATE_EVENT( FOnCategoryDragged, OnCategoryDragged )
		SLATE_EVENT( FOnContextMenuOpening, OnContextMenuOpening )
		SLATE_EVENT( FOnCreateWidgetForAction, OnCreateWidgetForAction )
		SLATE_EVENT( FOnCreateCustomRowExpander, OnCreateCustomRowExpander )
		SLATE_EVENT( FOnGetActionList, OnGetActionList )
		SLATE_EVENT( FOnCollectAllActions, OnCollectAllActions )
		SLATE_EVENT( FOnCollectStaticSections, OnCollectStaticSections )
		SLATE_EVENT( FOnCategoryTextCommitted, OnCategoryTextCommitted )
		SLATE_EVENT( FCanRenameSelectedAction, OnCanRenameSelectedAction )
		SLATE_EVENT( FGetSectionTitle, OnGetSectionTitle )
		SLATE_EVENT( FGetSectionToolTip, OnGetSectionToolTip )
		SLATE_EVENT( FGetSectionWidget, OnGetSectionWidget )
		SLATE_EVENT( FGetFilterText, OnGetFilterText )
		SLATE_EVENT( FOnActionMatchesName, OnActionMatchesName )
		SLATE_ARGUMENT( bool, AutoExpandActionMenu )
		SLATE_ARGUMENT( bool, AlphaSortItems )
		SLATE_ARGUMENT( bool, SortItemsRecursively )
		SLATE_ARGUMENT( bool, ShowFilterTextBox )
		SLATE_ARGUMENT( bool, UseSectionStyling )
		SLATE_ARGUMENT( bool, bAllowPreselectedItemActivation )
		SLATE_ARGUMENT( int32, DefaultRowExpanderBaseIndentLevel )
		SLATE_ARGUMENT( TArray<UEdGraphPin*>, DraggedFromPins )
		SLATE_ARGUMENT( UEdGraph*, GraphObj )
		SLATE_ARGUMENT( bool, bAutomaticallySelectSingleAction )

	SLATE_END_ARGS()

	UE_API void Construct( const FArguments& InArgs, bool bIsReadOnly = true );

	// FGCObject override
	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	UE_API virtual FString GetReferencerName() const override;

	/**
	 * Refreshes the actions that this widget should display
	 *
	 * @param bPreserveExpansion		TRUE if the expansion state of the tree should be preserved
	 * @param bHandleOnSelectionEvent		TRUE if the item should be selected and any actions that occur with selection will be handled. FALSE and only selection will occur */
	UE_API void RefreshAllActions(bool bPreserveExpansion, bool bHandleOnSelectionEvent = true);

	/** Returns a map of all top level sections and their current expansion state. */
	UE_API void GetSectionExpansion(TMap<int32, bool>& SectionExpansion) const;

	/** Sets the sections to be expanded of all top level sections. */
	UE_API void SetSectionExpansion(const TMap<int32, bool>& SectionExpansion);

protected:
	/** Tree view for showing actions */
	TSharedPtr< STreeView< TSharedPtr<FGraphActionNode> > > TreeView;
	/** Text box used for searching for actions */
	TSharedPtr<SSearchBox> FilterTextBox;

	/** List of all actions we can browser */
	TSharedPtr<FGraphActionListBuilderBase> AllActions;

	/** Flattened list of all actions passing the filter - access via GetFilteredActionNodes to ensure it's up to date */
	TArray< TSharedPtr<FGraphActionNode> > FilteredActionNodes;

	/** Root of filtered actions tree */
	TSharedPtr<FGraphActionNode> FilteredRootAction;

	/** Stored score for our current selection, so that we can quickly maintain selection when building the list asynchronously */
	float SelectedSuggestionScore;
	/** Stored index in the AllActions list builder that we think is the best fit */
	int32 SelectedSuggestionSourceIndex;
	/** The actual item selected - possibly redundant to selection information in the treeview, but the tree is rebuilt requently */
	TSharedPtr<FGraphActionNode> SelectedAction;

	/** Allows us to set selection (via keyboard) without triggering action */
	bool bIgnoreUIUpdate;
	/** Should we auto-expand categories */
	bool bAutoExpandActionMenu;
	/** Should we display the filter text box */
	bool bShowFilterTextBox;
	/** Don't sort items alphabetically */
	bool bAlphaSortItems;
	/** If only the top entries should be sorted or subentries as well */
	bool bSortItemsRecursively;
	/** Should the rows and sections be styled like the details panel? */
	bool bUseSectionStyling;
	/** True if the user is using the keyboard to navigatethe list, halting scoring of newly added entries so that selection is stable */
	bool bIsKeyboardNavigating;
	/** Whether we allow pre-selected items to be activated with a left-click */
	bool bAllowPreselectedItemActivation;
	/** Whether to automatically proceed with an action if it's the only one in the list. */
	bool bAutomaticallySelectSingleAction;
	/** The BaseIndentLevel of the default-created row expander. Not used with OnCreateCustomRowExpander. */
	int32 DefaultRowExpanderBaseIndentLevel;
	/** The slot that the selected item should be displayed at */
	int32 DisplayIndex;
	
	/** Delegate to call when action is selected */
	FOnActionSelected OnActionSelected;
	/** Delegate to call when action is double clicked */
	FOnActionDoubleClicked OnActionDoubleClicked;
	/** Delegate to call when an action is dragged. */
	FOnActionDragged OnActionDragged;
	/** Delegate to call when a category is dragged. */
	FOnCategoryDragged OnCategoryDragged;
	/** Delegate to call to create widget for an action */
	FOnCreateWidgetForAction OnCreateWidgetForAction;
	/** Delegate to call for creating a custom "expander" widget for indenting a menu row with */
	FOnCreateCustomRowExpander OnCreateCustomRowExpander;
	/** Delegate to call to get a custom action list */
	FOnGetActionList OnGetActionList;
	/** Delegate to call to collect all actions */
	FOnCollectAllActions OnCollectAllActions;
	/** Delegate to call to collect all always visible sections */
	FOnCollectStaticSections OnCollectStaticSections;
	/** Delegate to call to handle any post-category rename events */
	FOnCategoryTextCommitted OnCategoryTextCommitted;
	/** Delegate to call to check if a selected action is valid for renaming */
	FCanRenameSelectedAction OnCanRenameSelectedAction;
	/** Delegate to get the name of a section separator. */
	FGetSectionTitle OnGetSectionTitle;
	/** Delegate to get the tooltip of a section separator. */
	FGetSectionToolTip OnGetSectionToolTip;
	/** Delegate to get the widgets of a section separator. */
	FGetSectionWidget OnGetSectionWidget;
	/** Delegate to get the filter text if supplied from an external source */
	FGetFilterText OnGetFilterText;
	/** Delegate to check if an action matches a specified name (used for renaming items etc.) */
	FOnActionMatchesName OnActionMatchesName;

public:
	// SWidget interface
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override;
	// End of SWidget interface

	/** Get filter text box widget */
	UE_API TSharedRef<SEditableTextBox> GetFilterTextBox();

	/** Get action that is currently selected */
	UE_API void GetSelectedActions(TArray< TSharedPtr<FEdGraphSchemaAction> >& OutSelectedActions) const;

	/** Initiates a rename on the selected action node, if possible */
	UE_API void OnRequestRenameOnActionNode();

	/** Queries if a rename on the selected action node is possible */
	UE_API bool CanRequestRenameOnActionNode() const;

	/** Get category that is currently selected */
	UE_API FString GetSelectedCategoryName() const;
	
	/** Get category child actions that is currently selected */
	UE_API void GetSelectedCategorySubActions(TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const;

	/** Get category child actions for the passed in action */
	UE_API void GetCategorySubActions(TWeakPtr<FGraphActionNode> InAction, TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const;

	/**
	 * Selects an non-creation item in the list, searching by FName, deselects if name is none 
	 *
	 * @param	ItemName		The name of the item to select
	 * @param	SelectInfo		The selection type
	 * @param	SectionId		If known, the section Id to restrict the selection to, useful in the case of categories where they can exist multiple times
	 * @param	bIsCategory		TRUE if the selection is a category, categories obey different rules and it's hard to re-select properly without this knowledge
	 * @return					TRUE if the item was successfully selected or the tree cleared, FALSE if unsuccessful
	 */
	UE_API bool SelectItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo = ESelectInfo::Direct, int32 SectionId = INDEX_NONE, bool bIsCategory = false );

	/** Expands any category with the associated name */
	UE_API void ExpandCategory(const FText& CategoryName);

	/* Handler for mouse button going down */
	UE_API bool OnMouseButtonDownEvent( TWeakPtr<FEdGraphSchemaAction> InAction );

	/** Updates the displayed list starting from IdxStart, useful for async building the display list of actions */
	UE_API void UpdateForNewActions(int32 IdxStart);
	/** Regenerated filtered results (FilteredRootAction) based on filter text  */ 
	UE_API void GenerateFilteredItems(bool bPreserveExpansion);

	/** The last typed action within the graph action menu */
	static UE_API FString LastUsedFilterText;

protected:
	/** Get current filter text */
	UE_API FText GetFilterText() const;

	/** Change the selection to reflect the active suggestion */
	UE_API void MarkActiveSuggestion();

	/** Try to spawn the node reflected by the active suggestion */
	UE_API bool TryToSpawnActiveSuggestion();

	/** Returns true if the tree should be autoexpanded */
	UE_API bool ShouldExpandNodes() const;

	/** Checks if the passed in node is safe for renaming */
	UE_API bool CanRenameNode(TWeakPtr<FGraphActionNode> InNode) const;

	// Delegates

	/** Called when filter text changes */
	UE_API void OnFilterTextChanged( const FText& InFilterText );
	/** Called when enter is hit in search box */
	UE_API void OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);
	/** Get children  */
	UE_API void OnGetChildrenForCategory( TSharedPtr<FGraphActionNode> InItem, TArray< TSharedPtr<FGraphActionNode> >& OutChildren );
	/** Create widget for the supplied node */
	UE_API TSharedRef<ITableRow> MakeWidget( TSharedPtr<FGraphActionNode> InItem, const TSharedRef<STableViewBase>& OwnerTable, bool bIsReadOnly );

	/**
	 * Called when tree item is selected 
	 *
	 * @param	InSelectedItem	The action node that is being selected
	 * @param	SelectInfo		Selection type - Only OnMouseClick and OnKeyPress will trigger a call to HandleSelection
	 */
	UE_API void OnItemSelected( TSharedPtr< FGraphActionNode > InSelectedItem, ESelectInfo::Type SelectInfo );

	/**
	 * Executes the selection delegate providing it has been bound, and the provided action node given is valid and is an action node
	 *
	 * @param InselectedItem	The graph action node selected
	 *
	 * @return true if item selection delegate was executed
	 */
	UE_API bool HandleSelection( TSharedPtr< FGraphActionNode > &InSelectedItem, ESelectInfo::Type InSelectionType );

	/** Called when tree item is double clicked */
	UE_API void OnItemDoubleClicked( TSharedPtr< FGraphActionNode > InClickedItem );
	/** Called when tree item dragged */
	UE_API FReply OnItemDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );
	/** Callback when rename text is committed */
	UE_API void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, TWeakPtr< FGraphActionNode > InAction);
	/** Handler for when an item has scrolled into view after having been requested to do so */
	UE_API void OnItemScrolledIntoView( TSharedPtr<FGraphActionNode> InActionNode, const TSharedPtr<ITableRow>& InWidget );
	/** Callback for expanding tree items recursively */
	UE_API void OnSetExpansionRecursive(TSharedPtr<FGraphActionNode> InTreeNode, bool bInIsItemExpanded);
	/** Helper function for adding and scoring actions from our builder */
	UE_API void ScoreAndAddActions(int32 StartingIndex = INDEX_NONE);

	/** Helper functions for keyboard interaction */
	UE_API void SelectPreviousAction(int32 Num = 1);
	UE_API void SelectNextAction(int32 Num = 1);
	UE_API void SelectFirstAction();
	UE_API void SelectLastAction();
	/** General helper functions for accessing the filtered tree */
	UE_API TSharedPtr<FGraphActionNode> GetFirstAction();
	UE_API const TArray< TSharedPtr<FGraphActionNode> >& GetFilteredActionNodes(int32* OutSelectedIndex = nullptr);
	UE_API int32 GetTotalLeafNodes() const;
private:
	/** The pins that have been dragged off of to prompt the creation of this action menu. */
	TArray<UEdGraphPin*> DraggedFromPins;

	/** The graph that this menu is being constructed in */
	UEdGraph* GraphObj;
};

#undef UE_API
