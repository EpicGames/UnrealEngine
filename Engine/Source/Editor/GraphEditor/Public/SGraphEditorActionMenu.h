// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Widgets/Input/SEditableTextBox.h"
#endif
#include "SGraphActionMenu.h"
#include "Widgets/Layout/SBorder.h"

class SEditableTextBox;
class SGraphActionMenu;
class UEdGraph;
class UEdGraphPin;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;

/////////////////////////////////////////////////////////////////////////////////////////////////

class SGraphEditorActionMenu : public SBorder
{
public:
	SLATE_BEGIN_ARGS( SGraphEditorActionMenu )
		: _GraphObj( static_cast<UEdGraph*>(NULL) )
		, _NewNodePosition( FVector2f::ZeroVector )
		, _AutoExpandActionMenu( false )
		{}

		SLATE_ARGUMENT( UEdGraph*, GraphObj )
		SLATE_ARGUMENT( FDeprecateSlateVector2D, NewNodePosition )
		SLATE_ARGUMENT( TArray<UEdGraphPin*>, DraggedFromPins )
		SLATE_ARGUMENT( SGraphEditor::FActionMenuClosed, OnClosedCallback )
		SLATE_ARGUMENT( bool, AutoExpandActionMenu )
		SLATE_EVENT( SGraphActionMenu::FOnCreateWidgetForAction, OnCreateWidgetForAction )
	SLATE_END_ARGS()

	GRAPHEDITOR_API void Construct( const FArguments& InArgs );

	GRAPHEDITOR_API ~SGraphEditorActionMenu();

	GRAPHEDITOR_API void RefreshAllActions();

	GRAPHEDITOR_API TSharedRef<SEditableTextBox> GetFilterTextBox();

protected:
	UEdGraph* GraphObj;
	TArray<UEdGraphPin*> DraggedFromPins;
	FDeprecateSlateVector2D NewNodePosition;
	bool AutoExpandActionMenu;

	SGraphEditor::FActionMenuClosed OnClosedCallback;
	SGraphActionMenu::FOnCreateWidgetForAction OnCreateWidgetForAction;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	GRAPHEDITOR_API void OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType );

	/** Callback used to populate all actions list in SGraphActionMenu */
	GRAPHEDITOR_API void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
};
