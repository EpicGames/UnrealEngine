// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "BlueprintUtilities.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "GraphEditor.h"
#include "HAL/PlatformMath.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SNodePanel.h"
#include "SlateFwd.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class FActorDragDropOp;
class FDragDropEvent;
class ISlateStyle;
class IToolTip;
class SGraphPanel;
class SGraphPin;
class SInlineEditableTextBlock;
class SLevelOfDetailBranchNode;
class SToolTip;
class SVerticalBox;
class SWidget;
class UEdGraphNode;
class UEdGraphPin;
class UObject;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

/////////////////////////////////////////////////////
// SNodeTitle

template <>
struct TWidgetTypeTraits<class SNodeTitle>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SNodeTitle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNodeTitle)
		: _StyleSet(&FAppStyle::Get())
		, _Style(TEXT("Graph.Node.NodeTitle"))
		, _ExtraLineStyle(TEXT("Graph.Node.NodeTitleExtraLines"))
		{}

		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)

		// The style of the text block, which dictates the font, color, and shadow options. Style overrides all other properties!
		SLATE_ARGUMENT(FName, Style)

		// The style of any additional lines in the the text block
		SLATE_ARGUMENT(FName, ExtraLineStyle)

		// Title text to display, auto-binds to get the title if not set externally
		SLATE_ATTRIBUTE(FText, Text)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphNode* InNode);

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	/** Returns the main title for the node */
	UE_API FText GetHeadTitle() const;

	/** Get the size of this title the last time it was drawn */
	UE_API UE::Slate::FDeprecateVector2DResult GetTitleSize() const;

	/** Mark the cached title dirty */
	UE_API void MarkDirty();

protected:
	TWeakObjectPtr<UEdGraphNode> GraphNode;
	FNodeTextCache NodeTitleCache;
	FName ExtraLineStyle;
	const ISlateStyle* StyleSet;

	/** The cached head title to return */
	FText CachedHeadTitle;

	/** The title text to use, auto-binds to get the title if not set externally */
	TAttribute< FText > TitleText;

	/** The cached size of the title */
	FVector2f CachedSize;

protected:
	// Gets the expected node title
	UE_API FText GetNodeTitle() const;

	// Rebuilds the widget if needed
	UE_API void RebuildWidget();
};

/////////////////////////////////////////////////////
// SGraphNode

template <>
struct TWidgetTypeTraits<class SGraphNode>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphNode : public SNodePanel::SNode
{
public:
	// SWidget interface
	UE_API virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	UE_API virtual TSharedPtr<IToolTip> GetToolTip() override;
	UE_API virtual void OnToolTipClosing() override;

	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	// SNodePanel::SNode interface
	UE_API virtual void MoveTo( const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true ) override;
	UE_API virtual FVector2f GetPosition2f() const override;
	UE_API virtual FVector2f GetDesiredSizeForMarquee2f() const override;
	UE_API virtual FString GetNodeComment() const override;
	UE_API virtual UObject* GetObjectBeingDisplayed() const override;
	// End of SNodePanel::SNode interface

	/** Set attribute for determining if widget is editable */
	UE_API void SetIsEditable(TAttribute<bool> InIsEditable);

	/** Returns if widget is editable, additionally considers if the owning graph is read only */
	UE_API virtual bool IsNodeEditable() const;

	/** Set event when node is double clicked */
	UE_API void SetDoubleClickEvent(FSingleNodeEvent InDoubleClickEvent);

	/** @param OwnerPanel  The GraphPanel that this node belongs to */
	UE_API virtual void SetOwner( const TSharedRef<SGraphPanel>& OwnerPanel );

	/** @return the editable title for a node */
	UE_API FString GetEditableNodeTitle() const;

	/** @return the editable title for a node */
	UE_API FText GetEditableNodeTitleAsText() const;

	/** @return the tint for the node's title image */
	UE_API virtual FSlateColor GetNodeTitleColor() const;

	/** @return the tint for the node's comment */
	UE_API FSlateColor GetNodeCommentColor() const;

	/** @return the tint for the node's main body */
	UE_API virtual FSlateColor GetNodeBodyColor() const;

	/** @return the image brush to use for the node */
	UE_API virtual const FSlateBrush * GetNodeBodyBrush() const;

	/** @return the tint for the node's title icon */
	UE_API virtual FSlateColor GetNodeTitleIconColor() const;

	/** @return the tint for the node's title text */
	UE_API virtual FLinearColor GetNodeTitleTextColor() const;

	/** @return the tooltip to display when over the node */
	UE_API virtual FText GetNodeTooltip() const;
	
	/** @return the node being observed by this widget*/
	UE_API UEdGraphNode* GetNodeObj() const;

	/** @return the node under the mouse (either this node or one of its children) */
	UE_API virtual TSharedRef<SGraphNode> GetNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	UE_API TSharedPtr<SGraphPanel> GetOwnerPanel() const;

	/**
	 * Update this GraphNode to match the data that it is observing
	 */
	UE_API virtual void UpdateGraphNode();
	
	/** Create the widgets for pins on the node */
	UE_API virtual void CreatePinWidgets();

	/** Create a single pin widget */
	UE_API virtual void CreateStandardPinWidget(UEdGraphPin* Pin);

	/**
	 * Get all the pins found on this node.
	 *
	 * @param AllPins  The set of pins found on this node.
	 */
	UE_API void GetPins( TSet< TSharedRef<SWidget> >& AllPins ) const;
	UE_API void GetPins( TArray< TSharedRef<SWidget> >& AllPins ) const;

	/** 
	 * Find the pin that is hovered.
	 *
	 * @param MyGeometry  The geometry of the node
	 * @param MouseEvent  Information about the mouse
	 *
	 * @return A pointer to the pin widget hovered in this node, or invalid pointer if none.
	 */
	UE_API virtual TSharedPtr<SGraphPin> GetHoveredPin( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) const;

	UE_API TSharedPtr<SGraphPin> FindWidgetForPin( UEdGraphPin* ThePin ) const;

	UE_API void PlaySpawnEffect();

	/** Given a coordinate in SGraphNode space, return the same coordinate in graph space while taking zoom and panning of the parent graph into account */
	UE_API UE::Slate::FDeprecateVector2DResult NodeCoordToGraphCoord( const UE::Slate::FDeprecateVector2DParameter& PanelSpaceCoordinate ) const;

	UE_API UE::Slate::FDeprecateVector2DResult GetContentScale() const;
	UE_API FLinearColor GetColorAndOpacity() const;
	UE_API FLinearColor GetPinLabelColorAndOpacity() const;	

	/** Set event when text is committed on the node */
	UE_API void SetVerifyTextCommitEvent(FOnNodeVerifyTextCommit InOnVerifyTextCommit);
	/** Set event when text is committed on the node */
	UE_API void SetTextCommittedEvent(FOnNodeTextCommitted InDelegate);
	/** Set event when the user generates a warning tooltip because a connection was invalid */
	UE_API void SetDisallowedPinConnectionEvent(SGraphEditor::FOnDisallowedPinConnection InOnDisallowedPinConnection);
	/** called to replace this nodes comment text */
	UE_API virtual void OnCommentTextCommitted(const FText& NewComment, ETextCommit::Type CommitInfo);
	/** called when the node's comment bubble is toggled */
	UE_API virtual void OnCommentBubbleToggled(bool bInCommentBubbleVisible);
	/** returns true if a rename is pending on this node */
	bool IsRenamePending() const { return bRenameIsPending; }

	/** Requests a rename when the node was initially spawned */
	virtual void RequestRenameOnSpawn()
	{
		RequestRename();
	}

	/** flags node as rename pending if supported */
	UE_API void RequestRename();

	/** Sets node into rename state if supported */
	UE_API void ApplyRename();

	/** Cancels the rename editing mode */
	UE_API void CancelRename();

	/** return rect of the title area */
	UE_API virtual FSlateRect GetTitleRect() const;

	/** Called from drag drop code when a disallowed connection is hovered */
	UE_API void NotifyDisallowedPinConnection(const class UEdGraphPin* PinA, const class UEdGraphPin* PinB) const;

	/** Gets the unscaled position of the node from the last tick */
	UE::Slate::FDeprecateVector2DResult GetUnscaledPosition() const {return CachedUnscaledPosition;}

	/** Returns the current Node LOD or Highest LOD if unable to query */
	UE_API EGraphRenderingLOD::Type GetCurrentLOD() const;

	/** Called when GraphNode changes its error information, may be called when no change has actually occurred: */
	UE_API virtual void RefreshErrorInfo();
	
	/** Called when ed graph data is cleared, indicating this widget can no longer safely access GraphNode - forwards call to owned pins: */
	UE_API void InvalidateGraphData();

	/** Returns true if the node is hiding its pins */
	virtual bool IsHidingPinWidgets() const { return false; }

	/** Allows the node to override the pin border image */
	virtual TOptional<const FSlateBrush*> GetPinBorder(const SGraphPin* InGraphPin) const { return TOptional<const FSlateBrush*>(); }

	/** Allows the node to override the pin highlight color */
	virtual TOptional<FSlateColor> GetHighlightColor(const SGraphPin* InGraphPin) const { return TOptional<FSlateColor>(); }

	/** Allows the node to override the pin text color */
	virtual TOptional<FSlateColor> GetPinTextColor(const SGraphPin* InGraphPin) const { return TOptional<FSlateColor>(); }

	// snapping:
	TArray<FArrangedWidget> GetArrangedSnappedChildNodes(const FGeometry& Geometry);
	TSharedPtr<SGraphNode> GetSnappedChildForPosition(const FGeometry& Geometry, const FVector2f& Position, FVector2f& OutPositionInNode);
	FGeometry GetNodeGeometry(const FGeometry& Geometry) const;
	void DrawChildrenNodeShadow( const FGraphPanelSelectionSet& SelectionSet, FSlateWindowElementList& OutDrawElements, int32 NodeShadowsLayerId, const FGeometry& Geometry, FLinearColor Color);
	/** Returns the set of pins that should have their connections drawn, including nested pins */
	void GetPinsToDraw( TSet< TSharedRef<SWidget> >& AllPins ) const;
	/** Returns all of the pins, including those nested within this node */
	void GetPinsIncludingSnappedChildren( TSet< TSharedRef<SWidget> >& AllPins ) const;
	void ForEachNodeIncludingChildren(TFunctionRef<void(const TSharedRef<SGraphNode>&)> Ftor);
	void ForEachPinOnSnappedChild(TFunctionRef<void(const TSharedRef<SGraphPin>&)> Ftor) const;
	void ForEachNodeSnappedToControlFlowPin(TFunctionRef<void(const TSharedRef<SGraphNode>&)> Ftor) const;
	void ForEachNodeSnappedToDataPin(TFunctionRef<void(const TSharedRef<SGraphNode>&)> Ftor) const;
protected:
	UE_API SGraphNode();

	UE_API void PositionThisNodeBetweenOtherNodes(const TMap< UObject*, TSharedRef<SNode> >& NodeToWidgetLookup, UEdGraphNode* PreviousNode, UEdGraphNode* NextNode, float HeightAboveWire) const;
	UE_API void PositionThisNodeBetweenOtherNodes(const TMap< UObject*, TSharedRef<SNode> >& NodeToWidgetLookup, TSet<UEdGraphNode*>& PreviousNodes, TSet<UEdGraphNode*>& NextNodes, float HeightAboveWire) const;
	UE_API void PositionThisNodeBetweenOtherNodes(const FVector2f& PrevPos, const FVector2f& NextPos, float HeightAboveWire) const;

	/**
	 * Check whether drag and drop functionality is permitted on the given node 
	 */
	static UE_API bool CanAllowInteractionUsingDragDropOp( const UEdGraphNode* GraphNodePtr, const TSharedPtr<FActorDragDropOp>& DragDropOp );

	/**
	 *	Function to get error description string
	 *
	 *	@return string to be displayed as tooltip.
	 */
	UE_API FText GetErrorMsgToolTip() const;

	/**
	 * Add a new pin to this graph node. The pin must be newly created.
	 *
	 * @param PinToAdd   A new pin to add to this GraphNode.
	 */
	UE_API virtual void AddPin( const TSharedRef<SGraphPin>& PinToAdd );

	/** Hook that allows derived classes to supply their own SGraphPin derivatives for any pin. */
	UE_API virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const;

	/**
	 * Override this to provide support for an 'expensive' tooltip widget that is only built on demand
	 */
	virtual TSharedPtr<SToolTip> GetComplexTooltip() { return NULL; }

	// Override this to add widgets below the node and pins
	virtual void CreateBelowWidgetControls(TSharedPtr<SVerticalBox> MainBox) {}

	// Override this to add widgets below the pins but above advanced view arrow
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) {}

	/** Determines how the node title overflow is handled */
	virtual TOptional<ETextOverflowPolicy> GetNameOverflowPolicy() const { return {}; }

	/* Helper function to check if node can be renamed */
	UE_API virtual bool IsNameReadOnly () const;

	/** Called when text is being committed to check for validity */
	UE_API bool OnVerifyNameTextChanged ( const FText& InText, FText& OutErrorMessage );

	/* Called when text is committed on the node */
	UE_API void OnNameTextCommited ( const FText& InText, ETextCommit::Type CommitInfo ) ;

	/* Helper function to set the error color for the node */
	FSlateColor GetErrorColor() const	{return ErrorColor;}

	/** Controls wether to show or not the visual warning message */
	UE_API EVisibility VisualWarningVisibility() const;

	/** Function to get a visual warning description string(visual warnings does not break the build) */
	UE_API FText GetVisualWarningMsgToolTip() const;

	/* Helper function to set the error color for the node */
	UE_API FSlateColor GetVisualWarningColor() const;

	/** Helper function to get any error text for the node */
	FString GetErrorMessage() const {return ErrorMsg;}

	/** Called to set error text on the node */
	UE_API virtual void UpdateErrorInfo();

	/** Set-up the error reporting widget for the node */
	UE_API void SetupErrorReporting();

	// Should we use low-detail node titles?
	UE_API virtual bool UseLowDetailNodeTitles() const;

public:
	
	// Should we use low-detail pin names?
	virtual bool UseLowDetailPinNames() const { return false; }

protected:
	
	/** Return the desired comment bubble color */
	virtual FSlateColor GetCommentColor() const { return FLinearColor::White; }

	///// ADVANCED VIEW FUNCTIONS /////

	/** Create button to show/hide advanced pins */
	UE_API virtual void CreateAdvancedViewArrow(TSharedPtr<SVerticalBox> MainBox);

	/** Returns visibility of AdvancedViewButton */
	UE_API virtual EVisibility AdvancedViewArrowVisibility() const;

	/** Show/hide advanced view */
	UE_API virtual void OnAdvancedViewChanged( const ECheckBoxState NewCheckedState );

	/** hidden == unchecked, shown == checked */
	UE_API virtual ECheckBoxState IsAdvancedViewChecked() const;

	/** Up when shown, down when hidden */
	UE_API virtual const FSlateBrush* GetAdvancedViewArrow() const;

	/** Checks if the node is the only node selected */
	UE_API bool IsSelectedExclusively() const;

	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) {}
	UE_API virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle);
	
	/** Optionally create a widget to dock to the right in the title bar of the node. */
	UE_API virtual TSharedRef<SWidget> CreateTitleRightWidget();
	
	/** Create the inner node content area, including the left/right pin boxes */
	UE_API virtual TSharedRef<SWidget> CreateNodeContentArea();

	///// ADD PIN BUTTON FUNCTIONS /////

	/** Override this to create a button to add pins on the input side of the node */
	virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox) {};

	/** Override this to create a button to add pins on the output side of the node */
	virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) {};

	/** Creates widget for an Add pin button, which can then be added to the node */
	UE_API TSharedRef<SWidget> AddPinButtonContent(FText PinText, FText PinTooltipText, bool bRightSide = true, FString DocumentationExcerpt = FString(), TSharedPtr<SToolTip> CustomTooltip = NULL);

	/** Checks whether Add pin button should currently be visible */
	UE_API virtual EVisibility IsAddPinButtonVisible() const;

	/** Callback function executed when Add pin button is clicked */
	virtual FReply OnAddPin() {return FReply::Handled();}

	/* Populate a meta data tag with information about this graph node */
	UE_API virtual void PopulateMetaTag(class FGraphNodeMetaData* TagMeta) const;

	/** Returns TRUE if the input pin should be hidden from view */
	UE_API bool ShouldPinBeHidden(const UEdGraphPin* InPin) const;

	/** Returns the widget to use for the enabled state of the node */
	UE_API virtual TSharedPtr<SWidget> GetEnabledStateWidget() const;

	/** Returns the widget to use for the corner icon of the node */
	UE_API virtual TSharedRef<SWidget> CreateIconWidget();

protected:
	/** Input pin widgets on this node */
	TArray< TSharedRef<SGraphPin> > InputPins;
	/** Output pin widgets on this node */
	TArray< TSharedRef<SGraphPin> > OutputPins;
	/** The GraphPanel within in which this node resides.*/
	TWeakPtr<SGraphPanel> OwnerGraphPanelPtr;
	/** The GraphNode being observed by this widget */
	UEdGraphNode* GraphNode;
	/** The area where input pins reside */
	TSharedPtr<SVerticalBox> LeftNodeBox;
	/** The area where output pins reside */
	TSharedPtr<SVerticalBox> RightNodeBox;
	/** Used to display the name of the node and allow renaming of the node */
	TSharedPtr<SInlineEditableTextBlock> InlineEditableText;
	/** Error handling widget */
	TSharedPtr<class IErrorReportingWidget> ErrorReporting;
	/** Visual Warning handling widget */
	TSharedPtr<class IErrorReportingWidget> VisualWarningReporting;

	FCurveSequence SpawnAnim;
	FCurveHandle ZoomCurve;
	FCurveHandle FadeCurve;

	/* The margin used by the border containing the title. Can be changed by child classes if necessary.
	*  The extra padding on the right is for making the color spill stretch well past the node title */
	FMargin TitleBorderMargin = FMargin(10.f, 5.f, 30.f, 3.f);

	/** Is this node editable */
	TAttribute<bool> IsEditable;

	FSingleNodeEvent OnDoubleClick;

	// Is the current tooltip a complex one that should be dropped when the tooltip is no longer displayed?
	bool bProvidedComplexTooltip;

	// Is a rename operation pending
	bool bRenameIsPending;
	/** Called whenever the text on the node is being committed interactively by the user, validates the string for commit */
	FOnNodeVerifyTextCommit OnVerifyTextCommit;
	/** Called whenever the text on the node is committed interactively by the user */
	FOnNodeTextCommitted OnTextCommitted;
	/** Called when the user generates a warning tooltip because a connection was invalid */
	SGraphEditor::FOnDisallowedPinConnection OnDisallowedPinConnection;
	/** Used to report errors on the node */
	FString ErrorMsg;
	/** Used to set the error color */
	FSlateColor ErrorColor;
	/** Used to report visual warnings on the node (does not break build) */
	FString VisualWarningMsg;
	/** Used to set the soft error color */
	FSlateColor VisualWarningColor;


	/** Caches true position of node */
	FVector2f CachedUnscaledPosition;

	/** Cached icon color for the node */
	FLinearColor IconColor;

	/** Cached pointer to graph editor settings */
	const class UGraphEditorSettings* Settings;

private:
	TSharedPtr<SLevelOfDetailBranchNode> TitleLODBranchNode;
	friend struct FNodeSnappingManager; // want access to ControlFlowSequence, can reconsider design if snapping is successful
	TSharedPtr<SVerticalBox> ControlFlowSequence; // ref?
	TWeakPtr<SGraphNode> SnapParent; // dag so technically we can keep an owning reference.. but do we trust ourselves?
};

#undef UE_API
