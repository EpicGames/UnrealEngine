// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Layout/SlateRect.h"
#include "Math/Vector2D.h"
#include "SGraphNodeResizable.h"
#include "SNodePanel.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class FDragDropEvent;
class SBorder;
class SCommentBubble;
class SGraphNode;
class UEdGraphNode_Comment;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;


template <>
struct TWidgetTypeTraits<class SGraphNodeComment>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphNodeComment : public SGraphNodeResizable
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeComment){}
	SLATE_END_ARGS()

	//~ Begin SWidget Interface
	UE_API virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	UE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	//~ End SWidget Interface

	//~ Begin SNodePanel::SNode Interface
	UE_API virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	UE_API virtual void GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const override;
	virtual bool ShouldAllowCulling() const override { return true; }
	UE_API virtual int32 GetSortDepth() const override;
	UE_API virtual void EndUserInteraction() const override;
	UE_API virtual FString GetNodeComment() const override;
	//~ End SNodePanel::SNode Interface

	//~ Begin SPanel Interface
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~ End SPanel Interface

	//~ Begin SGraphNode Interface
	UE_API virtual bool IsNameReadOnly() const override;
	virtual FSlateColor GetCommentColor() const override { return GetCommentBodyColor(); }
	/** Requests a rename when the node was initially spawned */
	virtual void RequestRenameOnSpawn() 
	{ 
		RequestRename(); 
		ApplyRename(); 
	}
	//~ End SGraphNode Interface

	UE_API void Construct( const FArguments& InArgs, UEdGraphNode_Comment* InNode );

	/** return if the node can be selected, by pointing given location */
	UE_API virtual bool CanBeSelected( const FVector2f& MousePositionInNode ) const override;

	/** return size of the title bar */
	UE_API virtual FVector2f GetDesiredSizeForMarquee2f() const override;

	/** return rect of the title bar */
	UE_API virtual FSlateRect GetTitleRect() const override;

protected:
	//~ Begin SGraphNode Interface
	UE_API virtual void UpdateGraphNode() override;
	UE_API virtual void PopulateMetaTag(class FGraphNodeMetaData* TagMeta) const override;

	/**
	 * Helper method to update selection state of comment and any nodes 'contained' within it
	 * @param bSelected	If true comment is being selected, false otherwise
	 * @param bUpdateNodesUnderComment If true then force the rebuild of the list of nodes under the comment
	 */
	UE_API void HandleSelection(bool bIsSelected, bool bUpdateNodesUnderComment = false) const;

	/** Helper function to determine if a node is under this comment widget or not */
	UE_API virtual bool IsNodeUnderComment(UEdGraphNode_Comment* InCommentNode, const TSharedRef<SGraphNode> InNodeWidget) const;

	/** called when user is moving the comment node */
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;

	//~ Begin SGraphNodeResizable Interface
	UE_API virtual float GetTitleBarHeight() const override;
	UE_API virtual FSlateRect GetHitTestingBorder() const override;
	UE_API virtual FVector2f GetNodeMaximumSize2f() const override;
	//~ Begin SGraphNodeResizable Interface

	/** @return the color to tint the comment body */
	UE_API FSlateColor GetCommentBodyColor() const;

	/** @return the color to tint the title bar */
	UE_API FSlateColor GetCommentTitleBarColor() const;

	/** @return the color to tint the comment bubble */
	UE_API FSlateColor GetCommentBubbleColor() const;

private:
	
	/** Returns the width to wrap the text of the comment at */
	UE_API float GetWrapAt() const;

	/** The comment bubble widget (used when zoomed out) */
	TSharedPtr<SCommentBubble> CommentBubble;

	/** The current selection state of the comment */
	mutable bool bIsSelected;

	/** the title bar, needed to obtain it's height */
	TSharedPtr<SBorder> TitleBar;

protected:
	/** cached comment title */
	FString CachedCommentTitle;

	/** cached font size */
	int32 CachedFontSize;

	/** Was the bubble desired to be visible last frame? */
	mutable bool bCachedBubbleVisibility;

private:
	/** cached comment title */
	int32 CachedWidth;

	/** Local copy of the comment style */
	FInlineEditableTextBlockStyle CommentStyle;
};

#undef UE_API
