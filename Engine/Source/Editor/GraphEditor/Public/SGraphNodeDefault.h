// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "SGraphNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class UEdGraphNode;


template <>
struct TWidgetTypeTraits<class SGraphNodeDefault>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphNodeDefault : public SGraphNode
{
public:

	SLATE_BEGIN_ARGS( SGraphNodeDefault )
		: _GraphNodeObj( static_cast<UEdGraphNode*>(NULL) )
		{}

		SLATE_ARGUMENT( UEdGraphNode*, GraphNodeObj )
	SLATE_END_ARGS()

	UE_API void Construct( const FArguments& InArgs );
};

#undef UE_API
