// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "SGraphNodeResizable.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class SWidget;
class UEdGraphNode;
struct FGeometry;
struct FPointerEvent;

class SGraphNodeDocumentation : public SGraphNodeResizable
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeDocumentation){}
	SLATE_END_ARGS()

	//~ Begin SPanel Interface
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~ End SPanel Interface

	//~ Begin SWidget Interface
	UE_API void Construct( const FArguments& InArgs, UEdGraphNode* InNode );
	UE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	//~ End SWidget Interface

	//~ Begin SGraphNodeResizable Interface
	UE_API virtual FVector2f GetNodeMinimumSize2f() const override;
	UE_API virtual FVector2f GetNodeMaximumSize2f() const override;
	UE_API virtual float GetTitleBarHeight() const override;
	UE_API virtual FSlateRect GetHitTestingBorder() const override;
	//~ End SGraphNodeResizable Interface

protected:

	//~ Begin SGraphNode Interface
	UE_API virtual void UpdateGraphNode() override;
	virtual bool IsNameReadOnly () const override { return false; }
	//~ End SGraphNode Interface

	/** Retrieves the current documentation title based on the chosen excerpt */
	UE_API FText GetDocumentationTitle() const;
	/** Create documentation page from link and excerpt */
	UE_API TSharedPtr<SWidget> CreateDocumentationPage();
	/** Returns the width the documentation content must adhere to, used as a delegate in child widgets */
	UE_API FOptionalSize GetContentWidth() const;
	/** Returns the height the documentation content must adhere to, used as a delegate in child widgets */
	UE_API FOptionalSize GetContentHeight() const;
	/** Returns the child widget text wrapat size, used as a delegate during creation of documentation page */
	UE_API float GetDocumentationWrapWidth() const;
	/** Returns the current child widgets visibility for hit testing */
	UE_API EVisibility GetWidgetVisibility() const;
	/** Color of the page gradient start */
	UE_API FLinearColor GetPageGradientStartColor() const;
	/** Color of the page gradient end */
	UE_API FLinearColor GetPageGradientEndColor() const;

private:

	/** Node Title Bar */
	TSharedPtr<SWidget> TitleBar;
	/** Documentation excerpt maximum page size */
	FVector2f DocumentationSize;
	/** Content Widget with the desired size being managed */
	TSharedPtr<SWidget> ContentWidget;
	/** Tracks if child widgets are availble to hit test against */
	EVisibility ChildWidgetVisibility;
	/** Cached Documentation Link */
	FString CachedDocumentationLink;
	/** Cached Documentation Excerpt */
	FString CachedDocumentationExcerpt;

};

#undef UE_API
