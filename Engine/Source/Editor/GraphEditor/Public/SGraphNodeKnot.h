// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Vector2D.h"
#include "SGraphNodeDefault.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class FText;
class SCommentBubble;
class SGraphPin;
class UEdGraphPin;
struct FSlateBrush;


template <>
struct TWidgetTypeTraits<class SGraphNodeKnot>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** The visual representation of a control point meant to adjust how connections are routed, also known as a Reroute node.
 * The input knot node should have properly implemented ShouldDrawNodeAsControlPointOnly to return true with valid indices for its pins.
 */
class SGraphNodeKnot : public SGraphNodeDefault
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeKnot) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, class UEdGraphNode* InKnot);

	// SGraphNode interface
	UE_API virtual void UpdateGraphNode() override;
	UE_API virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	UE_API virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	UE_API virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual void RequestRenameOnSpawn() override { }
	// End of SGraphNode interface

protected:
	/** Returns Offset to center comment on the node's only pin */
	UE_API FVector2f GetCommentOffset() const;
protected:

	/** Toggles the hovered visibility state */
	UE_API virtual void OnCommentBubbleToggled(bool bInCommentBubbleVisible) override;

	/** If bHoveredCommentVisibility is true, hides the comment bubble after a change is committed */
	UE_API virtual void OnCommentTextCommitted(const FText& NewComment, ETextCommit::Type CommitInfo) override;

	/** The hovered visibility state. If false, comment bubble will only appear on hover. */
	bool bAlwaysShowCommentBubble;

	/** SharedPtr to comment bubble */
	TSharedPtr<SCommentBubble> CommentBubble;

	const FSlateBrush* ShadowBrush;
	const FSlateBrush* ShadowBrushSelected;
};


template <>
struct TWidgetTypeTraits<class SGraphPinKnot>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPinKnot : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinKnot) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

	// SWidget interface
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

protected:
	// Begin SGraphPin interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	UE_API virtual TSharedRef<FDragDropOperation> SpawnPinDragEvent(const TSharedRef<SGraphPanel>& InGraphPanel, const TArray< TSharedRef<SGraphPin> >& InStartingPins) override;
	UE_API virtual FReply OnPinMouseDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FSlateColor GetPinColor() const override;
	// End SGraphPin interface
};

#undef UE_API
