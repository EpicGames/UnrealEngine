// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_PromotableOperator.h"
#include "KismetNodes/SGraphNodeK2Sequence.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UK2Node_PromotableOperator;
struct FSlateBrush;

/**
 * A custom slate node for the promotable operator (K2Node_PromotableOperator)
 * These nodes include common math operations like Add, Subtract, Multiply, etc.
 * 
 * This custom node will provide a neat little secondary icon to visually show that
 * promotable operator pins can be converted to other types.
 */
class SGraphNodePromotableOperator : public SGraphNodeK2Sequence
{
public:

	SLATE_BEGIN_ARGS(SGraphNodePromotableOperator){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UK2Node_PromotableOperator* InNode);
	
	// SGraphNode interface
	virtual void CreatePinWidgets() override;
	// End of SGraphNode interface

protected:

	void LoadCachedIcons();

	const FSlateBrush* CachedOuterIcon;
	const FSlateBrush* CachedInnerIcon;
};
