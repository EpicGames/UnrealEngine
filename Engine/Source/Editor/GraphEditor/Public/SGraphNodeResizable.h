// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/SlateRect.h"
#include "Math/Vector2D.h"
#include "SGraphNode.h"
#include "Templates/SharedPointer.h"

#define UE_API GRAPHEDITOR_API

class FScopedTransaction;
struct FGeometry;
struct FPointerEvent;


template <>
struct TWidgetTypeTraits<class SGraphNodeResizable>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphNodeResizable : public SGraphNode
{
public:

	/**
	 * The resizable window zone the user is interacting with
	 */
	enum EResizableWindowZone
	{
		CRWZ_NotInWindow		= 0,
		CRWZ_InWindow			= 1,
		CRWZ_RightBorder		= 2,
		CRWZ_BottomBorder		= 3,
		CRWZ_BottomRightBorder	= 4,
		CRWZ_LeftBorder			= 5,
		CRWZ_TopBorder			= 6,
		CRWZ_TopLeftBorder		= 7,
		CRWZ_TopRightBorder		= 8,
		CRWZ_BottomLeftBorder	= 9,
		CRWZ_TitleBar			= 10,
	};

	//~ Begin SWidget Interface
	UE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	//~ End SWidget Interface
	
protected:

	/** Find the current window zone the mouse is in */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual SGraphNodeResizable::EResizableWindowZone FindMouseZone(const FVector2D& LocalMouseCoordinates) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual SGraphNodeResizable::EResizableWindowZone FindMouseZone(const FVector2f& LocalMouseCoordinates) const;

	/** @return true if the current window zone is considered a selection area */
	bool InSelectionArea() const { return InSelectionArea(MouseZone); }

	/** @return true if the passed zone is a selection area */
	UE_API bool InSelectionArea(EResizableWindowZone InZone) const;

	/** Function to store anchor point before resizing the node. The node will be anchored to this point when resizing happens*/
	UE_API void InitNodeAnchorPoint();

	/** Function to fetch the corrected node position based on anchor point*/
	UE_API UE::Slate::FDeprecateVector2DResult GetCorrectedNodePosition() const;

	/** Get the current titlebar size */
	UE_API virtual float GetTitleBarHeight() const;

	/** Return smallest desired node size */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual FVector2D GetNodeMinimumSize() const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual FVector2f GetNodeMinimumSize2f() const;

	/** Return largest desired node size */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API virtual FVector2D GetNodeMaximumSize() const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual FVector2f GetNodeMaximumSize2f() const;

	//** Return slate rect border for hit testing */
	UE_API virtual FSlateRect GetHitTestingBorder() const;

protected:

	/** The non snapped size of the node for fluid resizing */
	FDeprecateSlateVector2D DragSize;

	/** The desired size of the node set during a drag */
	FDeprecateSlateVector2D UserSize;

	/** The original size of the node while resizing */
	FDeprecateSlateVector2D StoredUserSize;

	/** The resize transaction */
	TSharedPtr<FScopedTransaction> ResizeTransactionPtr;

	/** Anchor point used to correct node position on resizing the node*/
	FDeprecateSlateVector2D NodeAnchorPoint;

	/** The current window zone the mouse is in */
	EResizableWindowZone MouseZone;

	/** If true the user is actively dragging the node */
	bool bUserIsDragging;

};

#undef UE_API
