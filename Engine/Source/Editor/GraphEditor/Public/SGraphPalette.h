// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMiscDefines.h"
#include "SGraphActionMenu.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class FDragDropEvent;
class SInlineEditableTextBlock;
class SWidget;
struct FEdGraphSchemaAction;
struct FGeometry;
struct FGraphActionListBuilderBase;
struct FPointerEvent;
struct FSlateBrush;
struct FSlateFontInfo;

//////////////////////////////////////////////////////////////////////////

/** Widget for displaying a single item  */
class SGraphPaletteItem : public SCompoundWidget
{
public:
	/** Delegate executed when mouse button goes down */
	FCreateWidgetMouseButtonDown MouseButtonDownDelegate;
	/** The item that we want to display with this widget */
	TWeakPtr<FEdGraphSchemaAction>	ActionPtr;
	/** Holds the inline renaming widget if one was created */
	TSharedPtr< SInlineEditableTextBlock > InlineRenameWidget;

public:
	SLATE_BEGIN_ARGS( SGraphPaletteItem ) {};
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData);

	// SWidget interface
	UE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	// End of SWidget interface

protected:
	/** Creates an icon type widget, including big tooltip link */
	UE_API TSharedRef<SWidget> CreateIconWidget( const FText& IconToolTip, const FSlateBrush* IconBrush, const FSlateColor& IconColor, const FString& DocLink, const FString& DocExcerpt, const FSlateBrush* SecondaryIconBrush, const FSlateColor& SecondaryColor );
	/** Create an icon type widget */
	UE_API TSharedRef<SWidget> CreateIconWidget(const FText& IconToolTip, const FSlateBrush* IconBrush, const FSlateColor& IconColor);

	UE_DEPRECATED(5.0, "CreateTextSlotWidget that takes a font parameter is deprecated")
	virtual TSharedRef<SWidget> CreateTextSlotWidget(const FSlateFontInfo& NameFont, FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnly)
	{
		return CreateTextSlotWidget(InCreateData, bIsReadOnly);
	}

	/* Create the text widget */
	UE_API virtual TSharedRef<SWidget> CreateTextSlotWidget( FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnly );

	/** Callback when rename text is being verified on text changed */
	UE_API virtual bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage);

	/** Callback when rename text is committed */
	UE_API virtual void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

	/** Callback to get the display text */ 
	UE_API virtual FText GetDisplayText() const;

	/** Callback to get the tooltip */
	UE_API virtual FText GetItemTooltip() const;
};

//////////////////////////////////////////////////////////////////////////

class SGraphPalette : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SGraphPalette )
		{
			_AutoExpandActionMenu = false;
		}
		SLATE_ARGUMENT( bool, AutoExpandActionMenu )
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	virtual ~SGraphPalette() = default;

protected:
	UE_API virtual TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData);

	UE_API virtual FReply OnActionDragged(const TArray< TSharedPtr<FEdGraphSchemaAction> >& InActions, const FPointerEvent& MouseEvent);

	UE_API virtual void RefreshActionsList(bool bPreserveExpansion);

	/** Callback used to populate all actions list in SGraphActionMenu */
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) = 0;

protected:
	TSharedPtr<SGraphActionMenu> GraphActionMenu;
};

#undef UE_API
