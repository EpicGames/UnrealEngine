// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "BlueprintUtilities.h"
#include "ConnectionDrawingPolicy.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "Framework/MarqueeRect.h"
#include "GraphEditAction.h"
#include "GraphEditor.h"
#include "GraphSplineOverlapResult.h"
#include "HAL/PlatformMath.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Layout/Clipping.h"
#include "Layout/Geometry.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "SGraphNode.h"
#include "SGraphPin.h"
#include "SNodePanel.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/GCObject.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class FActiveTimerHandle;
class FArrangedChildren;
class FDragDropEvent;
class FPaintArgs;
class FReferenceCollector;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class FWidgetStyle;
class IMenu;
class IToolTip;
class SGraphNode;
class SWidget;
class UEdGraph;
class UEdGraphNode;
class UObject;
struct FDiffSingleResult;
struct FEdGraphEditAction;
struct FGuid;

DECLARE_DELEGATE( FOnUpdateGraphPanel )

// Arguments when the graph panel wants to open a context menu
struct FGraphContextMenuArguments
{
	// The endpoint of the drag or the location of the right-click
	FDeprecateSlateVector2D NodeAddPosition;

	// The source node if there are any
	UEdGraphNode* GraphNode;

	// The source pin if there is one
	UEdGraphPin* GraphPin;

	// 
	TArray<UEdGraphPin*> DragFromPins;
};

// Data we need to store while the user is slicing through connections in the graph
struct FConnectionSlice
{
	// The start/end points of the slice line, in graph coords
	FMarqueeRect Line;

	// The pin connections which were intersecting the line last frame when the connection policy was drawn,
	// and which should be broken if this slice is executed
	TArray<TPair<FEdGraphPinReference, FEdGraphPinReference>> ConnectionsIntersectingSliceLine;

	explicit FConnectionSlice(const FVector2f StartPosition)
		: Line(StartPosition) { }
};


template <>
struct TWidgetTypeTraits<class SGraphPanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SGraphPanel : public SNodePanel, public FGCObject
{
	SLATE_DECLARE_WIDGET_API(SGraphPanel, SNodePanel, UE_API)
public:
	DECLARE_DELEGATE_RetVal_OneParam(FActionMenuContent, FOnGetContextMenuFor, const FGraphContextMenuArguments& /*SpawnInfo*/)

	SLATE_BEGIN_ARGS( SGraphPanel )
		: _OnGetContextMenuFor()
		, _OnSelectionChanged()
		, _OnNodeDoubleClicked()
		, _GraphObj( static_cast<UEdGraph*>(NULL) )
		, _InitialZoomToFit( false )
		, _IsEditable( true )
		, _DisplayAsReadOnly( false )
		, _ShowGraphStateOverlay(true)
		, _AllowConnectionSlicing(true)
		, _ShouldDrawBackground(true)
		, _ShouldDrawSurroundingShadow(true)
		, _OnUpdateGraphPanel()
		, _AllowZoom(true)
		, _AllowPanning(true)
		{
			_Clipping = EWidgetClipping::ClipToBounds;
		}

		SLATE_EVENT( FOnGetContextMenuFor, OnGetContextMenuFor )
		SLATE_EVENT( SGraphEditor::FOnSelectionChanged, OnSelectionChanged )
		SLATE_EVENT(FSingleNodeEvent, OnNodeDoubleClicked)
		SLATE_EVENT(SGraphEditor::FOnDropActors, OnDropActors)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SGraphEditor::FOnDropActors ConvertOnDropActorDelegate(const SGraphEditor::FOnDropActor& LegacyDelegate)
		{
			if (LegacyDelegate.IsBound())
			{
				return SGraphEditor::FOnDropActors::CreateLambda([LegacyDelegate](const TArray< TWeakObjectPtr<AActor> >& Actors, UEdGraph* InGraph, const FVector2f& InDropLocation)
				{
					LegacyDelegate.Execute(Actors, InGraph, FVector2D(InDropLocation));
				});
			}

			return SGraphEditor::FOnDropActors();
		}
		SLATE_EVENT_DEPRECATED(5.6, "Use SGraphEditor::FOnDropActors.",
			SGraphEditor::FOnDropActor,
			OnDropActor, OnDropActors, ConvertOnDropActorDelegate)
		static SGraphEditor::FOnDropStreamingLevels ConvertOnDropStreamingLevelDelegate(const SGraphEditor::FOnDropStreamingLevel& LegacyDelegate)
		{
			if (LegacyDelegate.IsBound())
			{
				return SGraphEditor::FOnDropStreamingLevels::CreateLambda([LegacyDelegate](const TArray< TWeakObjectPtr<ULevelStreaming> >& Levels, UEdGraph* InGraph, const FVector2f& InDropLocation)
				{
					LegacyDelegate.Execute(Levels, InGraph, FVector2D(InDropLocation));
				});
			}

			return SGraphEditor::FOnDropStreamingLevels();
		}
		SLATE_EVENT_DEPRECATED(5.6, "Use SGraphEditor::FOnDropStreamingLevels.",
			SGraphEditor::FOnDropStreamingLevel,
			OnDropStreamingLevel, OnDropStreamingLevels, ConvertOnDropStreamingLevelDelegate)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
		SLATE_EVENT( SGraphEditor::FOnDropStreamingLevels, OnDropStreamingLevels )
		SLATE_ARGUMENT( class UEdGraph*, GraphObj )
		SLATE_ARGUMENT( TSharedPtr<TArray<FDiffSingleResult>>, DiffResults )
		SLATE_ATTRIBUTE( int32, FocusedDiffResult )
		SLATE_ARGUMENT( bool, InitialZoomToFit )
		SLATE_ATTRIBUTE( bool, IsEditable )
		SLATE_ATTRIBUTE( bool, DisplayAsReadOnly )
		/** Show overlay elements for the graph state such as the PIE and read-only borders and text */
		SLATE_ATTRIBUTE(bool, ShowGraphStateOverlay)
		SLATE_ATTRIBUTE(bool, AllowConnectionSlicing)
		SLATE_ATTRIBUTE(bool, ShouldDrawBackground)
		SLATE_ATTRIBUTE(bool, ShouldDrawSurroundingShadow)
		SLATE_EVENT( FOnNodeVerifyTextCommit, OnVerifyTextCommit )
		SLATE_EVENT( FOnNodeTextCommitted, OnTextCommitted )
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static SGraphEditor::FOnSpawnNodeByShortcutAtLocation ConvertOnSpawnNodeByShortcutDelegate(const SGraphEditor::FOnSpawnNodeByShortcut& LegacyDelegate)
		{
			if (LegacyDelegate.IsBound())
			{
				return SGraphEditor::FOnSpawnNodeByShortcutAtLocation::CreateLambda([LegacyDelegate](FInputChord InInputChord, const FVector2f& InLocation)
				{
					return LegacyDelegate.Execute(InInputChord, FVector2D(InLocation));
				});
			}

			return SGraphEditor::FOnSpawnNodeByShortcutAtLocation();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		SLATE_EVENT_DEPRECATED(5.6, "Use SGraphEditor::FOnSpawnNodeByShortcutAtLocation.",
			PRAGMA_DISABLE_DEPRECATION_WARNINGS SGraphEditor::FOnSpawnNodeByShortcut PRAGMA_ENABLE_DEPRECATION_WARNINGS,
			OnSpawnNodeByShortcut, OnSpawnNodeByShortcutAtLocation, ConvertOnSpawnNodeByShortcutDelegate)
		SLATE_EVENT( SGraphEditor::FOnSpawnNodeByShortcutAtLocation, OnSpawnNodeByShortcutAtLocation )
		SLATE_EVENT( FOnUpdateGraphPanel, OnUpdateGraphPanel )
		SLATE_EVENT( SGraphEditor::FOnDisallowedPinConnection, OnDisallowedPinConnection )
		SLATE_EVENT( SGraphEditor::FOnDoubleClicked, OnDoubleClicked )
		SLATE_EVENT( SGraphEditor::FOnMouseButtonDown, OnMouseButtonDown )
		SLATE_EVENT( SGraphEditor::FOnNodeSingleClicked, OnNodeSingleClicked )
		//SLATE_ATTRIBUTE( FGraphAppearanceInfo, Appearance )
		SLATE_ARGUMENT(bool, AllowZoom)
		SLATE_ARGUMENT(bool, AllowPanning)
	SLATE_END_ARGS()

	UE_API SGraphPanel();

	/**
	 * Construct a widget
	 *
	 * @param InArgs    The declaration describing how the widgets should be constructed.
	 */
	UE_API void Construct( const FArguments& InArgs );

	// Destructor
	UE_API ~SGraphPanel();
public:
	// SWidget interface
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	UE_API virtual bool CustomPrepass(float LayoutScaleMultiplier) override;
	// End of SWidget interface

	// SNodePanel interface
	UE_API virtual TSharedPtr<SWidget> OnSummonContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual bool OnHandleLeftMouseRelease(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void AddGraphNode(const TSharedRef<SNode>& NodeToAdd) override;
	UE_API virtual void RemoveAllNodes() override;
	// End of SNodePanel interface

	// FGCObject interface.
	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	UE_API virtual FString GetReferencerName() const override;
	// End of FGCObject interface.

	UE_API void ArrangeChildrenForContextMenuSummon(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const;
	UE_API TSharedPtr<SWidget> SummonContextMenu(const UE::Slate::FDeprecateVector2DParameter& WhereToSummon, const UE::Slate::FDeprecateVector2DParameter& WhereToAddNode, UEdGraphNode* ForNode, UEdGraphPin* ForPin, const TArray<UEdGraphPin*>& DragFromPins);
	UE_API void SummonCreateNodeMenuFromUICommand(uint32 NumNodesAdded);
	UE_API void DismissContextMenu();

	UE_API void OnBeginMakingConnection(UEdGraphPin* InOriginatingPin);
	UE_API void OnBeginMakingConnection(FGraphPinHandle PinHandle);
	UE_API void OnStopMakingConnection(bool bForceStop = false);
	UE_API void PreservePinPreviewUntilForced();

	/** Indicate that the connection from the given start to the given end pins is being relinked. A preview connection is being drawn for the relinked connection. */
	UE_API void OnBeginRelinkConnection(const FGraphPinHandle& InSourcePinHandle, const FGraphPinHandle& InTargetPinHandle);

	/** The relink connection operation either got cancelled or has successfully been executed. Preview connection won't be drawn anymore. */
	UE_API void OnEndRelinkConnection(bool bForceStop = false);

	/** True in case a connection is currently being relinked, false if not. */
	UE_API bool IsRelinkingConnection() const;

	/** Update this GraphPanel to match the data that it is observing. Expected to be called during ticking. */
	UE_API void Update();

	/** Purges the existing visual representation (typically followed by an Update call in the next tick) */
	UE_API void PurgeVisualRepresentation();

	/** Use to determine if a comment title is currently visible */
	UE_API bool IsNodeTitleVisible(const class UEdGraphNode* Node, bool bRequestRename);

	/** Use to determine if a rectangle is currently visible */
	UE_API bool IsRectVisible(const UE::Slate::FDeprecateVector2DParameter &TopLeft, const UE::Slate::FDeprecateVector2DParameter &BottomRight);

	/** Focuses the view on rectangle, zooming if neccesary */
	UE_API bool JumpToRect(const UE::Slate::FDeprecateVector2DParameter &BottomLeft, const UE::Slate::FDeprecateVector2DParameter &TopRight);

	UE_API void JumpToNode(const class UEdGraphNode* JumpToMe, bool bRequestRename, bool bSelectNode);

	UE_API void JumpToPin(const class UEdGraphPin* JumptToMe);

	UE_API void GetAllPins(TSet< TSharedRef<SWidget> >& AllPins);

	UE_API void AddPinToHoverSet(UEdGraphPin* HoveredPin);
	UE_API void RemovePinFromHoverSet(UEdGraphPin* UnhoveredPin);

	SGraphEditor::EPinVisibility GetPinVisibility() const { return PinVisibility; }
	void SetPinVisibility(SGraphEditor::EPinVisibility InVisibility) { PinVisibility = InVisibility; }

	UEdGraph* GetGraphObj() const { return GraphObj; }

	/** helper to attach graph events to sub node, which won't be placed directly on the graph */
	UE_API void AttachGraphEvents(TSharedPtr<SGraphNode> CreatedSubNode);

	/** Returns if this graph is editable */
	bool IsGraphEditable() const { return GetIsEditable(); }

	/** Attempt to retrieve the bounds for the specified node */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API bool GetBoundsForNode(const UObject* InNode, FVector2D& MinCorner, FVector2D& MaxCorner, float Padding = 0.0f) const;
	UE_API bool GetBoundsForNode(const UObject* InNode, FVector2f& MinCorner, FVector2f& MaxCorner, float Padding = 0.0f) const;

	/** Straighten all connections between the selected nodes */
	UE_API void StraightenConnections();
	
	/** Straighten any connections attached to the specified pin, optionally limiting to the specified pin to align */
	UE_API void StraightenConnections(UEdGraphPin* SourcePin, UEdGraphPin* PinToAlign = nullptr);

	/** Refresh the visual state of a single node */
	UE_API void RefreshNode(UEdGraphNode& Node);

	/** When the graph panel needs to be dynamically refreshing for animations, this function is registered to tick and invalidate the UI. */
	UE_API EActiveTimerReturnType InvalidatePerTick(double InCurrentTime, float InDeltaTime);

	/** Sets the current widget factory. */
	UE_API void SetNodeFactory(const TSharedRef<class FGraphNodeFactory>& NewNodeFactory);

	static bool IsSnappingEnabled();
protected:

	UE_API void NotifyGraphChanged ( const struct FEdGraphEditAction& InAction);

	UE_API const TSharedRef<SGraphNode> GetChild(int32 ChildIndex);

	/** Flag to control AddNode, more readable than a bool:*/
	enum AddNodeBehavior
	{
		CheckUserAddedNodesList,
		WasUserAdded,
		NotUserAdded
	};

	/** Helper method to add a new node to the panel */
	UE_API void AddNode(class UEdGraphNode* Node, AddNodeBehavior Behavior);

	/** Helper method to remove a node from the panel */
	UE_API void RemoveNode(const UEdGraphNode* Node);

	/** Helper method to remove all nodes from the panel holding garbage / invalid pointers */
	UE_API void RemoveAllNodesWithInvalidPointers();

	/** Called when the graph selection changed */
	UE_API virtual void OnSelectionChanged(const TSet<UObject*>& Objects) override;

public:
	/** Pin marked via shift-clicking */
	TWeakPtr<SGraphPin> MarkedPin;

	/** Get a graph node widget from the specified GUID, if it applies to any nodes in this graph */
	UE_API TSharedPtr<SGraphNode> GetNodeWidgetFromGuid(FGuid Guid) const;

	/** Get a list of selected editor graph nodes from the selection manager. */
	UE_API TArray<UEdGraphNode*> GetSelectedGraphNodes() const;

	const FGraphSplineOverlapResult& GetPreviousFrameSplineOverlap() const { return PreviousFrameSplineOverlap; }

protected:
	void SetShowGraphStateOverlay(TAttribute<bool> InShowGraphStateOverlayAttribute) { ShowGraphStateOverlayAttribute.Assign(*this, MoveTemp(InShowGraphStateOverlayAttribute)); }
	bool GetShowGraphStateOverlay() const { return ShowGraphStateOverlayAttribute.Get(); }
	TSlateAttributeRef<bool> GetShowGraphStateOverlayAttribute() const { return TSlateAttributeRef<bool>{SharedThis(this), ShowGraphStateOverlayAttribute}; }

	void SetShouldDrawBackground(TAttribute<bool> InShouldDrawBackgroundAttribute) { ShouldDrawBackgroundAttribute.Assign(*this, MoveTemp(InShouldDrawBackgroundAttribute)); }
	bool GetShouldDrawBackground() const { return ShouldDrawBackgroundAttribute.Get(); }
	TSlateAttributeRef<bool> GetShouldDrawBackgroundAttribute() const { return TSlateAttributeRef<bool>{SharedThis(this), ShouldDrawBackgroundAttribute}; }

	void SetShouldDrawSurroundingShadow(TAttribute<bool> InShouldDrawSurroundingShadowAttribute) { ShouldDrawSurroundingShadowAttribute.Assign(*this, MoveTemp(InShouldDrawSurroundingShadowAttribute)); }
	bool GetShouldDrawSurroundingShadow() const { return ShouldDrawSurroundingShadowAttribute.Get(); }
	TSlateAttributeRef<bool> GetShouldDrawSurroundingShadowAttribute() const { return TSlateAttributeRef<bool>{SharedThis(this), ShouldDrawSurroundingShadowAttribute}; }

	void SetDisplayAsReadOnly(TAttribute<bool> InDisplayAsReadOnlyAttribute) { DisplayAsReadOnlyAttribute.Assign(*this, MoveTemp(InDisplayAsReadOnlyAttribute)); }
	bool GetDisplayAsReadOnly() const { return DisplayAsReadOnlyAttribute.Get(); }
	TSlateAttributeRef<bool> GetDisplayAsReadOnlyAttribute() const { return TSlateAttributeRef<bool>{SharedThis(this), DisplayAsReadOnlyAttribute}; }

private:

	/** A map of guid -> graph nodes */
	TMap<FGuid, TWeakPtr<SGraphNode>> NodeGuidMap;

	/** List of currently relinked connections. */
	TArray<FConnectionDrawingPolicy::FRelinkConnection> RelinkConnections;

protected:
	TObjectPtr<UEdGraph> GraphObj;
	
	// if this graph is displaying the results of a diff, this will provide info
	// on how to display the nodes
	TSharedPtr<TArray<FDiffSingleResult>> DiffResults;
	TAttribute<int32> FocusedDiffResult;

	// Should we ignore the OnStopMakingConnection unless forced?
	bool bPreservePinPreviewConnection;

	/** Pin visibility mode */
	SGraphEditor::EPinVisibility PinVisibility;

	/** List of pins currently being hovered over */
	TSet< FEdGraphPinReference > CurrentHoveredPins;

	/** Time since the last mouse enter/exit on a pin */
	double TimeWhenMouseEnteredPin;
	double TimeWhenMouseLeftPin;
	
	bool bInSlicingMode = false;

	/** Sometimes the panel draws a preview connector; e.g. when the user is connecting pins */
	TArray< FGraphPinHandle > PreviewConnectorFromPins;
	FDeprecateSlateVector2D PreviewConnectorEndpoint;
	mutable bool bIsDrawStateCached = false;

	/** Last mouse position seen, used for paint-centric highlighting */
	FDeprecateSlateVector2D SavedMousePosForOnPaintEventLocalSpace;
	
	/** The overlap results from the previous OnPaint call */
	FDeprecateSlateVector2D PreviousFrameSavedMousePosForSplineOverlap;
	FGraphSplineOverlapResult PreviousFrameSplineOverlap;
	FGraphSplineOverlapResult LastClickSplineOverlap;

	/** The mouse state from the last mouse move event, used to synthesize pin actions when hovering over a spline on the panel */
	FGeometry LastPointerGeometry;
	FPointerEvent LastPointerEvent;

	/** Invoked when we need to summon a context menu */
	FOnGetContextMenuFor OnGetContextMenuFor;

	/** Invoked when an actor is dropped onto the panel */
	SGraphEditor::FOnDropActors OnDropActors;

	/** Invoked when a streaming level is dropped onto the panel */
	SGraphEditor::FOnDropStreamingLevels OnDropStreamingLevels;

	/** What to do when a node is double-clicked */
	FSingleNodeEvent OnNodeDoubleClicked;

	/** Bouncing curve */
	FCurveSequence BounceCurve;

	/** Geometry cache */
	mutable FDeprecateSlateVector2D CachedAllottedGeometryScaledSize;

	/** Invoked when text is being committed on panel to verify it */
	FOnNodeVerifyTextCommit OnVerifyTextCommit;

	/** Invoked when text is committed on panel */
	FOnNodeTextCommitted OnTextCommitted;

	/** Invoked when the panel is updated */
	FOnUpdateGraphPanel OnUpdateGraphPanel;

	/** Called when the user generates a warning tooltip because a connection was invalid */
	SGraphEditor::FOnDisallowedPinConnection OnDisallowedPinConnection;

	/** Called when the graph itself is double clicked */
	SGraphEditor::FOnDoubleClicked OnDoubleClicked;

	/** Called when the graph itself is clicked */
	SGraphEditor::FOnMouseButtonDown OnClicked;

	/** Called when the graph selection changed */
	SGraphEditor::FOnSelectionChanged OnSelectionChangedDelegate;

	/** Whether to draw the overlay indicating we're in PIE */
	bool bShowPIENotification;

	/** Whether to draw decorations for graph state (PIE / ReadOnly etc.) */
	UE_DEPRECATED(5.8, "Use SetShowGraphStateOverlay / GetShowGraphStateOverlay / GetShowGraphStateOverlayAttribute")
	TSlateDeprecatedTAttribute<bool> ShowGraphStateOverlay;

	/** Whether to allow slicing of connections in this graph panel */
	TAttribute<bool> AllowConnectionSlicing;

	/** Whether this panel should be drawing a background */
	UE_DEPRECATED(5.8, "Use SetShouldDrawBackground / GetShouldDrawBackground / GetShouldDrawBackgroundAttribute")
	TSlateDeprecatedTAttribute<bool> ShouldDrawBackground;

	/** Whether this panel should be drawing a sunken shadow on its borders */
	UE_DEPRECATED(5.8, "Use SetShouldDrawSurroundingShadow / GetShouldDrawSurroundingShadow / GetShouldDrawSurroundingShadowAttribute")
	TSlateDeprecatedTAttribute<bool> ShouldDrawSurroundingShadow;

	/**
	 * The last created policy which is cached to be used on MouseMove to invalidate when hovering connections.
	 * This must not be used for drawing outside of Paint
	 */
	mutable TUniquePtr<FConnectionDrawingPolicy> ConnectionDrawingPolicy;

private:
	/** Set of nodes selected by the user, tracked while a visual update is pending */
	TSet<TWeakObjectPtr<class UEdGraphNode>> UserSelectedNodes;

	/** Set of user-added nodes for the panel, tracked while a visual update is pending */
	TSet<const class UEdGraphNode*> UserAddedNodes;

	/** Whether to draw decorations for graph state (PIE / ReadOnly etc.) */
	TSlateAttribute<bool> ShowGraphStateOverlayAttribute;

	/** Whether this panel should be drawing a background */
	TSlateAttribute<bool> ShouldDrawBackgroundAttribute;

	/** Whether this panel should be drawing a sunken shadow on its borders */
	TSlateAttribute<bool> ShouldDrawSurroundingShadowAttribute;

	/** Should the graph display all nodes in a read-only state (grayed)? This does not affect functionality of using them (IsEditable) */
	TSlateAttribute<bool> DisplayAsReadOnlyAttribute;

	FOnGraphChanged::FDelegate MyRegisteredGraphChangedDelegate;
	FDelegateHandle            MyRegisteredGraphChangedDelegateHandle;
private:
	/** Called when PIE begins */
	UE_API void OnBeginPIE( const bool bIsSimulating );

	/** Called when PIE ends */
	UE_API void OnEndPIE( const bool bIsSimulating );

	/** Called when watched graph changes */
	UE_API void OnGraphChanged( const FEdGraphEditAction& InAction );

	/** Update all selected nodes position by provided vector2d */
	UE_API void UpdateSelectedNodesPositions(const FVector2f& PositionIncrement);

	/** Handle updating the spline hover state */
	UE_API bool OnSplineHoverStateChanged(const FGraphSplineOverlapResult& NewSplineHoverState);

	/** Returns the pin that we're considering as hovered if we are hovering over a spline; may be null */
	UE_API class SGraphPin* GetBestPinFromHoveredSpline() const;

	/** Returns the top most graph node under the mouse pointer. Returns nullptr if no node found */
	UE_API TSharedPtr<SGraphNode> GetGraphNodeUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Returns a pin that is under the mouse, given a specified node. Returns nullptr if no valid node is given or no pin found */
	UE_API UEdGraphPin* GetPinUnderMouse(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TSharedPtr<SGraphNode> GraphNode) const;

	/** If the spawned nodes were auto-wired from any of the dragged pins, then this will try to make the newly connected pin end up at SpawnGraphPosition */
	UE_API void AdjustNewlySpawnedNodePositions(TArrayView<UEdGraphNode* const> SpawnedNodes, TArrayView<UEdGraphPin*> DraggedFromPins, FVector2f SpawnGraphPosition);

	/** Will move a group of nodes by the amount needed for an anchor pin to be at a certain position */
	UE_API void MoveNodesToAnchorPinAtGraphPosition(TArrayView<UEdGraphNode* const> NodesToMove, FGraphPinHandle PinToAnchor, FVector2f DesiredPinGraphPosition);

	/** Begins a drag operation that disconnects the pin at ClosestPin, but only from the ConnectedPin, rather than all connections */
	UE_API FReply OnMouseDownDisconnectClosestPinFromWire(TSharedRef<class SGraphPin> ClosestPin, TSharedRef<class SGraphPin> ConnectedPin);

	/** Node snapping is a private level extension, give it access to private graph editor state: */
	friend struct FNodeSnappingManager;

	/** Handle to timer callback that allows the UI to refresh it's arrangement each tick, allows animations to occur within the UI */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandleInvalidatePerTick;

	/** Amount of time left to invalidate the UI per tick */
	float TimeLeftToInvalidatePerTick;

	/** The current node factory to create nodes, pins and connections. Uses the static FNodeFactory if not set. */
	TSharedPtr<class FGraphNodeFactory> NodeFactory;

	/** Weak pointer to the last summoned context menu, for dismissing it when requested. */
	TWeakPtr<IMenu> ContextMenu;

	/** A flag indicating that we need to check if we have any nodes which use invalid pointers */
	bool bCheckNodeGraphObjValidity;

	/** Exists while the user is making a slice, and tracks the line points and any connections intersecting the line */
	TOptional<FConnectionSlice> CurrentSlice;
};

#undef UE_API
