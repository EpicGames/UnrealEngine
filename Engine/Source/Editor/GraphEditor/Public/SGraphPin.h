// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "SGraphNode.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class FDragDropEvent;
class FDragDropOperation;
class FPinValueInspectorTooltip;
class IToolTip;
class SGraphNode;
class SGraphPanel;
class SGraphPin;
class SHorizontalBox;
class SImage;
class SLevelOfDetailBranchNode;
class SWidget;
class SWrapBox;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

#define NAME_DefaultPinLabelStyle TEXT("Graph.Node.PinName")

/////////////////////////////////////////////////////
// FGraphPinHandle

/** A handle to a pin, defined by its owning node's GUID, and the pin's name. Used to reference a pin without referring to its widget */
struct FGraphPinHandle
{
	/** The GUID of the node to which this pin belongs */
	FGuid NodeGuid;

	/** The GUID of the pin we are referencing */
	FGuid PinId;

	/**
	 * Default constructor
	 * Will contain a invalid node and pin GUID and IsValid() will return false.
	 */
	FGraphPinHandle() = default;

	/** Constructor */
	UE_API FGraphPinHandle(UEdGraphPin* InPin);

	/** */
	UE_API UEdGraphPin* GetPinObj(const SGraphPanel& InPanel) const;

	/** Find a pin widget in the specified panel from this handle */
	UE_API TSharedPtr<class SGraphPin> FindInGraphPanel(const class SGraphPanel& InPanel) const;

	/** */
	bool IsValid() const
	{
		return PinId.IsValid() && NodeGuid.IsValid();
	}

	/** */
	bool operator==(const FGraphPinHandle& Other) const
	{
		return PinId == Other.PinId && NodeGuid == Other.NodeGuid;
	}

	/** */
	friend inline uint32 GetTypeHash(const FGraphPinHandle& Handle)
	{
		return HashCombine(GetTypeHash(Handle.PinId), GetTypeHash(Handle.NodeGuid));
	}
};

/////////////////////////////////////////////////////
// SGraphPin


template <>
struct TWidgetTypeTraits<class SGraphPin>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Represents a pin on a node in the GraphEditor
 */
class SGraphPin : public SBorder
{
public:

	SLATE_BEGIN_ARGS(SGraphPin)
		: _PinLabelStyle(NAME_DefaultPinLabelStyle)
		, _UsePinColorForText(false)
		, _SideToSideMargin(5.0f)
		{}
		SLATE_ARGUMENT(FName, PinLabelStyle)
		SLATE_ARGUMENT(bool, UsePinColorForText)
		SLATE_ARGUMENT(float, SideToSideMargin)
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InPin	The pin to create a widget for
	 */
	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

public:
	UE_API SGraphPin();
	UE_API virtual ~SGraphPin();

	/** Set attribute for determining if pin is editable */
	UE_API void SetIsEditable(TAttribute<bool> InIsEditable);

	/** Retrieves the full horizontal box that contains the pin's row content */
	TWeakPtr<SHorizontalBox> GetFullPinHorizontalRowWidget()
	{
		return FullPinHorizontalRowWidget;
	}

	/** Handle clicking on the pin */
	UE_API virtual FReply OnPinMouseDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent);

	/** Handle clicking on the pin name */
	UE_API FReply OnPinNameMouseDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent);

	// SWidget interface
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	UE_API virtual TSharedPtr<IToolTip> GetToolTip() override;
	// End of SWidget interface

	/** Refresh the level of detail applied to the pin name */
	UE_API void RefreshLOD();

public:
	UE_API UEdGraphPin* GetPinObj() const;

	/** @param OwnerNode  The SGraphNode that this pin belongs to */
	UE_API void SetOwner( const TSharedRef<SGraphNode> OwnerNode );

	/** @param PinObj The UEdGraphPin object this pin widget represents */
	UE_API void SetPinObj(UEdGraphPin* PinObj);

	/** @return whether this pin is incoming or outgoing */
	UE_API EEdGraphPinDirection GetDirection() const;

	/** @return whether this pin is an array value */
	UE_API bool IsArray() const;

	/** @return whether this pin is a set value */
	UE_API bool IsSet() const;

	/** @return whether this pin is a map value */
	UE_API bool IsMap() const;

	/** @return whether this pin is passed by mutable ref */
	UE_API bool IsByMutableRef() const;

	/** @return whether this pin is passed by mutable ref */
	UE_API bool IsDelegate() const;

	/** @return whether this pin is connected to another pin */
	UE_API bool IsConnected() const;

	/** @return whether to fade out the pin's connections */
	UE_API bool AreConnectionsFaded() const;

	/** Tries to handle making a connection to another pin, depending on the schema and the pins it may do:  
		 - Nothing
		 - Break existing links on either side while making the new one
		 - Just make the new one
		 @return Whether a connection was actually made or not
     */
	UE_API virtual bool TryHandlePinConnection(SGraphPin& OtherSPin);

	/** @return Visibility based on whether or not we should show the inline editable text field for this pin */
	UE_API virtual EVisibility GetDefaultValueVisibility() const;

	/** Control whether we draw the label on this pin */
	UE_API void SetShowLabel(bool bNewDrawLabel);

	/** Allows the connection drawing policy to control the pin color */
	void SetPinColorModifier(FLinearColor InColor)
	{
		PinColorModifier = InColor;
	}

	void SetDiffHighlighted(bool bHighlighted)
	{
		bIsDiffHighlighted = bHighlighted;
	}

	/** Allows Diff to highlight pins */
	void SetPinDiffColor(TOptional<FLinearColor> InColor)
	{
		PinDiffColor = InColor;
	}

	/** Makes Pin Connection Wires transparent */
	void SetFadeConnections(bool bInFadeConnections)
	{
		bFadeConnections = bInFadeConnections;
	}

	/** Set this pin to only be used to display default value */
	UE_API void SetOnlyShowDefaultValue(bool bNewOnlyShowDefaultValue);

	/** If pin in node is visible at all */
	UE_API EVisibility IsPinVisibleAsAdvanced() const;

	/** Gets Node Offset */
	UE_API UE::Slate::FDeprecateVector2DResult GetNodeOffset() const;

	/** Returns whether or not this pin is currently connectable */
	UE_API bool GetIsConnectable() const;

	/** Build the widget we should put into the 'default value' space, shown when nothing connected */
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget();

	/** Get the widget created by GetDefaultValueWidget() */
	TSharedRef<SWidget>	GetValueWidget() const { return ValueWidget.ToSharedRef(); }

	/** Get the label/value part of the pin widget */
	TSharedRef<SWrapBox> GetLabelAndValue() const { return LabelAndValue.ToSharedRef(); }

	/** True if pin can be edited */
	UE_API bool IsEditingEnabled() const;

	/** True if the pin's default value can be edited, false if it is read only */
	bool GetDefaultValueIsEditable() const { return !GraphPinObj->bDefaultValueIsReadOnly; }
	
	/** 
	  * Called when ed graph data is cleared, indicating this widget can no longer safely access GraphPinObj 
	  * Typically an SGraphPin is destroyed before the ed graph data is actually deleted but because SWidget
	  * is reference counted we have had problems in the past with the SGraphPin (or Node) living unexpectedly 
	  * long. If such a situation occurs the code attempting to own the SWidget should be demoted to a WeakPtr,
	  * but for added safety we track graph data invalidation here:
	  */
	void InvalidateGraphData() { bGraphDataInvalid = true; }

	/** Override the visual look of the pin by providing two custom brushes */
	UE_API void SetCustomPinIcon(const FSlateBrush* InConnectedBrush, const FSlateBrush* InDisconnectedBrush);

	/** @returns true if we have a valid PinValueInspector tooltip */
	UE_API bool HasInteractiveTooltip() const;

	/** Enables or disables drag and drop on the pin */
	void EnableDragAndDrop(bool bEnable) { bDragAndDropEnabled = bEnable; }

	bool SnapSGraphNode(TSharedRef<SGraphNode> Node);
	void ClearSnappedSGraphNode();
	bool IsSnapped() const;
	TSharedPtr<SGraphNode> GetSnappedChildForPosition(const FGeometry& MyGeometry, const FVector2f& Position, FVector2f& OutPositionInNode) const;
	TOptional<FArrangedWidget> GetArrangedSnappedChildNode(const FGeometry& MyGeometry) const;
	TSharedPtr<SGraphNode> GetSnappedNode() const;
protected:

	/** If true the graph pin subclass is responsible for setting the IsEnabled delegates for the aspects it cares about. If false, the default value widget enabling is done by the base class */
	virtual bool DoesWidgetHandleSettingEditingEnabled() const { return false; }

	UE_API FText GetPinLabel() const;

	/** If we should show the label on this pin */
	EVisibility GetPinLabelVisibility() const
	{
		return bShowLabel ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/** Get the widget we should put in the label space, which displays the name of the pin.*/
	UE_API virtual TSharedRef<SWidget> GetLabelWidget(const FName& InPinLabelStyle);

	/** @return The brush with which to paint this graph pin's incoming/outgoing bullet point */
	UE_API virtual const FSlateBrush* GetPinIcon() const;

	/** @return the brush which is to paint the 'secondary icon' for this pin, (e.g. value type for Map pins */
	UE_API const FSlateBrush* GetSecondaryPinIcon() const;

	UE_API const FSlateBrush* GetPinBorder() const;
	
	/** @return The status brush to show after/before the name (on an input/output) */
	UE_API virtual const FSlateBrush* GetPinStatusIcon() const;

	/** @return Should we show a status brush after/before the name (on an input/output) */
	UE_API virtual EVisibility GetPinStatusIconVisibility() const;

	/** Toggle the watch pinning */
	UE_API virtual FReply ClickedOnPinStatusIcon();

	/** @return The color that we should use to draw this pin */
	UE_API virtual FSlateColor GetPinColor() const;

	/** @return The color that we should use to draw the highlight for this pin */
	UE_API virtual FSlateColor GetHighlightColor() const;

	UE_API virtual FSlateColor GetPinDiffColor() const;

	/** @return The secondary color that we should use to draw this pin (e.g. value color for Map pins) */
	UE_API FSlateColor GetSecondaryPinColor() const;

	/** @return The color that we should use to draw this pin's text */
	UE_API virtual FSlateColor GetPinTextColor() const;

	/** @return The tooltip to display for this pin */
	UE_API FText GetTooltipText() const;

	/** Gets the window location (in screen coords) for an interactive tooltip (e.g. pin value inspector) */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API void GetInteractiveTooltipLocation(FVector2D& InOutDesiredLocation) const;
	UE_API void GetInteractiveTooltipLocation(FVector2f& InOutDesiredLocation) const;

	UE_API TOptional<EMouseCursor::Type> GetPinCursor() const;

	/** Spawns a FDragConnection or similar class for the pin drag event */
	UE_API virtual TSharedRef<FDragDropOperation> SpawnPinDragEvent(const TSharedRef<class SGraphPanel>& InGraphPanel, const TArray< TSharedRef<SGraphPin> >& InStartingPins);

	// Should we use low-detail pin names?
	UE_API virtual bool UseLowDetailPinNames() const;

	// What widget should we show when we change detail slot?
	UE_API virtual TSharedRef<SWidget> OnGetActiveDetailSlotContent(bool bChangedToHighDetail) const;

	/** Determines the pin's visibility based on the LOD factor, when it is low LOD, no hit test will occur */
	UE_API EVisibility GetPinVisiblity() const;

public:

	/** Returns the current pin image widget that is being used */
	UE_API TSharedPtr<SWidget> GetPinImageWidget() const;

	/** Sets the pin widget for this pin */
	UE_API void SetPinImageWidget(TSharedRef<SWidget> NewWidget);

protected:
	friend struct FNodeSnappingManager; // want access to owner, can reconsider design if snapping is successful
	/** The GraphNode that owns this pin */
	TWeakPtr<SGraphNode> OwnerNodePtr;

	/** Image of pin */
	TSharedPtr<SWidget> PinImage;
	TSharedPtr<SBox> SnappedDataNodeContainer;
	TWeakPtr<SGraphNode> SnappedNode;

	/** Label and value */
	TSharedPtr<SWrapBox> LabelAndValue;

	/** Horizontal box that holds the full detail pin widget, useful for outsiders to inject widgets into the pin */
	TWeakPtr<SHorizontalBox> FullPinHorizontalRowWidget;

	/** The slot in which PinImage will be attached when PinNameLODBranchNode switches to HighDetail. Might be stale or null. */
	SHorizontalBox::FSlot* PinImageHighDetailSlot;

	/** Value widget for the pin, created with GetDefaultValueWidget() */
	TSharedPtr<SWidget> ValueWidget;

	/** Value inspector tooltip while debugging */
	TWeakPtr<FPinValueInspectorTooltip> ValueInspectorTooltip;

	/** The GraphPin that this widget represents. */
	class UEdGraphPin* GraphPinObj;

	/** Is this pin editable */
	TAttribute<bool> IsEditable;

	/** Color modifier for use by the connection drawing policy */
	FLinearColor PinColorModifier;

	/** Cached offset from owning node to approximate position of culled pins */
	FDeprecateSlateVector2D CachedNodeOffset;

	/** Set of pins that are currently being hovered */
	TSet< FEdGraphPinReference > HoverPinSet;

	/** If set, this will change the color of the pin's background highlight */
	TOptional<FLinearColor> PinDiffColor;

	//@TODO: Want to cache these once for all SGraphPins, but still handle slate style updates
	const FSlateBrush* CachedImg_ArrayPin_Connected;
	const FSlateBrush* CachedImg_ArrayPin_Disconnected;
	const FSlateBrush* CachedImg_RefPin_Connected;
	const FSlateBrush* CachedImg_RefPin_Disconnected;
	const FSlateBrush* CachedImg_Pin_Connected;
	const FSlateBrush* CachedImg_Pin_Disconnected;
	const FSlateBrush* CachedImg_DelegatePin_Connected;
	const FSlateBrush* CachedImg_DelegatePin_Disconnected;
	const FSlateBrush* CachedImg_PosePin_Connected;
	const FSlateBrush* CachedImg_PosePin_Disconnected;
	const FSlateBrush* CachedImg_SetPin;
	const FSlateBrush* CachedImg_MapPinKey;
	const FSlateBrush* CachedImg_MapPinValue;

	const FSlateBrush* CachedImg_Pin_Background;
	const FSlateBrush* CachedImg_Pin_BackgroundHovered;
	
	const FSlateBrush* CachedImg_Pin_DiffOutline;

	const FSlateBrush* Custom_Brush_Connected;
	const FSlateBrush* Custom_Brush_Disconnected;

	/** flag indicating that graph data has been deleted by the user */
	bool bGraphDataInvalid;

	/** If we should draw the label on this pin */
	bool bShowLabel;

	/** If we should draw the label on this pin */
	bool bOnlyShowDefaultValue;

	/** true when we're moving links between pins. */
	bool bIsMovingLinks;

	/** TRUE if the pin should use the Pin's color for the text */
	bool bUsePinColorForText;

	/** TRUE if the pin should allow any drag and drop */
	bool bDragAndDropEnabled;

	/** True if this pin is being diffed and it's currently selected in the diff view. Highlights this pin */
	bool bIsDiffHighlighted;

	/** TRUE if the connections from this pin should be drawn at a lower opacity */
	bool bFadeConnections;

private:
	/** Cached value of snapping feature flag */
	bool bIsSnappingEnabled;

	TSharedPtr<SLevelOfDetailBranchNode> PinNameLODBranchNode;
	friend class SGraphPanel;
};

#undef UE_API
