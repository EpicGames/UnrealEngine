// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

#define UE_API GRAPHEDITOR_API

class SComboButton;
template <typename ItemType> class SListView;

/** A delegate type invoked when a selection changes somewhere. */
DECLARE_DELEGATE_RetVal_OneParam(FText, FGetComboItemDisplayString, int32);

template <>
struct TWidgetTypeTraits<class SPinComboBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

//Class implementation to create combo box
class SPinComboBox : public SCompoundWidget
{
public:
	typedef SListView< TSharedPtr<int32> > SComboList;
	typedef TSlateDelegates< TSharedPtr<int32> >::FOnSelectionChanged FOnSelectionChanged;

	SLATE_BEGIN_ARGS( SPinComboBox ){}		
		SLATE_ATTRIBUTE( TArray< TSharedPtr<int32> >, ComboItemList )
		SLATE_ATTRIBUTE( FString, VisibleText )
		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )
		SLATE_EVENT( FGetComboItemDisplayString, OnGetDisplayName )
		SLATE_EVENT( FGetComboItemDisplayString, OnGetTooltip )
	SLATE_END_ARGS()

	//Construct combo box using combo button and combo list
	UE_API void Construct( const FArguments& InArgs );

	//Function to return currently selected string
	const TSharedPtr<int32> GetSelectedItem()
	{
		return CurrentSelection.Pin();
	}

	UE_API void RemoveItemByIndex(int32 InIndexToRemove);
	
private:

	//Function to set the newly selected item
	UE_API void OnSelectionChangedInternal( TSharedPtr<int32> NewSelection, ESelectInfo::Type SelectInfo );

	//Get string to display
	FText OnGetVisibleTextInternal() const
	{
		return FText::FromString(VisibleText.Get());
	}

	// Function to create each row of the combo widget
	UE_API TSharedRef<ITableRow> OnGenerateComboWidget( TSharedPtr<int32> InComboIndex, const TSharedRef<STableViewBase>& OwnerTable );
	
	/** Returns the target width of the enum selection menu. */
	UE_API FOptionalSize GetMenuDesiredWidth() const;
	
	FText GetRowString(int32 RowIndex) const
	{
		return OnGetDisplayName.Execute(RowIndex);
	}

	/** Callback for Slate to retrieve the tooltip */
	FText GetRowTooltip(int32 RowIndex) const
	{
		return OnGetTooltip.Execute(RowIndex);
	}

private:
	/** List of items in our combo box. Only generated once as combo items dont change at runtime */
	TArray< TSharedPtr<int32> > ComboItemList;
	
	TAttribute< FString > VisibleText;
	TSharedPtr< SComboButton > ComboButton;
	TSharedPtr< SComboList > ComboList;
	FOnSelectionChanged OnSelectionChanged;
	TWeakPtr<int32> CurrentSelection;

	FGetComboItemDisplayString OnGetDisplayName;

	/** Delegate for finding the tooltip */
	FGetComboItemDisplayString OnGetTooltip;
};

#undef UE_API
