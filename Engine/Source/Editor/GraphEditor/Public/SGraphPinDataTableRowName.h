// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataTableEditorUtils.h"
#include "Engine/DataTable.h"
#include "SGraphPinNameList.h"
#include "UObject/SoftObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API GRAPHEDITOR_API

class UDataTable;
class UEdGraphPin;

class SGraphPinDataTableRowName : public SGraphPinNameList, public FDataTableEditorUtils::INotifyOnDataTableChanged
{
public:
	SLATE_BEGIN_ARGS(SGraphPinDataTableRowName) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj, class UDataTable* InDataTable);

	UE_API SGraphPinDataTableRowName();
	UE_API virtual ~SGraphPinDataTableRowName();

	// FDataTableEditorUtils::INotifyOnDataTableChanged
	UE_API virtual void PreChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info) override;
	UE_API virtual void PostChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info) override;

protected:

	UE_API void RefreshNameList();

	TSoftObjectPtr<class UDataTable> DataTable;
};

#undef UE_API
