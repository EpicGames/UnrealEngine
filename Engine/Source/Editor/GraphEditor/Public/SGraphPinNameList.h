// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "SNameComboBox.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class FName;
class SWidget;
class UEdGraphPin;

class SGraphPinNameList : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinNameList) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj, const TArray<TSharedPtr<FName>>& InNameList);

protected:

	/**
	 *	Function to create class specific widget.
	 *
	 *	@return Reference to the newly created widget object
	 */
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;

	/**
	 *	Function to set the newly selected index
	 *
	 * @param NewSelection The newly selected item in the combo box
	 * @param SelectInfo Provides context on how the selection changed
	 */
	UE_API void ComboBoxSelectionChanged(TSharedPtr<FName> StringItem, ESelectInfo::Type SelectInfo);

	TSharedPtr<class SNameComboBox>	ComboBox;

	/** The actual list of FName values to choose from */
	TArray<TSharedPtr<FName>> NameList;
};

#undef UE_API
