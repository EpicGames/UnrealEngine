// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API GRAPHEDITOR_API

class FText;
class SGraphPanel;
class SWidget;
class UEdGraph;

// This widget provides a fully-zoomed-out preview of a specified graph
class SGraphPreviewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGraphPreviewer)
		: _ShowGraphStateOverlay(true)
	{}
		SLATE_ATTRIBUTE( FText, CornerOverlayText )
		/** Show overlay elements for the graph state such as the PIE and read-only borders and text */
		SLATE_ATTRIBUTE(bool, ShowGraphStateOverlay)
		SLATE_ARGUMENT( TSharedPtr<SWidget>, TitleBar )
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraph* InGraphObj);

protected:
	/** Delegate handler for graph panel updates */
	UE_API void OnUpdateGraphPanel();

private:
	/** Helper function used to refresh the graph */
	UE_API EActiveTimerReturnType RefreshGraphTimer(const double InCurrentTime, const float InDeltaTime);

protected:
	/// The Graph we are currently viewing
	UEdGraph* EdGraphObj;

	// As node's bounds dont get updated immediately, to truly zoom out to fit we need to tick a few times
	int32 NeedsRefreshCounter;

	// The underlying graph panel
	TSharedPtr<SGraphPanel> GraphPanel;
};

#undef UE_API
