// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API GRAPHEDITOR_API

class SWidget;

//#include "UnrealString.h"

/**
 * A combo box that shows FName content.
 */
class SNameComboBox : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(FString, FGetNameComboLabel, TSharedPtr<FName>);
	typedef TSlateDelegates< TSharedPtr<FName> >::FOnSelectionChanged FOnNameSelectionChanged;

	SLATE_BEGIN_ARGS( SNameComboBox ) 
		: _ComboBoxStyle(&FCoreStyle::Get().GetWidgetStyle< FComboBoxStyle >("ComboBox"))
		, _ColorAndOpacity( FSlateColor::UseForeground() )
		, _ContentPadding(FMargin(4.0, 2.0))
		, _OnGetNameLabelForItem()
		{}

		SLATE_STYLE_ARGUMENT(FComboBoxStyle, ComboBoxStyle)

		/** Selection of FNames to pick from */
		SLATE_ARGUMENT( TArray< TSharedPtr<FName> >*, OptionsSource )

		/** Sets the font used to draw the text */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** Text color and opacity */
		SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

		/** Visual padding of the button content for the combobox */
		SLATE_ATTRIBUTE( FMargin, ContentPadding )

		/** Called when the FName is chosen. */
		SLATE_EVENT( FOnNameSelectionChanged, OnSelectionChanged)

		/** Called when the combo box is opened */
		SLATE_EVENT( FOnComboBoxOpening, OnComboBoxOpening )

		/** Called when combo box needs to establish selected item */
		SLATE_ARGUMENT( TSharedPtr<FName>, InitiallySelectedItem )

		/** [Optional] Called to get the label for the currently selected item */
		SLATE_EVENT( FGetNameComboLabel, OnGetNameLabelForItem ) 
	SLATE_END_ARGS()

	UE_API void Construct( const FArguments& InArgs );

	/** Called to create a widget for each FName */
	UE_API TSharedRef<SWidget> MakeItemWidget( TSharedPtr<FName> StringItem );

	UE_API void SetSelectedItem (TSharedPtr<FName> NewSelection);

	/** Returns the currently selected FName */
	TSharedPtr<FName> GetSelectedItem()
	{
		return SelectedItem;
	}

	/** Request to reload the name options in the combobox from the OptionsSource attribute */
	UE_API void RefreshOptions();

	/** Clears the selected item in the name combo */
	UE_API void ClearSelection();

private:
	TSharedPtr<FName> OnGetSelection() const {return SelectedItem;}

	/** Called when selection changes in the combo pop-up */
	UE_API void OnSelectionChanged(TSharedPtr<FName> Selection, ESelectInfo::Type SelectInfo);

	/** Helper method to get the text for a given item in the combo box */
	UE_API FText GetSelectedNameLabel() const;

	UE_API FText GetItemNameLabel(TSharedPtr<FName> StringItem) const;

private:

	/** Called to get the text label for an item */
	FGetNameComboLabel GetTextLabelForItem;

	/** The FName item selected */
	TSharedPtr<FName> SelectedItem;

	/** Array of shared pointers to FNames so combo widget can work on them */
	TArray< TSharedPtr<FName> >		Names;

	/** The combo box */
	TSharedPtr< SComboBox< TSharedPtr<FName> > >	NameCombo;

	/** Forwarding Delegate */
	FOnNameSelectionChanged SelectionChanged;

	/** Sets the font used to draw the text */
	TAttribute< FSlateFontInfo > Font;
};

#undef UE_API
