// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DiffResults.h"
#include "Framework/Commands/InputChord.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Geometry.h"
#include "Layout/LayoutUtils.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "MarqueeOperation.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Types/PaintArgs.h"
#include "Types/SlateEnums.h"
#include "Types/SlateVector2.h"
#include "Types/WidgetMouseEventsDelegate.h"
#include "UObject/GCObject.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"
#include "Widgets/SWidget.h"
#include "SNodePanel.generated.h"

#define UE_API GRAPHEDITOR_API

class FActiveTimerHandle;
class FReferenceCollector;
class FScopedTransaction;
class FSlateWindowElementList;
class FWidgetStyle;
class UObject;
struct FCaptureLostEvent;
struct FDiffSingleResult;
struct FFocusEvent;
struct FKeyEvent;
struct FMarqueeOperation;
struct FPointerEvent;
struct FSlateBrush;
struct Rect;

//@TODO: Too generic of a name to expose at this scope
typedef class UObject* SelectedItemType;

// Level of detail for graph rendering (lower numbers are 'further away' with fewer details)
namespace EGraphRenderingLOD
{
	enum Type
	{
		// Detail level when zoomed all the way out (all performance optimizations enabled)
		LowestDetail,

		// Detail level that text starts being disabled because it is unreadable
		LowDetail,

		// Detail level at which text starts to get hard to read but is still drawn
		MediumDetail,

		// Detail level when zoomed in at 1:1
		DefaultDetail,

		// Detail level when fully zoomed in (past 1:1)
		FullyZoomedIn,
	};
}

UENUM()
enum class EGraphZoomLimitHandling : uint8
{
	/** By default, hold down Ctrl to break past the zoom limit. */
	Default,
	/** Always allow breaking through the zoom limit. */
	AllowLimitBreak,
	/** Never allow breaking through the zoom limit. */
	DisallowLimitBreak
};

// Context passed in when getting popup info
struct FNodeInfoContext
{
public:
	bool bSelected;
};

// Entry for an overlay brush in the node panel
struct FOverlayBrushInfo
{
public:
	/** Brush to draw */
	const FSlateBrush* Brush;
	/** Scale of animation to apply */
	FDeprecateSlateVector2D AnimationEnvelope;
	/** Offset origin of the overlay from the widget */
	FDeprecateSlateVector2D OverlayOffset;

public:
	FOverlayBrushInfo()
		: Brush(NULL)
		, AnimationEnvelope(0.0f, 0.0f)
		, OverlayOffset(0.f, 0.f)
	{
	}

	FOverlayBrushInfo(const FSlateBrush* InBrush)
		: Brush(InBrush)
		, AnimationEnvelope(0.0f, 0.0f)
		, OverlayOffset(0.f, 0.f)
	{
	}

	FOverlayBrushInfo(const FSlateBrush* InBrush, float HorizontalBounce)
		: Brush(InBrush)
		, AnimationEnvelope(HorizontalBounce, 0.0f)
		, OverlayOffset(0.f, 0.f)
	{
	}
};

// Entry for an overlay widget in the node panel
struct FOverlayWidgetInfo
{
public:
	/** Widget to use */
	TSharedPtr<SWidget> Widget;

	/** Offset origin of the overlay from the widget */
	FDeprecateSlateVector2D OverlayOffset;

public:
	FOverlayWidgetInfo()
		: Widget(nullptr)
		, OverlayOffset(0.f, 0.f)
	{
	}

	FOverlayWidgetInfo(TSharedPtr<SWidget> InWidget)
		: Widget(InWidget)
		, OverlayOffset(0.f, 0.f)
	{
	}
};

// Entry for an information popup in the node panel
struct FGraphInformationPopupInfo
{
public:
	const FSlateBrush* Icon;
	FLinearColor BackgroundColor;
	FString Message;
public:
	FGraphInformationPopupInfo(const FSlateBrush* InIcon, FLinearColor InBackgroundColor, const FString& InMessage)
		: Icon(InIcon)
		, BackgroundColor(InBackgroundColor)
		, Message(InMessage)
	{
	}
};

/**
 * Interface for ZoomLevel values
 * Provides mapping for a range of virtual ZoomLevel values to actual node scaling values
 */
struct FZoomLevelsContainer
{
	/** 
	 * @param InZoomLevel virtual zoom level value
	 * 
	 * @return associated scaling value
	 */
	virtual float						GetZoomAmount(int32 InZoomLevel) const = 0;
	
	/** 
	 * @param InZoomAmount scaling value
	 * 
	 * @return nearest ZoomLevel mapping for provided scale value
	 */
	virtual int32						GetNearestZoomLevel(float InZoomAmount) const = 0;
	
	/** 
	 * @param InZoomLevel virtual zoom level value
	 * 
	 * @return associated friendly name
	 */
	virtual FText						GetZoomText(int32 InZoomLevel) const = 0;
	
	/** 
	 * @return count of supported zoom levels
	 */
	virtual int32						GetNumZoomLevels() const = 0;
	
	/** 
	 * @return the optimal(1:1) zoom level value, default zoom level for the graph
	 */
	virtual int32						GetDefaultZoomLevel() const = 0;
	
	/** 
	 * @param InZoomLevel virtual zoom level value
	 *
	 * @return associated LOD value
	 */
	virtual EGraphRenderingLOD::Type	GetLOD(int32 InZoomLevel) const = 0;

	virtual EGraphZoomLimitHandling GetZoomLimitHandling() const { return EGraphZoomLimitHandling::Default; }
	
	// Necessary for Mac OS X to compile 'delete <pointer_to_this_object>;'
	virtual ~FZoomLevelsContainer( void ) = default;
};

struct FGraphSelectionManager : public FGCObject
{
	TSet<TObjectPtr<class UObject>> SelectedNodes;

	/** Invoked when the selected graph nodes have changed. */
	SGraphEditor::FOnSelectionChanged OnSelectionChanged;
public:
	/** @return the set of selected nodes */
	UE_API const FGraphPanelSelectionSet& GetSelectedNodes() const;

	/** Select just the specified node */
	UE_API void SelectSingleNode(SelectedItemType Node);

	/** Reset the selection state of all nodes */
	UE_API void ClearSelectionSet();

	/** Returns true if any nodes are selected */
	bool AreAnyNodesSelected() const
	{
		return SelectedNodes.Num() > 0;
	}

	/** Changes the selection set to contain exactly all of the passed in nodes */
	UE_API void SetSelectionSet(FGraphPanelSelectionSet& NewSet);

	/**
	 * Add or remove a node from the selection set
	 *
	 * @param Node      Node the affect.
	 * @param bSelect   true to select the node; false to unselect.
	 */
	UE_API void SetNodeSelection(SelectedItemType Node, bool bSelect);

	/** @return true if Node is selected; false otherwise */
	UE_API bool IsNodeSelected(SelectedItemType Node) const;

	// Handle the selection mechanics of starting to drag a node
	UE_API void StartDraggingNode(SelectedItemType NodeBeingDragged, const FPointerEvent& MouseEvent);

	// Handle the selection mechanics when a node is clicked on
	UE_API void ClickedOnNode(SelectedItemType Node, const FPointerEvent& MouseEvent);

	UE_API void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FGraphSelectionManager");
	}
};

/**
 * This class is designed to serve as the base class for a panel/canvas that contains interactive widgets
 * which can be selected and moved around by the user.  It also manages zooming and panning, allowing a larger
 * virtual space to be used for the widget placement.
 *
 * The user is responsible for creating widgets (which must be derived from SNode) and any custom drawing
 * code desired.  The other main restriction is that each SNode instance must have a unique UObject* associated
 * with it.
 */
namespace ENodeZone
{
	enum Type
	{
		TopLeft,
		TopCenter,
		TopRight,

		Left,
		Center,
		Right,

		BottomLeft,
		BottomCenter,
		BottomRight,

		Count
	};
}

template <>
struct TWidgetTypeTraits<class SNodePanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SNodePanel : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SNodePanel, SPanel, UE_API)
public:

	class SNode : public SPanel
	{
		SLATE_DECLARE_WIDGET_API(SNode, SPanel, UE_API)
	public:

		/** A slot that support alignment of content and padding and z-order */
		class FNodeSlot : public TBasicLayoutWidgetSlot<FNodeSlot>
		{
		public:
			friend SNode;

			FNodeSlot()
				: FNodeSlot(ENodeZone::TopLeft)
			{ }

			FNodeSlot(ENodeZone::Type InZone)
				: TBasicLayoutWidgetSlot<FNodeSlot>()
				, Zone(InZone)
				, Offset(*this, FVector2f::ZeroVector)
				, Size(*this, FVector2f::ZeroVector)
				, AllowScale(*this, true)
				, bSizeAttrIsSet(false)
			{
			}

			SLATE_SLOT_BEGIN_ARGS(FNodeSlot, TBasicLayoutWidgetSlot<FNodeSlot>)
				SLATE_ATTRIBUTE_DEPRECATED(FVector2D, SlotOffset, 5.6, "Use SlotOffset2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
				SLATE_ATTRIBUTE(FVector2f, SlotOffset2f)
				SLATE_ATTRIBUTE_DEPRECATED(FVector2D, SlotSize, 5.6, "UseSlotSize2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
				SLATE_ATTRIBUTE(FVector2f, SlotSize2f)
				SLATE_ATTRIBUTE(bool, AllowScaling)
			SLATE_SLOT_END_ARGS()

			UE_API void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs);
			static UE_API void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer);

		public:
			ENodeZone::Type GetZoneType() const
			{
				return Zone;
			}

			UE_DEPRECATED(5.6, "This exists as a temporary bridge for deprecation of SlotSize Attribute. Please use the FVector2f version.")
			void SetSlotOffset(TAttribute<FVector2D> InOffset)
			{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Offset_Deprecated = MoveTemp(InOffset);
				SetSlotOffset(TAttribute<FVector2f>::CreateLambda([this]()
				{
					return UE::Slate::CastToVector2f(Offset_Deprecated.Get());
				}));
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			void SetSlotOffset(TAttribute<FVector2f> InOffset)
			{
				Offset.Assign(*this, MoveTemp(InOffset));
			}

			UE::Slate::FDeprecateVector2DResult GetSlotOffset() const
			{
				return Offset.Get();
			}

			UE_DEPRECATED(5.6, "This exists as a temporary bridge for deprecation of SlotSize Attribute. Please use the FVector2f version.")
			void SetSlotSize(TAttribute<FVector2D> InSize)
			{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Size_Deprecated = MoveTemp(InSize);
				SetSlotSize(TAttribute<FVector2f>::CreateLambda([this]()
				{
					return UE::Slate::CastToVector2f(Size_Deprecated.Get());
				}));
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			void SetSlotSize(TAttribute<FVector2f> InSize)
			{
				bSizeAttrIsSet = InSize.IsSet();
				Size.Assign(*this, MoveTemp(InSize));
			}

			UE::Slate::FDeprecateVector2DResult GetSlotSize() const
			{
				return Size.Get();
			}

			void SetAllowScaling(TAttribute<bool> InAllowScaling)
			{
				AllowScale.Assign(*this, MoveTemp(InAllowScaling));
			}

			bool GetAllowScaling() const
			{
				return AllowScale.Get();
			}

		private:
			/** The child widget contained in this slot. */
			ENodeZone::Type Zone;
			TSlateSlotAttribute<FVector2f> Offset;
			TSlateSlotAttribute<FVector2f> Size;
			UE_DEPRECATED(5.6, "This exists as a temporary bridge for deprecation of SlotSize Attribute.")
			TAttribute<FVector2D> Size_Deprecated;
			UE_DEPRECATED(5.6, "This exists as a temporary bridge for deprecation of SlotSize Attribute.")
			TAttribute<FVector2D> Offset_Deprecated;
			TSlateSlotAttribute<bool> AllowScale;
			bool bSizeAttrIsSet;
		};

		typedef TSet<TWeakPtr<SNodePanel::SNode>> FNodeSet;

		//~ Begin SWidget Interface
		virtual void OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override
		{
			SPanel::OnMouseEnter(InGeometry, InMouseEvent);

			if (TSharedPtr<SNodePanel> ParentPanel = GetParentPanel())
			{
				ParentPanel->SetCurrentHoveredNode(GetObjectBeingDisplayed());
			}
		}

		virtual void OnMouseLeave(const FPointerEvent& InMouseEvent) override
		{
			SPanel::OnMouseLeave(InMouseEvent);

			if (TSharedPtr<SNodePanel> ParentPanel = GetParentPanel())
			{
				UObject* PreviousHoveredNode = ParentPanel->GetCurrentHoveredNode();
				
				// Only unset the current hovered node on mouse leave if its this node
				if (PreviousHoveredNode == GetObjectBeingDisplayed())
				{
					ParentPanel->SetCurrentHoveredNode(nullptr);
				}
			}
		}
		//~ End SWidget Interface

		// SPanel Interface
		virtual FChildren* GetChildren() override
		{
			return &Children;
		}

		virtual FVector2D ComputeDesiredSize(float) const override
		{
			for( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
			{
				if( Children[ ChildIndex ].Zone == ENodeZone::Center )
				{
					const FNodeSlot& CenterZone = Children[ ChildIndex ];
					const EVisibility ChildVisibility = CenterZone.GetWidget()->GetVisibility();
					if( ChildVisibility != EVisibility::Collapsed )
					{
						return ( FVector2D(CenterZone.GetWidget()->GetDesiredSize() + CenterZone.GetPadding().GetDesiredSize()) ) * FVector2D(GetDesiredSizeScale());
					}
				}
			}
			return FVector2D::ZeroVector;
		}

		virtual float GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const override
		{
			const FNodeSlot& ThisSlot = Children[ChildIndex];
			if (!ThisSlot.AllowScale.Get())
			{
				// Child slots that do not allow zooming should scale themselves to negate the node panel's zoom.
				TSharedPtr<SNodePanel> ParentPanel = GetParentPanel();
				if (ParentPanel.IsValid())
				{
					return 1.0f / ParentPanel->GetZoomAmount();
				}
			}

			return 1.0f;
		}

		virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override
		{
			for( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
			{
				const FNodeSlot& CurChild = Children[ChildIndex];
				const EVisibility ChildVisibility = CurChild.GetWidget()->GetVisibility();
				if ( ArrangedChildren.Accepts(ChildVisibility) )
				{
					const FMargin SlotPadding(CurChild.GetPadding());
					// If this child is not allowed to scale, its scale relative to its parent should undo the parent widget's scaling.
					FVector2f Size;

					if( CurChild.bSizeAttrIsSet )
					{
						Size = CurChild.Size.Get();
					}
					else
					{
						AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(AllottedGeometry.GetLocalSize().X, CurChild, SlotPadding);
						AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, CurChild, SlotPadding);
						Size = FVector2f( XResult.Size, YResult.Size );
					}
					const FArrangedWidget ChildGeom =
						AllottedGeometry.MakeChild(
						CurChild.GetWidget(),
						CurChild.Offset.Get(),
						Size,
						GetRelativeLayoutScale(ChildIndex, AllottedGeometry.Scale)
					);
					ArrangedChildren.AddWidget( ChildVisibility, ChildGeom );
				}
			}
		}

		virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
		{
			FArrangedChildren ArrangedChildren( EVisibility::Visible );
			{
				ArrangeChildren( AllottedGeometry, ArrangedChildren );
			}

			int32 MaxLayerId = LayerId;
			for( int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex )
			{
				const FArrangedWidget& CurWidget = ArrangedChildren[ ChildIndex ];

				if (!IsChildWidgetCulled(MyCullingRect, CurWidget))
				{
					const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(Args.WithNewParent(this), CurWidget.Geometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));
					MaxLayerId = FMath::Max(MaxLayerId, CurWidgetsMaxLayerId);
				}
				else
				{
					//SlateGI - RemoveContent
				}
			}

			return MaxLayerId;
		}
		// End of SPanel Interface


		using FScopedWidgetSlotArguments = TPanelChildren<FNodeSlot>::FScopedWidgetSlotArguments;
		FScopedWidgetSlotArguments GetOrAddSlot( const ENodeZone::Type SlotId )
		{
			// Return existing
			int32 InsertIndex = INDEX_NONE;
			for( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
			{
				if( Children[ ChildIndex ].Zone == SlotId )
				{
					Children.RemoveAt(ChildIndex);
					InsertIndex = ChildIndex;
				}
			}

			// Add new
			return FScopedWidgetSlotArguments{ MakeUnique<FNodeSlot>(SlotId), Children, InsertIndex };
		}

		FNodeSlot* GetSlot( const ENodeZone::Type SlotId )
		{
			FNodeSlot* Result = nullptr;
			// Return existing
			for( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
			{
				if( Children[ ChildIndex ].Zone == SlotId )
				{
					Result = &Children[ ChildIndex ];
					break;
				}
			}
			return Result;
		}

		void RemoveSlot( const ENodeZone::Type SlotId )
		{
			for( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
			{
				if( Children[ ChildIndex ].Zone == SlotId )
				{
					Children.RemoveAt( ChildIndex );
					break;
				}
			}
		}

		/**
		* @param NewPosition	The Node should be relocated to this position in the graph panel
		* @param NodeFilter		Set of nodes to prevent movement on, after moving successfully a node is added to this set.
		* @param bMarkDirty		If we should mark nodes as dirty on move
		*/
		UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
		virtual void MoveTo(const FVector2D& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
		{
			MoveTo(UE::Slate::CastToVector2f(NewPosition), NodeFilter, bMarkDirty);
		}
		virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true)
		{
		}

		/** @return the Node's position within the graph */
		UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
		virtual FVector2D GetPosition() const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
		{
			return FVector2D(GetPosition2f());
		}
		virtual FVector2f GetPosition2f() const
		{
			return FVector2f::ZeroVector;
		}

		/** @return a user-specified comment on this node; the comment gets drawn in a bubble above the node */
		virtual FString GetNodeComment() const
		{
			return FString();
		}

		/** @return The backing object, used as a unique identifier in the selection set, etc... */
		virtual UObject* GetObjectBeingDisplayed() const
		{
			return NULL;
		}

		/** @return The brush to use for drawing the shadow for this node */
		virtual const FSlateBrush* GetShadowBrush(bool bSelected) const
		{
			return bSelected ? FAppStyle::GetBrush(TEXT("Graph.Node.ShadowSelected")) : FAppStyle::GetBrush(TEXT("Graph.Node.Shadow"));
		}

		struct DiffHighlightInfo
		{
			const FSlateBrush* Brush;
			const FLinearColor Tint;
		};
		
		/** @return Collection of brushes layered to outline node with the DiffResult color */
		TArray<SNodePanel::SNode::DiffHighlightInfo> GetDiffHighlights(const FDiffSingleResult& DiffResult) const;

		/** used by GetDiffHighlights to generate outlines for diffed nodes */
		virtual void GetDiffHighlightBrushes(const FSlateBrush*& BackgroundOut, const FSlateBrush*& ForegroundOut) const
		{
			BackgroundOut = FAppStyle::GetBrush(TEXT("Graph.Node.DiffHighlight"));
			ForegroundOut = FAppStyle::GetBrush(TEXT("Graph.Node.DiffHighlightShading"));
		}

		/** Populate the brushes array with any overlay brushes to render */
		UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
		virtual void GetOverlayBrushes(bool bSelected, const FVector2D WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
		{
			GetOverlayBrushes(bSelected, UE::Slate::CastToVector2f(WidgetSize), Brushes);
		}
		virtual void GetOverlayBrushes(bool bSelected, const FVector2f& WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const
		{
		}

		/** Populate the widgets array with any overlay widgets to render */
		UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
		virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2D& WidgetSize) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
		{
			return GetOverlayWidgets(bSelected, UE::Slate::CastToVector2f(WidgetSize));
		}
		virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
		{
			return TArray<FOverlayWidgetInfo>();
		}

		/** Populate the popups array with any popups to render */
		virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
		{
		}	

		/** Returns true if this node is dependent on the location of other nodes (it can only depend on the location of first-pass only nodes) */
		virtual bool RequiresSecondPassLayout() const
		{
			return false;
		}

		/** Performs second pass layout; only called if RequiresSecondPassLayout returned true */
		virtual void PerformSecondPassLayout(const TMap< UObject*, TSharedRef<SNode> >& InNodeToWidgetLookup) const
		{
		}

		/** Return false if this node should not be culled. Useful for potentially large nodes that may be improperly culled. */
		virtual bool ShouldAllowCulling() const
		{
			return true;
		}

		/** return if the node can be selected, by pointing given location */
		UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
		virtual bool CanBeSelected(const FVector2D& MousePositionInNode) const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
		{
			return CanBeSelected(UE::Slate::CastToVector2f(MousePositionInNode));
		}
		virtual bool CanBeSelected(const FVector2f& MousePositionInNode) const
		{
			return true;
		}

		/** Called when user interaction has completed */
		virtual void EndUserInteraction() const {}

		/** 
		 *	override, when area used to select node, should be different, than it's size
		 *	e.g. comment node - only title bar is selectable
		 *	return size of node used for Marquee selecting
		 */
		UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
		virtual FVector2D GetDesiredSizeForMarquee() const UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
		{
			return FVector2D(GetDesiredSizeForMarquee2f());
		}
		virtual FVector2f GetDesiredSizeForMarquee2f() const
		{
			return UE::Slate::CastToVector2f(GetDesiredSize());
		}

		// Returns node sort depth, defaults to and is generally 0 for most nodes
		virtual int32 GetSortDepth() const { return 0; }

		// Node Sort Operator
		bool operator < ( const SNodePanel::SNode& NodeIn ) const
		{
			return GetSortDepth() < NodeIn.GetSortDepth();
		}

		void SetParentPanel(const TSharedPtr<SNodePanel>& InParent)
		{
			ParentPanelPtr = InParent;
		}

	protected:
		SNode()
		: DesiredSizeScaleAttribute(*this, FVector2f::UnitVector)
		, Children(this)
		{
			bHasRelativeLayoutScale = true;
		}

	protected:
		void SetDesiredSizeScale(TAttribute<FDeprecateSlateVector2D> InDesiredSizeScale)
		{
			DesiredSizeScaleAttribute.Assign(*this, MoveTemp(InDesiredSizeScale));
		}
		TSlateAttributeRef<FDeprecateSlateVector2D> GetDesiredSizeScaleAttribute() const
		{
			return TSlateAttributeRef<FDeprecateSlateVector2D>{SharedThis(this), DesiredSizeScaleAttribute};
		}
		FDeprecateSlateVector2D GetDesiredSizeScale() const
		{
			return DesiredSizeScaleAttribute.Get();
		}

	protected:

		UE_DEPRECATED(5.8, "Use SetDesiredSizeScale / GetDesiredSizeScale / GetDesiredSizeScaleAttribute")
		TSlateDeprecatedTAttribute<FDeprecateSlateVector2D> DesiredSizeScale;

		// SBorder Begin
		UE_DEPRECATED(5.8, "The attribute was unused and can be removed.")
		TSlateDeprecatedTAttribute<const FSlateBrush*> BorderImage;
		UE_DEPRECATED(5.8, "The attribute was unused and can be removed.")
		TSlateDeprecatedTAttribute<FSlateColor> BorderBackgroundColor;

		/** Whether or not to show the disabled effect when this border is disabled */
		UE_DEPRECATED(5.8, "The attribute was unused and can be removed.")
		TSlateDeprecatedTAttribute<bool> ShowDisabledEffect;
		/** Mouse event handlers */
		UE_DEPRECATED(5.8, "The EventHandler was unused and can be removed.")
		FPointerEventHandler MouseButtonDownHandler;
		UE_DEPRECATED(5.8, "The EventHandler was unused and can be removed.")
		FPointerEventHandler MouseButtonUpHandler;
		UE_DEPRECATED(5.8, "The EventHandler was unused and can be removed.")
		FPointerEventHandler MouseMoveHandler;
		UE_DEPRECATED(5.8, "The EventHandler was unused and can be removed.")
		FPointerEventHandler MouseDoubleClickHandler;
		// SBorder End

		// SPanel Begin
		/** The layout scale to apply to this widget's contents; useful for animation. */
		UE_DEPRECATED(5.8, "The attribute was unused and can be removed.")
		TSlateDeprecatedTAttribute<FDeprecateSlateVector2D> ContentScale;
		/** The color and opacity to apply to this widget and all its descendants. */
		UE_DEPRECATED(5.8, "The attribute was unused and can be removed.")
		TSlateDeprecatedTAttribute<FLinearColor> ColorAndOpacity;
		/** Optional foreground color that will be inherited by all of this widget's contents */
		UE_DEPRECATED(5.8, "The attribute was unused and can be removed.")
		TSlateDeprecatedTAttribute<FSlateColor> ForegroundColor;
		// SPanel End

	private:
		TSlateAttribute<FDeprecateSlateVector2D> DesiredSizeScaleAttribute;

	private:

		TSharedPtr<SNodePanel> GetParentPanel() const
		{
			return ParentPanelPtr.Pin();
		}

		TPanelChildren<FNodeSlot> Children;
		TWeakPtr<SNodePanel> ParentPanelPtr;

	};

	UE_API SNodePanel();

	// SPanel interface
	UE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	UE_DEPRECATED(5.8, "Use GetVisibleChildren for visible children, or GetManagedChildren to get all the nodes of this panel.")
	UE_API virtual FChildren* GetChildren() override;
	// End of SPanel interface

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	UE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UE_API virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UE_API virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	UE_API virtual FReply OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& GestureEvent ) override;
	UE_API virtual FReply OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent ) override;
	UE_API virtual float GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const override;
	// End of SWidget interface
public:
	/**
	 * Is the given node being observed by a widget in this panel?
	 * 
	 * @param Node   The node to look for.
	 *
	 * @return True if the node is being observed by some widget in this panel; false otherwise.
	 */
	UE_API bool Contains(UObject* Node) const;

	/** @retun the zoom amount; e.g. a value of 0.25f results in quarter-sized nodes */
	UE_API float GetZoomAmount() const;
	/** @return Zoom level as a pretty string */
	UE_API FText GetZoomText() const;
	UE_API FSlateColor GetZoomTextColorAndOpacity() const;

	/** @return the view offset in graph space */
	UE_API UE::Slate::FDeprecateVector2DResult GetViewOffset() const;

	/** Sets the current view offset */
	void SetViewOffset(const UE::Slate::FDeprecateVector2DResult& InViewOffset) { ViewOffsetAttribute.Assign(*this, InViewOffset); }

	/** 
	 * when a panel is scrolling/zooming to a target, this can be called to get it's destination
	 * @param TopLeft top left corner of the destination
	 * @param BottomRight bottom right corner of the destination
	 * @return true if there's a scrolling/zooming target and false if there is no destination
	 */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API bool GetZoomTargetRect(FVector2D& TopLeft, FVector2D& BottomRight) const;
	UE_API bool GetZoomTargetRect(FVector2f& TopLeft, FVector2f& BottomRight) const;

	/** @return the current zoom level */
	int32 GetZoomLevel() const { return ZoomLevel; }

	/** Sets the current zoom level */
	UE_API void SetZoomLevel(int32 InZoomLevel);

	/** @return the current view bookmark ID */
	const FGuid& GetViewBookmarkId() const { return CurrentBookmarkGuid; }

	/** Given a coordinate in panel space (i.e. panel widget space), return the same coordinate in graph space while taking zoom and panning into account */
	UE_API UE::Slate::FDeprecateVector2DResult PanelCoordToGraphCoord(const UE::Slate::FDeprecateVector2DParameter& PanelSpaceCoordinate) const;

	/** Restore the graph panel to the supplied view offset/zoom. Also, optionally set the current bookmark ID. */
	UE_API void RestoreViewSettings(const UE::Slate::FDeprecateVector2DParameter& InViewOffset, float InZoomAmount, const FGuid& InBookmarkGuid = FGuid());

	/** Get the grid snap size */
	static UE_API uint32 GetSnapGridSize();

	/** 
	 * Zooms out to fit either all nodes or only the selected ones.
	 * @param bOnlySelection Whether to zoom to fit around only the current selection (if false, will zoom to the extents of all nodes)
	 */
	UE_API void ZoomToFit(bool bOnlySelection);

	/** Get the bounding area for the currently selected nodes 
		@return false if nothing is selected					*/
	UE_API bool GetBoundsForSelectedNodes(/*out*/ class FSlateRect& Rect, float Padding = 0.0f);

	/** @return the position where where nodes should be pasted (i.e. from the clipboard) */
	UE_API UE::Slate::FDeprecateVector2DResult GetPastePosition() const;

	/** Ask panel to scroll to location */
	UE_API void RequestDeferredPan(const UE::Slate::FDeprecateVector2DParameter& TargetPosition);

	/** If it is focusing on a particular object */
	UE_API bool HasDeferredObjectFocus() const;

	/** Query whether this graph is about to start panning/zooming towards a destination */
	UE_API bool HasDeferredZoomDestination() const;

	/** Commit transactions for any node movements */
	UE_API void FinalizeNodeMovements();

	/** Returns the current LOD level of this panel, based on the zoom factor */
	EGraphRenderingLOD::Type GetCurrentLOD() const { return CurrentLOD; }

	/** Returns if the panel has been panned or zoomed since the last update */
	UE_API bool HasMoved() const;

	/** Returns only the visible children as this is the function called by the invalidation system */
	UE_DEPRECATED(5.8, "Use GetVisibleChildren for visible children, or GetManagedChildren to get all the nodes of this panel. GetAllChildren needs to return only the VisibleChildren to work with invalidation.")
	UE_API virtual FChildren* GetAllChildren() override;

	/** Returns all the visible panel children */
	UE_API FChildren* GetVisibleChildren();
	UE_API FChildren* GetManagedChildren();

protected:
	/** Initialize members */
	UE_API void Construct();

	/**
	 * Zooms to the specified target rect
	 * @param TopLeft The top left corner of the target rect
	 * @param BottomRight The bottom right corner of the target rect
	 */
	UE_API void ZoomToTarget(const FVector2f& TopLeft, const FVector2f& BottomRight);

	/** Update the new view offset location  */
	UE_API void UpdateViewOffset(const FGeometry& MyGeometry, const FVector2f& TargetPosition);

	/** Compute much panel needs to change to pan to location  */
	static UE_API FVector2f ComputeEdgePanAmount(const FGeometry& MyGeometry, const FVector2f& MouseEvent);

	/** Given a coordinate in graph space (e.g. a node's position), return the same coordinate in widget space while taking zoom and panning into account */
	UE_API FVector2f GraphCoordToPanelCoord(const UE::Slate::FDeprecateVector2DParameter& GraphSpaceCoordinate) const;

	/** Given a rectangle in panel space, return a rectangle in graph space. */
	UE_API FSlateRect PanelRectToGraphRect(const FSlateRect& PanelSpaceRect) const;

	/** 
	 * Lets the CanvasPanel know that the user is interacting with a node.
	 * 
	 * @param InNodeToDrag   The node that the user wants to drag
	 * @param GrabOffset     Where within the node the user grabbed relative to split between inputs and outputs.
	 */
	UE_API virtual void OnBeginNodeInteraction(const TSharedRef<SNode>& InNodeToDrag, const FVector2D& GrabOffset) UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION;
	UE_API virtual void OnBeginNodeInteraction(const TSharedRef<SNode>& InNodeToDrag, const FVector2f& GrabOffset);

	/** 
	 * Lets the CanvasPanel know that the user has ended interacting with a node.
	 * 
	 * @param InNodeToDrag   The node that the user was to dragging
	 */
	UE_API virtual void OnEndNodeInteraction(const TSharedRef<SNode>& InNodeToDrag);

	/** Figure out which nodes intersect the marquee rectangle */
	UE_API void FindNodesAffectedByMarquee(FGraphPanelSelectionSet& OutAffectedNodes) const;

	/**
	 * Apply the marquee operation to the current selection
	 *
	 * @param InMarquee            The marquee operation to apply.
	 * @param CurrentSelection     The selection before the marquee operation.
	 * @param OutNewSelection      The selection resulting from Marquee being applied to CurrentSelection.
	 */
	static UE_API void ApplyMarqueeSelection(const FMarqueeOperation& InMarquee, const FGraphPanelSelectionSet& CurrentSelection, TSet<TObjectPtr<UObject>>& OutNewSelection);

	/**
	 * On the next tick, centers and selects the widget associated with the object if it exists
	 *
	 * @param ObjectToSelect	The object to select, and potentially center on
	 * @param bCenter			Whether or not to center the graph node
	 */
	UE_API void SelectAndCenterObject(const UObject* ObjectToSelect, bool bCenter);

	/**
	 * On the next tick, centers the widget associated with the object if it exists
	 *
	 * @param ObjectToCenter	The object to center
	 */
	UE_API void CenterObject(const UObject* ObjectToCenter);

	/** Add a slot to the CanvasPanel dynamically */
	UE_API virtual void AddGraphNode(const TSharedRef<SNode>& NodeToAdd);

	/** Remove all nodes from the panel */
	UE_API virtual void RemoveAllNodes();
	
	/** Populate visibile children array */
	UE_API virtual void PopulateVisibleChildren(const FGeometry& AllottedGeometry);

	/** Arrange child nodes - allows derived classes to supply non-node children in OnArrangeChildren */
	UE_API virtual void ArrangeChildNodes(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const;

	// Paint the background as lines
	UE_API void PaintBackgroundAsLines(const FSlateBrush* BackgroundImage, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;

	// Paint the well shadow (around the perimeter)
	UE_API void PaintSurroundSunkenShadow(const FSlateBrush* ShadowImage, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId) const;

	// Paint the marquee selection rectangle
	UE_API void PaintMarquee(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId) const;

	// Paint the software mouse if necessary
	UE_API void PaintSoftwareCursor(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId) const;

	// Paint a comment bubble
	UE_API void PaintComment(const FString& CommentText, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 DrawLayerId, const FLinearColor& CommentTinting, float& HeightAboveNode, const FWidgetStyle& InWidgetStyle ) const;

	/** Determines if a specified node is not visually relevant. */
	UE_API bool IsNodeCulled(const TSharedRef<SNode>& Node, const FGeometry& AllottedGeometry) const;

protected:
	///////////
	// INTERFACE TO IMPLEMENT
	///////////
	/** @return the widget in the summoned context menu that should be focused. */
	virtual TSharedPtr<SWidget> OnSummonContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return TSharedPtr<SWidget>(); }
	virtual bool OnHandleLeftMouseRelease(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return false; }
protected:
	/**
	 * Get the bounds of the given node
	 * @return True if successful
	 */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API bool GetBoundsForNode(const UObject* InNode, /*out*/ FVector2D& MinCorner, /*out*/ FVector2D& MaxCorner, float Padding = 0.0f) const;
	UE_API bool GetBoundsForNode(const UObject* InNode, /*out*/ FVector2f& MinCorner, /*out*/ FVector2f& MaxCorner, float Padding = 0.0f) const;

	/**
	 * Get the bounds of the selected nodes 
	 * @param bSelectionSetOnly If true, limits the query to just the selected nodes.  Otherwise it does all nodes.
	 * @return True if successful
	 */
	UE_DEPRECATED(5.6, "Use the version of the function accepting FVector2f; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
	UE_API bool GetBoundsForNodes(bool bSelectionSetOnly, /*out*/ FVector2D& MinCorner, /*out*/ FVector2D& MaxCorner, float Padding = 0.0f) const;
	UE_API bool GetBoundsForNodes(bool bSelectionSetOnly, /*out*/ FVector2f& MinCorner, /*out*/ FVector2f& MaxCorner, float Padding = 0.0f) const;

	/**
	 * Scroll the view to the desired location 
	 * @return true when the desired location is reached
	 */
	UE_API bool ScrollToLocation(const FGeometry& MyGeometry, FVector2f DesiredCenterPosition, const float InDeltaTime);

	/**
	 * Zoom to fit the desired size 
	 * @return true when zoom fade has completed & fits the desired size
	 */
	UE_API bool ZoomToLocation(const FVector2f& CurrentSizeWithoutZoom, const FVector2f& DesiredSize, bool bDoneScrolling);

	/**
	 * Change zoom level by the specified zoom level delta, about the specified origin.
	 */
	UE_API void ChangeZoomLevel(int32 ZoomLevelDelta, const FVector2f& WidgetSpaceZoomOrigin, bool bOverrideZoomLimiting);

	// Should be called whenever the zoom level has changed
	UE_API void PostChangedZoom();

	// Fires up a per-tick function to zoom the graph to fit
	UE_API void RequestZoomToFit();

	// Cancels any active zoom-to-fit action
	UE_API void CancelZoomToFit();

	/** Called when the graph selection changed */
	UE_API virtual void OnSelectionChanged(const TSet<UObject*>& Objects);

public:
	
	// Sets the zoom levels container
	template<typename T>
	void SetZoomLevelsContainer()
	{
		SetZoomLevelsContainer(MakeShared<T>());
	}

	void SetZoomLevelsContainer(const TSharedPtr<FZoomLevelsContainer>& InZoomLevels)
	{
		ZoomLevels = InZoomLevels;
		OldZoomAmount = InZoomLevels->GetZoomAmount(ZoomLevel);
		ZoomLevel = PreviousZoomLevel = InZoomLevels->GetNearestZoomLevel(OldZoomAmount);
		PostChangedZoom();
	}

	const TSharedPtr<FZoomLevelsContainer>& GetZoomLevels() const { return ZoomLevels; }

	void SetCurrentHoveredNode(UObject* InNode) { CurrentHoveredNode = InNode; }
	UObject* GetCurrentHoveredNode() { return CurrentHoveredNode; }

protected:
	void SetIsEditable(TAttribute<bool> InIsEditableAttribute) { IsEditableAttribute.Assign(*this, MoveTemp(InIsEditableAttribute)); }
	bool GetIsEditable() const { return IsEditableAttribute.Get(); }
	TSlateAttributeRef<bool> GetIsEditableAttribute() const
	{
		return TSlateAttributeRef<bool>{SharedThis(this), IsEditableAttribute};
	}

	TSlateAttributeRef<FVector2f> GetViewOffsetAttribute() const
	{
		return TSlateAttributeRef<FVector2f>{SharedThis(this), ViewOffsetAttribute};
	}

protected:

	// The interface for mapping ZoomLevel values to actual node scaling values
	TSharedPtr<FZoomLevelsContainer> ZoomLevels;

	/** The position within the graph at which the user is looking */
	UE_DEPRECATED(5.8, "Use SetViewOffset / GetViewOffset / GetViewOffsetAttribute")
	FDeprecateSlateVector2D ViewOffset;

	/** The position within the graph at which the user was looking last tick */
	FDeprecateSlateVector2D OldViewOffset;

	/** How zoomed in/out we are. e.g. 0.25f results in quarter-sized nodes. */
	int32 ZoomLevel;

	/** Previous Zoom Level */
	int32 PreviousZoomLevel;

	/** The actual scalar zoom amount last tick */
	float OldZoomAmount;

	/** Are we panning the view at the moment? */
	bool bIsPanning;

	/** Are we zooming the view with trackpad at the moment? */
	bool bIsZoomingWithTrackpad;

	/** The graph node widgets owned by this panel */
	TUnmanagedSlotlessChildren<SNode> Children;
	TSlotlessChildren<SNode> VisibleChildren;

	/** The node that the user is dragging. Null when they are not dragging a node. */
	TWeakPtr<SNode> NodeUnderMousePtr;

	/** Where in the title the user grabbed to initiate the drag */
	FDeprecateSlateVector2D NodeGrabOffset;

	/** The total distance that the mouse has been dragged while down */
	float TotalMouseDelta;

	/** The additive X and Y components of mouse drag (used when zooming) */
	float TotalMouseDeltaXY;

	/** Offset in the panel the user started the LMB+RMB zoom from */
	FDeprecateSlateVector2D ZoomStartOffset;

	/** Cumulative magnify delta from trackpad gesture */
	float TotalGestureMagnify;
public:
	/** Nodes selected in this instance of the editor; the selection is per-instance of the GraphEditor */
	FGraphSelectionManager SelectionManager;

protected:
	/** The currently hovered SNode's UObject */
	TObjectPtr<class UObject> CurrentHoveredNode;

	/** A pending marquee operation if it's active */
	FMarqueeOperation Marquee;

	/** Is the graph editable (can nodes be moved, etc...)? */
	UE_DEPRECATED(5.8, "Use SetIsEditable / GetIsEditable / GetIsEditableAttribute")
	TSlateDeprecatedTAttribute<bool> IsEditable;

	/** Given a node, find the corresponding widget */
	TMap< UObject*, TSharedRef<SNode> > NodeToWidgetLookup;

	/** If not empty and a part of this panel, this node will be selected and brought into view on the next Tick */
	TSet<const UObject*> DeferredSelectionTargetObjects;
	/** If non-null and a part of this panel, this node will be brought into view on the next Tick */
	const UObject* DeferredMovementTargetObject;

	/** Deferred zoom to selected node extents */
	bool bDeferredZoomToSelection;

	/** Deferred zoom to node extents */
	bool bDeferredZoomToNodeExtents;

	/** Zoom selection padding */
	float ZoomPadding;

	/** Allow Zoom overall */
	bool bAllowZoom;

	/** Allow Panning overall */
	bool bAllowPanning;

	/** Allow continous zoom interpolation? */
	bool bAllowContinousZoomInterpolation;

	/** Teleport immediately, or smoothly scroll when doing a deferred zoom */
	bool bTeleportInsteadOfScrollingWhenZoomingToFit;

	/** Fade on zoom for graph */
	FCurveSequence ZoomLevelGraphFade;

	/** Curve that handles fading the 'Zoom +X' text */
	FCurveSequence ZoomLevelFade;

	/** The position where we should paste when a user executes the paste command. */
	FDeprecateSlateVector2D PastePosition;

	/** Position to pan to */
	FDeprecateSlateVector2D DeferredPanPosition;

	/** true if pending request for deferred panning */
	bool bRequestDeferredPan;

	/**	The current position of the software cursor */
	FDeprecateSlateVector2D SoftwareCursorPosition;

	/**	Whether the software cursor should be drawn */
	bool bShowSoftwareCursor;

	/** Current LOD level for nodes/pins */
	EGraphRenderingLOD::Type CurrentLOD;

	/** Invoked when the user may be attempting to spawn a node using a shortcut */
	SGraphEditor::FOnSpawnNodeByShortcutAtLocation OnSpawnNodeByShortcutAtLocation;

	/** The last key chord detected in this graph panel */
	FInputChord LastKeyChordDetected;

	/** The current transaction for undo/redo */
	TSharedPtr<FScopedTransaction> ScopedTransactionPtr;

	/** Cached geometry for use within the active timer */
	FGeometry CachedGeometry;

	/** A flag to detect when a visual update is pending to prevent deferred
	    commands like zoom to fit from running when there are no widgets */
	bool bVisualUpdatePending;
	
	/** Node positions pre-drag, used to limit transaction creation on drag */
	TMap<TWeakPtr<SNode>, FVector2f> OriginalNodePositions;

	/** Called when the user left clicks on a node without dragging */
	SGraphEditor::FOnNodeSingleClicked OnNodeSingleClicked;
	
	/** Implementation detail for node snapping, experimental and not user extensible */
	TPimplPtr<struct FNodeSnappingManager> NodeSnappingManager;
	struct FNodeSnappingManager* GetSnappingManager();
	const struct FNodeSnappingManager* GetSnappingManager() const;

private:
	/** Active timer that handles deferred zooming until the target zoom is reached */
	UE_API EActiveTimerReturnType HandleZoomToFit(double InCurrentTime, float InDeltaTime);

private:
	/** Is the graph editable (can nodes be moved, etc...)? */
	TSlateAttribute<bool> IsEditableAttribute;

	/** The position within the graph at which the user is looking */
	TSlateAttribute<FVector2f> ViewOffsetAttribute;

	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Zoom target rectangle */
	FVector2f ZoomTargetTopLeft;
	FVector2f ZoomTargetBottomRight;

	/** Current view bookmark info */
	FGuid CurrentBookmarkGuid;
};

template <>
struct TWidgetTypeTraits<SNodePanel::SNode>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

#undef UE_API
